// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieScene2DTransformPropertySystem.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include "Animation/MovieSceneUMGComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene2DTransformPropertySystem)


UMovieScene2DTransformPropertySystem::UMovieScene2DTransformPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneUMGComponentTypes::Get()->WidgetTransform);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieScene2DTransformPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}

