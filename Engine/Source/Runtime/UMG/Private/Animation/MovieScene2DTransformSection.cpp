// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieScene2DTransformSection.h"
#include "UObject/SequencerObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Slate/WidgetTransform.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "Animation/MovieScene2DTransformPropertySystem.h"
#include "Animation/MovieScene2DTransformTrack.h"
#include "Animation/MovieSceneUMGComponentTypes.h"
#include "Algo/AnyOf.h"

#include "MovieScene.h"
#include "MovieSceneTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene2DTransformSection)


#if WITH_EDITOR

struct F2DTransformSectionEditorData
{
	F2DTransformSectionEditorData(EMovieScene2DTransformChannel Mask)
	{
		FText TranslationGroup = NSLOCTEXT("MovieScene2DTransformSection", "Translation", "Translation");
		FText RotationGroup = NSLOCTEXT("MovieScene2DTransformSection", "Rotation", "Rotation");
		FText ScaleGroup = NSLOCTEXT("MovieScene2DTransformSection", "Scale", "Scale");
		FText ShearGroup = NSLOCTEXT("MovieScene2DTransformSection", "Shear", "Shear");

		MetaData[0].SetIdentifiers("Translation.X", FCommonChannelData::ChannelX, TranslationGroup);
		MetaData[0].SubPropertyPath = MetaData[0].Name;
		MetaData[0].bEnabled = EnumHasAllFlags(Mask, EMovieScene2DTransformChannel::TranslationX);
		MetaData[0].SortOrder = 0;
		MetaData[0].bCanCollapseToTrack = false;

		MetaData[1].SetIdentifiers("Translation.Y", FCommonChannelData::ChannelY, TranslationGroup);
		MetaData[1].SubPropertyPath = MetaData[1].Name;
		MetaData[1].bEnabled = EnumHasAllFlags(Mask, EMovieScene2DTransformChannel::TranslationY);
		MetaData[1].SortOrder = 1;
		MetaData[1].bCanCollapseToTrack = false;

		MetaData[2].SetIdentifiers("Angle", NSLOCTEXT("MovieScene2DTransformSection", "AngleText", "Angle"), RotationGroup);
		MetaData[2].SubPropertyPath = MetaData[2].Name;
		MetaData[2].bEnabled = EnumHasAllFlags(Mask, EMovieScene2DTransformChannel::Rotation);
		MetaData[2].SortOrder = 2;
		MetaData[2].bCanCollapseToTrack = false;

		MetaData[3].SetIdentifiers("Scale.X", FCommonChannelData::ChannelX, ScaleGroup);
		MetaData[3].SubPropertyPath = MetaData[3].Name;
		MetaData[3].bEnabled = EnumHasAllFlags(Mask, EMovieScene2DTransformChannel::ScaleX);
		MetaData[3].SortOrder = 3;
		MetaData[3].bCanCollapseToTrack = false;
		MetaData[3].bCanPreserveRatio = true;

		MetaData[4].SetIdentifiers("Scale.Y", FCommonChannelData::ChannelY, ScaleGroup);
		MetaData[4].SubPropertyPath = MetaData[4].Name;
		MetaData[4].bEnabled = EnumHasAllFlags(Mask, EMovieScene2DTransformChannel::ScaleY);
		MetaData[4].SortOrder = 4;
		MetaData[4].bCanCollapseToTrack = false;
		MetaData[4].bCanPreserveRatio = true;

		MetaData[5].SetIdentifiers("Shear.X", FCommonChannelData::ChannelX, ShearGroup);
		MetaData[5].SubPropertyPath = MetaData[5].Name;
		MetaData[5].bEnabled = EnumHasAllFlags(Mask, EMovieScene2DTransformChannel::ShearX);
		MetaData[5].SortOrder = 5;
		MetaData[5].bCanCollapseToTrack = false;

		MetaData[6].SetIdentifiers("Shear.Y", FCommonChannelData::ChannelY, ShearGroup);
		MetaData[6].SubPropertyPath = MetaData[6].Name;
		MetaData[6].bEnabled = EnumHasAllFlags(Mask, EMovieScene2DTransformChannel::ShearY);
		MetaData[6].SortOrder = 6;
		MetaData[6].bCanCollapseToTrack = false;

		ExternalValues[0].OnGetExternalValue = ExtractTranslationX;
		ExternalValues[1].OnGetExternalValue = ExtractTranslationY;
		ExternalValues[2].OnGetExternalValue = ExtractRotation;
		ExternalValues[3].OnGetExternalValue = ExtractScaleX;
		ExternalValues[4].OnGetExternalValue = ExtractScaleY;
		ExternalValues[5].OnGetExternalValue = ExtractShearX;
		ExternalValues[6].OnGetExternalValue = ExtractShearY;
	}

	static TOptional<float> ExtractTranslationX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? static_cast<float>(Bindings->GetCurrentValue<FWidgetTransform>(InObject).Translation.X) : TOptional<float>();
	}
	static TOptional<float> ExtractTranslationY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? static_cast<float>(Bindings->GetCurrentValue<FWidgetTransform>(InObject).Translation.Y) : TOptional<float>();
	}

	static TOptional<float> ExtractRotation(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FWidgetTransform>(InObject).Angle : TOptional<float>();
	}

	static TOptional<float> ExtractScaleX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? static_cast<float>(Bindings->GetCurrentValue<FWidgetTransform>(InObject).Scale.X) : TOptional<float>();
	}
	static TOptional<float> ExtractScaleY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? static_cast<float>(Bindings->GetCurrentValue<FWidgetTransform>(InObject).Scale.Y) : TOptional<float>();
	}

	static TOptional<float> ExtractShearX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? static_cast<float>(Bindings->GetCurrentValue<FWidgetTransform>(InObject).Shear.X) : TOptional<float>();
	}
	static TOptional<float> ExtractShearY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? static_cast<float>(Bindings->GetCurrentValue<FWidgetTransform>(InObject).Shear.Y) : TOptional<float>();
	}

	FMovieSceneChannelMetaData      MetaData[7];
	TMovieSceneExternalValue<float> ExternalValues[7];
};

#endif	// WITH_EDITOR

UMovieScene2DTransformSection::UMovieScene2DTransformSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

	TransformMask = EMovieScene2DTransformChannel::AllTransform;

	BlendType = EMovieSceneBlendType::Absolute;
	bSupportsInfiniteRange = true;
}

FMovieScene2DTransformMask UMovieScene2DTransformSection::GetMask() const
{
	return TransformMask;
}

void UMovieScene2DTransformSection::SetMask(FMovieScene2DTransformMask NewMask)
{
	TransformMask = NewMask;
	ChannelProxy = nullptr;
}

EMovieSceneChannelProxyType UMovieScene2DTransformSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	F2DTransformSectionEditorData EditorData(TransformMask.GetChannels());

	Channels.Add(Translation[0], EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(Translation[1], EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(Rotation,       EditorData.MetaData[2], EditorData.ExternalValues[2]);
	Channels.Add(Scale[0],       EditorData.MetaData[3], EditorData.ExternalValues[3]);
	Channels.Add(Scale[1],       EditorData.MetaData[4], EditorData.ExternalValues[4]);
	Channels.Add(Shear[0],       EditorData.MetaData[5], EditorData.ExternalValues[5]);
	Channels.Add(Shear[1],       EditorData.MetaData[6], EditorData.ExternalValues[6]);

#else

	Channels.Add(Translation[0]);
	Channels.Add(Translation[1]);
	Channels.Add(Rotation);
	Channels.Add(Scale[0]);
	Channels.Add(Scale[1]);
	Channels.Add(Shear[0]);
	Channels.Add(Shear[1]);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));

	return EMovieSceneChannelProxyType::Dynamic;
}

FMovieScene2DTransformMask UMovieScene2DTransformSection::GetMaskByName(const FName& InName) const
{
	if (InName.ToString() == NSLOCTEXT("MovieScene2DTransformSection", "Translation", "Translation").ToString())
	{
		return EMovieScene2DTransformChannel::Translation;
	}
	else if (InName == TEXT("Translation.X"))
	{
		return EMovieScene2DTransformChannel::TranslationX;
	}
	else if (InName == TEXT("Translation.Y"))
	{
		return EMovieScene2DTransformChannel::TranslationY;
	}
	else if (InName.ToString() == NSLOCTEXT("MovieScene2DTransformSection", "Angle", "Angle").ToString() ||
			InName.ToString() == NSLOCTEXT("MovieScene2DTransformSection", "Rotation", "Rotation").ToString())
	{
		return EMovieScene2DTransformChannel::Rotation;
	}
	else if (InName.ToString() == NSLOCTEXT("MovieScene2DTransformSection", "Scale", "Scale").ToString())
	{
		return EMovieScene2DTransformChannel::Scale;
	}
	else if (InName == TEXT("Scale.X"))
	{
		return EMovieScene2DTransformChannel::ScaleX;
	}
	else if (InName == TEXT("Scale.Y"))
	{
		return EMovieScene2DTransformChannel::ScaleY;
	}
	else if (InName.ToString() == NSLOCTEXT("MovieScene2DTransformSection", "Shear", "Shear").ToString())
	{
		return EMovieScene2DTransformChannel::Shear;
	}
	else if (InName == TEXT("Shear.X"))
	{
		return EMovieScene2DTransformChannel::ShearX;
	}
	else if (InName == TEXT("Shear.Y"))
	{
		return EMovieScene2DTransformChannel::ShearY;
	}

	return EMovieScene2DTransformChannel::AllTransform;
}

FWidgetTransform UMovieScene2DTransformSection::GetCurrentValue(const UObject* Object) const
{
	UMovieScene2DTransformTrack* Track = GetTypedOuter<UMovieScene2DTransformTrack>();
	return Track->GetCurrentValue<FWidgetTransform>(Object).Get(FWidgetTransform{});
}

void UMovieScene2DTransformSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	EMovieScene2DTransformChannel Channels      = (EMovieScene2DTransformChannel)TransformMask.GetChannels();
	FBuiltInComponentTypes*       Components    = FBuiltInComponentTypes::Get();
	FMovieSceneUMGComponentTypes* UMGComponents = FMovieSceneUMGComponentTypes::Get();

	UMovieScene2DTransformTrack* Track = GetTypedOuter<UMovieScene2DTransformTrack>();

	const bool ActiveChannelsMask[] = {
		EnumHasAnyFlags(Channels, EMovieScene2DTransformChannel::TranslationX) && Translation[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieScene2DTransformChannel::TranslationY) && Translation[1].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieScene2DTransformChannel::Rotation) && Rotation.HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieScene2DTransformChannel::ScaleX) && Scale[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieScene2DTransformChannel::ScaleY) && Scale[1].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieScene2DTransformChannel::ShearX) && Shear[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieScene2DTransformChannel::ShearY) && Shear[1].HasAnyData(),
	};

	if (!Algo::AnyOf(ActiveChannelsMask))
	{
		return;
	}

	FGuid ObjectBindingID = Params.GetObjectBindingID();

	// Create a new entity for this section
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(Components->PropertyBinding, Track->GetPropertyBinding())
		.AddConditional(Components->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
		.AddConditional(Components->FloatChannel[0],      &Translation[0], ActiveChannelsMask[0])
		.AddConditional(Components->FloatChannel[1],      &Translation[1], ActiveChannelsMask[1])
		.AddConditional(Components->FloatChannel[2],      &Rotation,       ActiveChannelsMask[2])
		.AddConditional(Components->FloatChannel[3],      &Scale[0],       ActiveChannelsMask[3])
		.AddConditional(Components->FloatChannel[4],      &Scale[1],       ActiveChannelsMask[4])
		.AddConditional(Components->FloatChannel[5],      &Shear[0],       ActiveChannelsMask[5])
		.AddConditional(Components->FloatChannel[6],      &Shear[1],       ActiveChannelsMask[6])
		.AddTag(UMGComponents->WidgetTransform.PropertyTag)
	);
}

