// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieScene2DTransformTrack.h"
#include "Animation/MovieScene2DTransformSection.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene2DTransformTrack)

UMovieScene2DTransformTrack::UMovieScene2DTransformTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(48, 227, 255, 65);
#endif

	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieScene2DTransformTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScene2DTransformSection::StaticClass();
}

UMovieSceneSection* UMovieScene2DTransformTrack::CreateNewSection()
{
	return NewObject<UMovieScene2DTransformSection>(this, NAME_None, RF_Transactional);
}
