// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieSceneMarginPropertySystem.h"
#include "Animation/MovieSceneUMGComponentTypes.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMarginPropertySystem)


UMovieSceneMarginPropertySystem::UMovieSceneMarginPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneUMGComponentTypes::Get()->Margin);

	if (HasAnyFlags(RF_ClassDefaultObject))
{
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
	}
	}

void UMovieSceneMarginPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
	{
	Super::OnRun(InPrerequisites, Subsequents);
	}


