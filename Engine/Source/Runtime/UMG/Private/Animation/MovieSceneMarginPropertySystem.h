// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "Systems/MovieScenePropertySystem.h"
#include "MovieSceneMarginPropertySystem.generated.h"

Expose_TNameOf(FMargin);

UCLASS(MinimalAPI)
class UMovieSceneMarginPropertySystem : public UMovieScenePropertySystem
{
public:

	GENERATED_BODY()
	
	UMG_API UMovieSceneMarginPropertySystem(const FObjectInitializer& ObjInit);

	UMG_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

