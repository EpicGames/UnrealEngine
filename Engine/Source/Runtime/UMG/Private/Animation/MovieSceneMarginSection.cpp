// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieSceneMarginSection.h"
#include "Animation/MovieSceneMarginTrack.h"
#include "Animation/MovieSceneUMGComponentTypes.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieScenePropertyTemplate.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMarginSection)

#if WITH_EDITOR

struct FMarginSectionEditorData
{
	FMarginSectionEditorData()
	{
		MetaData[0].SetIdentifiers("Left", NSLOCTEXT("MovieSceneMarginSection", "LeftText", "Left"));
		MetaData[0].SubPropertyPath = MetaData[0].Name;
		MetaData[0].SortOrder = 0;
		MetaData[0].bCanCollapseToTrack = false;

		MetaData[1].SetIdentifiers("Top", NSLOCTEXT("MovieSceneMarginSection", "TopText", "Top"));
		MetaData[1].SubPropertyPath = MetaData[1].Name;
		MetaData[1].SortOrder = 1;
		MetaData[1].bCanCollapseToTrack = false;

		MetaData[2].SetIdentifiers("Right", NSLOCTEXT("MovieSceneMarginSection", "RightText", "Right"));
		MetaData[2].SubPropertyPath = MetaData[2].Name;
		MetaData[2].SortOrder = 2;
		MetaData[2].bCanCollapseToTrack = false;

		MetaData[3].SetIdentifiers("Bottom", NSLOCTEXT("MovieSceneMarginSection", "BottomText", "Bottom"));
		MetaData[3].SubPropertyPath = MetaData[3].Name;
		MetaData[3].SortOrder = 3;
		MetaData[3].bCanCollapseToTrack = false;

		ExternalValues[0].OnGetExternalValue = ExtractLeftChannel;
		ExternalValues[1].OnGetExternalValue = ExtractTopChannel;
		ExternalValues[2].OnGetExternalValue = ExtractRightChannel;
		ExternalValues[3].OnGetExternalValue = ExtractBottomChannel;
	}

	static TOptional<float> ExtractLeftChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FMargin>(InObject).Left : TOptional<float>();
	}
	static TOptional<float> ExtractTopChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FMargin>(InObject).Top : TOptional<float>();
	}
	static TOptional<float> ExtractRightChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FMargin>(InObject).Right : TOptional<float>();
	}
	static TOptional<float> ExtractBottomChannel(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? Bindings->GetCurrentValue<FMargin>(InObject).Bottom : TOptional<float>();
	}

	FMovieSceneChannelMetaData      MetaData[4];
	TMovieSceneExternalValue<float> ExternalValues[4];
};

#endif	// WITH_EDITOR

UMovieSceneMarginSection::UMovieSceneMarginSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

	BlendType = EMovieSceneBlendType::Absolute;

	FMovieSceneChannelProxyData Channels;
	bSupportsInfiniteRange = true;

#if WITH_EDITOR

	static const FMarginSectionEditorData EditorData;
	Channels.Add(LeftCurve,   EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(TopCurve,    EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(RightCurve,  EditorData.MetaData[2], EditorData.ExternalValues[2]);
	Channels.Add(BottomCurve, EditorData.MetaData[3], EditorData.ExternalValues[3]);

#else

	Channels.Add(LeftCurve);
	Channels.Add(TopCurve);
	Channels.Add(RightCurve);
	Channels.Add(BottomCurve);

#endif
	
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

bool UMovieSceneMarginSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneMarginSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneUMGComponentTypes* UMGComponents = FMovieSceneUMGComponentTypes::Get();

	FPropertyTrackEntityImportHelper(UMGComponents->Margin)
		.AddConditional(Components->FloatChannel[0], &LeftCurve, LeftCurve.HasAnyData())
		.AddConditional(Components->FloatChannel[1], &TopCurve, TopCurve.HasAnyData())
		.AddConditional(Components->FloatChannel[2], &RightCurve, RightCurve.HasAnyData())
		.AddConditional(Components->FloatChannel[3], &BottomCurve, BottomCurve.HasAnyData())
		.Commit(this, Params, OutImportedEntity);
}

