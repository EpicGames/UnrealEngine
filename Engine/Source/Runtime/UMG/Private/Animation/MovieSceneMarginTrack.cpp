// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieSceneMarginTrack.h"
#include "Animation/MovieSceneMarginSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMarginTrack)


UMovieSceneMarginTrack::UMovieSceneMarginTrack(const FObjectInitializer& Init)
	: Super(Init)
{
	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneMarginTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneMarginSection::StaticClass();
}

UMovieSceneSection* UMovieSceneMarginTrack::CreateNewSection()
{
	return NewObject<UMovieSceneMarginSection>(this, NAME_None, RF_Transactional);
}


