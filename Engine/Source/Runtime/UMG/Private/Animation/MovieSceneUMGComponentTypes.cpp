// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieSceneUMGComponentTypes.h"
#include "Animation/WidgetMaterialTrackUtilities.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Animation/MovieSceneMarginSection.h"
#include "Animation/MovieScene2DTransformSection.h"

#include "Components/Widget.h"
#include "Components/CanvasPanelSlot.h"

namespace UE
{
namespace MovieScene
{

static bool GMovieSceneUMGComponentTypesDestroyed = false;
static TUniquePtr<FMovieSceneUMGComponentTypes> GMovieSceneUMGComponentTypes;

void UnpackChannelsFromOperational(const FIntermediateMargin& In, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(decltype(UMovieSceneMarginSection::LeftCurve)   , 0, In, Left  ));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(decltype(UMovieSceneMarginSection::TopCurve)    , 1, In, Top   ));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(decltype(UMovieSceneMarginSection::RightCurve)  , 2, In, Right ));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(decltype(UMovieSceneMarginSection::BottomCurve) , 3, In, Bottom));
}

void UnpackChannelsFromOperational(const FIntermediateWidgetTransform& In, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<TDecay<decltype(UMovieScene2DTransformSection::Translation[0])>::Type>(In.TranslationX, 0), FName("Translation.X")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<TDecay<decltype(UMovieScene2DTransformSection::Translation[1])>::Type>(In.TranslationY, 1), FName("Translation.Y")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<decltype(UMovieScene2DTransformSection::Rotation)>(                    In.Rotation,     2), FName("Rotation")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<TDecay<decltype(UMovieScene2DTransformSection::Scale[0])>::Type>(      In.ScaleX,       3), FName("Scale.X")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<TDecay<decltype(UMovieScene2DTransformSection::Scale[1])>::Type>(      In.ScaleY,       4), FName("Scale.Y")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<TDecay<decltype(UMovieScene2DTransformSection::Shear[0])>::Type>(      In.ShearX,       5), FName("Shear.X")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<TDecay<decltype(UMovieScene2DTransformSection::Shear[1])>::Type>(      In.ShearY,       6), FName("Shear.Y")));
}

void ConvertOperationalProperty(const FIntermediateWidgetTransform& In, FWidgetTransform& Out)
{
	Out.Translation.X = In.TranslationX;
	Out.Translation.Y = In.TranslationY;
	Out.Angle = static_cast<float>(In.Rotation);
	Out.Scale.X = In.ScaleX;
	Out.Scale.Y = In.ScaleY;
	Out.Shear.X = In.ShearX;
	Out.Shear.Y = In.ShearY;
}
void ConvertOperationalProperty(const FWidgetTransform& In, FIntermediateWidgetTransform& Out)
{
	Out.TranslationX = In.Translation.X;
	Out.TranslationY = In.Translation.Y;
	Out.Rotation = In.Angle;
	Out.ScaleX = In.Scale.X;
	Out.ScaleY = In.Scale.Y;
	Out.ShearX = In.Shear.X;
	Out.ShearY = In.Shear.Y;
}

void ConvertOperationalProperty(const FIntermediateMargin& In, FMargin& Out)
{
	Out.Top = static_cast<float>(In.Top);
	Out.Right = static_cast<float>(In.Right);
	Out.Bottom = static_cast<float>(In.Bottom);
	Out.Left = static_cast<float>(In.Left);
}

void ConvertOperationalProperty(const FMargin& In, FIntermediateMargin& Out)
{
	Out.Top = In.Top;
	Out.Right = In.Right;
	Out.Bottom = In.Bottom;
	Out.Left = In.Left;
}

static float GetRenderOpacity(const UObject* Object)
{
	return CastChecked<const UWidget>(Object)->GetRenderOpacity();
}

static void SetRenderOpacity(UObject* Object, float InRenderOpacity)
{
	CastChecked<UWidget>(Object)->SetRenderOpacity(InRenderOpacity);
}

static FIntermediateWidgetTransform GetRenderTransform(const UObject* Object)
{
	FWidgetTransform Transform = CastChecked<const UWidget>(Object)->GetRenderTransform();

	FIntermediateWidgetTransform IntermediateTransform{};
	ConvertOperationalProperty(Transform, IntermediateTransform);
	return IntermediateTransform;
}

static void SetRenderTransform(UObject* Object, const FIntermediateWidgetTransform& InRenderTransform)
{
	FWidgetTransform Transform{};
	ConvertOperationalProperty(InRenderTransform, Transform);

	CastChecked<UWidget>(Object)->SetRenderTransform(Transform);
}

FIntermediateMargin GetLayoutDataOffsets(const UObject* Object)
{
	FIntermediateMargin Margin;
	ConvertOperationalProperty(CastChecked<UCanvasPanelSlot>(Object)->GetOffsets(), Margin);
	return Margin;
}
void SetLayoutDataOffsets(UObject* Object, const FIntermediateMargin& InOffsets)
{
	FMargin Margin;
	ConvertOperationalProperty(InOffsets, Margin);
	CastChecked<UCanvasPanelSlot>(Object)->SetOffsets(Margin);
}


FMovieSceneUMGComponentTypes::FMovieSceneUMGComponentTypes()
{
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FComponentRegistry*     ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

	ComponentRegistry->NewPropertyType(Margin, TEXT("FMargin Property"));

	ComponentRegistry->NewPropertyType(WidgetTransform, TEXT("FWidgetTransform Property"));

	ComponentRegistry->NewComponentType(&WidgetMaterialPath, TEXT("Widget Material Path"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&WidgetMaterialHandle, TEXT("Widget Material Handle"), EComponentTypeFlags::CopyToOutput);

	/** Initializer that initializes the value of an FWidgetMaterialHandle derived from an FWidgetMaterialPath */
	struct FWidgetMaterialHandleInitializer : TChildEntityInitializer<FWidgetMaterialPath, FWidgetMaterialHandle>
	{
		explicit FWidgetMaterialHandleInitializer(TComponentTypeID<FWidgetMaterialPath> Path, TComponentTypeID<FWidgetMaterialHandle> Handle)
			: TChildEntityInitializer<FWidgetMaterialPath, FWidgetMaterialHandle>(Path, Handle)
		{}

		virtual void Run(const FEntityRange& ChildRange, const FEntityAllocation* ParentAllocation, TArrayView<const int32> ParentAllocationOffsets)
		{
			TComponentReader<FWidgetMaterialPath>   PathComponents        = ParentAllocation->ReadComponents(this->GetParentComponent());
			TComponentWriter<FWidgetMaterialHandle> HandleComponents      = ChildRange.Allocation->WriteComponents(this->GetChildComponent(), FEntityAllocationWriteContext::NewAllocation());
			TOptionalComponentReader<UObject*>      BoundObjectComponents = ChildRange.Allocation->TryReadComponents(FBuiltInComponentTypes::Get()->BoundObject);
			if (!ensure(BoundObjectComponents))
			{
				return;
			}

			for (int32 Index = 0; Index < ChildRange.Num; ++Index)
			{
				const int32 ParentIndex = ParentAllocationOffsets[Index];
				const int32 ChildIndex  = ChildRange.ComponentStartOffset + Index;

				UWidget* Widget = Cast<UWidget>(BoundObjectComponents[ChildIndex]);
				if (Widget)
				{
					HandleComponents[ChildIndex] = WidgetMaterialTrackUtilities::GetMaterialHandle(Widget, PathComponents[ParentIndex].Path);
				}
			}
		}
	};

	ComponentRegistry->Factories.DefineChildComponent(FWidgetMaterialHandleInitializer(WidgetMaterialPath, WidgetMaterialHandle));

	FMovieSceneTracksComponentTypes::Get()->Accessors.Float.Add(UWidget::StaticClass(), "RenderOpacity", &GetRenderOpacity, &SetRenderOpacity);

	CustomWidgetTransformAccessors.Add(UWidget::StaticClass(), "RenderTransform", &GetRenderTransform, &SetRenderTransform);
	CustomMarginAccessors.Add(UCanvasPanelSlot::StaticClass(), "LayoutData.Offsets", &GetLayoutDataOffsets, &SetLayoutDataOffsets);

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Margin, TEXT("Apply FMargin Properties"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediateMargin::Left)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediateMargin::Top)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediateMargin::Right)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediateMargin::Bottom)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&CustomMarginAccessors)
	.Commit();

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(WidgetTransform, TEXT("Call UUserWidget::SetRenderTransform"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediateWidgetTransform::TranslationX)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediateWidgetTransform::TranslationY)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediateWidgetTransform::Rotation)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediateWidgetTransform::ScaleX)
	.AddComposite(BuiltInComponents->DoubleResult[4], &FIntermediateWidgetTransform::ScaleY)
	.AddComposite(BuiltInComponents->DoubleResult[5], &FIntermediateWidgetTransform::ShearX)
	.AddComposite(BuiltInComponents->DoubleResult[6], &FIntermediateWidgetTransform::ShearY)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&CustomWidgetTransformAccessors)
	.Commit();
}

FMovieSceneUMGComponentTypes::~FMovieSceneUMGComponentTypes()
{
}

void FMovieSceneUMGComponentTypes::Destroy()
{
	GMovieSceneUMGComponentTypes.Reset();
	GMovieSceneUMGComponentTypesDestroyed = true;
}

FMovieSceneUMGComponentTypes* FMovieSceneUMGComponentTypes::Get()
{
	if (!GMovieSceneUMGComponentTypes.IsValid())
	{
		check(!GMovieSceneUMGComponentTypesDestroyed);
		GMovieSceneUMGComponentTypes.Reset(new FMovieSceneUMGComponentTypes);
	}
	return GMovieSceneUMGComponentTypes.Get();
}


} // namespace MovieScene
} // namespace UE
