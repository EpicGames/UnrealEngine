// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieSceneWidgetMaterialSystem.h"
#include "Animation/MovieSceneUMGComponentTypes.h"
#include "Animation/WidgetMaterialTrackUtilities.h"

#include "EntitySystem/MovieSceneEntityMutations.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include "Materials/MaterialInstanceDynamic.h"

#include "UMGPrivate.h"
#include "Components/Widget.h"

#include "String/Join.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneWidgetMaterialSystem)

namespace UE::MovieScene
{

FWidgetMaterialAccessor::FWidgetMaterialAccessor(const FWidgetMaterialKey& InKey)
	: Widget(CastChecked<UWidget>(InKey.Object.ResolveObjectPtr(), ECastCheckedType::NullAllowed))
{
	// Only assign the material handle if the widget itself is still valid
	// otherwise accessors constructed with the material handle will consider it valid
	if (Widget)
	{
		WidgetMaterialHandle = InKey.WidgetMaterialHandle;
	}
}

FWidgetMaterialAccessor::FWidgetMaterialAccessor(UObject* InObject, FWidgetMaterialHandle InWidgetMaterialHandle)
	: Widget(Cast<UWidget>(InObject))
	, WidgetMaterialHandle(MoveTemp(InWidgetMaterialHandle))
{
	// Object must be a widget
	if (InObject && !Widget)
	{
		UE_LOGF(LogUMG, Warning, "Cannot animate widget material on object %ls of type %ls",
			*InObject->GetName(), *InObject->GetClass()->GetName()
		);
	}
}

FWidgetMaterialAccessor::operator bool() const
{
	return Widget != nullptr;
}

FString FWidgetMaterialAccessor::ToString() const
{
	return FString::Printf(TEXT("Brush on widget %s"), *Widget->GetPathName());
}

UMaterialInterface* FWidgetMaterialAccessor::GetMaterial() const
{
	if (WidgetMaterialHandle.IsValid())
	{
		return WidgetMaterialHandle.GetMaterial();
	}

	return nullptr;
}

void FWidgetMaterialAccessor::SetMaterial(UMaterialInterface* InMaterial)
{
	if (WidgetMaterialHandle.IsValid())
	{
		WidgetMaterialHandle.SetMaterial(InMaterial, Widget);
	}
}

UMaterialInstanceDynamic* FWidgetMaterialAccessor::CreateDynamicMaterial(UMaterialInterface* InMaterial)
{
	// Need to create a new MID, either because the parent has changed, or because one doesn't already exist
	TStringBuilder<128> DynamicName;
	InMaterial->GetFName().ToString(DynamicName);
	DynamicName.Append(TEXT("_Animated"));
	FName UniqueDynamicName = MakeUniqueObjectName(Widget, UMaterialInstanceDynamic::StaticClass() , DynamicName.ToString());

	UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create(InMaterial, Widget, UniqueDynamicName);
	SetMaterial(MID);
	return MID;
}

TAutoRegisterPreAnimatedStorageID<FPreAnimatedWidgetMaterialSwitcherStorage> FPreAnimatedWidgetMaterialSwitcherStorage::StorageID;
TAutoRegisterPreAnimatedStorageID<FPreAnimatedWidgetMaterialParameterStorage> FPreAnimatedWidgetMaterialParameterStorage::StorageID;

} // namespace UE::MovieScene

UMovieSceneWidgetMaterialSystem::UMovieSceneWidgetMaterialSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneUMGComponentTypes*    WidgetComponents  = FMovieSceneUMGComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = WidgetComponents->WidgetMaterialHandle;
	Phase = ESystemPhase::Instantiation;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), BuiltInComponents->ObjectResult);
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
		DefineComponentProducer(GetClass(), TracksComponents->BoundMaterial);
		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneWidgetMaterialSystem::OnLink()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*       BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneUMGComponentTypes* WidgetComponents  = FMovieSceneUMGComponentTypes::Get();

	SystemImpl.MaterialSwitcherStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedWidgetMaterialSwitcherStorage>();
	SystemImpl.MaterialParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedWidgetMaterialParameterStorage>();

	SystemImpl.OnLink(Linker, BuiltInComponents->BoundObject, WidgetComponents->WidgetMaterialHandle);
}

void UMovieSceneWidgetMaterialSystem::OnUnlink()
{
	SystemImpl.OnUnlink(Linker);
}

void UMovieSceneWidgetMaterialSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*       BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneUMGComponentTypes* WidgetComponents  = FMovieSceneUMGComponentTypes::Get();

	SystemImpl.OnRun(Linker, BuiltInComponents->BoundObject, WidgetComponents->WidgetMaterialHandle, InPrerequisites, Subsequents);
}

void UMovieSceneWidgetMaterialSystem::SavePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*       BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneUMGComponentTypes* WidgetComponents  = FMovieSceneUMGComponentTypes::Get();

	SystemImpl.SavePreAnimatedState(Linker, BuiltInComponents->BoundObject, WidgetComponents->WidgetMaterialHandle, InParameters);
}
