// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/MovieSceneWidgetMaterialTrack.h"
#include "Animation/WidgetMaterialTrackUtilities.h"
#include "Animation/MovieSceneUMGComponentTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneWidgetMaterialTrack)

UMovieSceneWidgetMaterialTrack::UMovieSceneWidgetMaterialTrack( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{

}

UMovieSceneSection* UMovieSceneWidgetMaterialTrack::CreateNewSection()
{
	UMovieSceneSection* NewSection = NewObject<UMovieSceneParameterSection>(this, NAME_None, RF_Transactional);
	NewSection->SetBlendType(EMovieSceneBlendType::Absolute);
	NewSection->SetRange(TRange<FFrameNumber>::All());
	return NewSection;
}

bool UMovieSceneWidgetMaterialTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneParameterSection::StaticClass();
}

void UMovieSceneWidgetMaterialTrack::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	// These tracks don't define any entities for themselves
	checkf(false, TEXT("This track should never have created entities for itself - this assertion indicates an error in the entity-component field"));
}

void UMovieSceneWidgetMaterialTrack::ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*       BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneUMGComponentTypes* WidgetComponents  = FMovieSceneUMGComponentTypes::Get();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(WidgetComponents->WidgetMaterialPath, FWidgetMaterialPath(BrushPropertyNamePath))
		// If the section has no valid blend type (legacy data), make it use absolute blending.
		// Otherwise, the base section class will add the appropriate blend type tag in BuildDefaultComponents.
		.AddTagConditional(BuiltInComponents->Tags.AbsoluteBlend, !Section->GetBlendType().IsValid())
	);
}

bool UMovieSceneWidgetMaterialTrack::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const FMovieSceneTrackEvaluationField& LocalEvaluationField = GetEvaluationField();

	// Define entities for every entry in our evaluation field
	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : LocalEvaluationField.Entries)
	{
		UMovieSceneParameterSection* ParameterSection = Cast<UMovieSceneParameterSection>(Entry.Section);
		if (!ParameterSection || IsRowEvalDisabled(ParameterSection->GetRowIndex()))
		{
			continue;
		}

		TRange<FFrameNumber> SectionEffectiveRange = TRange<FFrameNumber>::Intersection(EffectiveRange, Entry.Range);
		if (!SectionEffectiveRange.IsEmpty())
		{
			FMovieSceneEvaluationFieldEntityMetaData SectionMetaData = InMetaData;
			SectionMetaData.Flags = Entry.Flags;
			SectionMetaData.Condition = MovieSceneHelpers::GetSequenceCondition(this, ParameterSection, true);

			ParameterSection->ExternalPopulateEvaluationField(SectionEffectiveRange, SectionMetaData, OutFieldBuilder);
		}
	}

	return true;
}


FName UMovieSceneWidgetMaterialTrack::GetTrackName() const
{ 
	return TrackName;
}


#if WITH_EDITORONLY_DATA
FText UMovieSceneWidgetMaterialTrack::GetDefaultDisplayName() const
{
	return FText::Format(NSLOCTEXT("UMGAnimation", "MaterialTrackFormat", "{0} Material"), FText::FromName( TrackName ) );
}
#endif


void UMovieSceneWidgetMaterialTrack::SetBrushPropertyNamePath( TArray<FName> InBrushPropertyNamePath )
{
	BrushPropertyNamePath = InBrushPropertyNamePath;
	TrackName = WidgetMaterialTrackUtilities::GetTrackNameFromPropertyNamePath( BrushPropertyNamePath );
}

