// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/UMGSequencePlayer.h"

#include "Animation/UMGSequenceTickManager.h"
#include "Animation/WidgetAnimation.h"
#include "Animation/WidgetAnimationState.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "IMovieScenePlaybackClient.h"
#include "MovieSceneFwd.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UMGSequencePlayer)

UUMGSequencePlayer::UUMGSequencePlayer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UUMGSequencePlayer::InitSequencePlayer(FWidgetAnimationState& InState)
{
	WidgetAnimationHandle = InState.GetAnimationHandle();

	RootTemplateInstance.Initialize(InState.GetSharedPlaybackState().ToSharedRef());
}

void UUMGSequencePlayer::InitSequencePlayer(UWidgetAnimation& InAnimation, UUserWidget& InUserWidget)
{
	// Nothing, stubbed out.
	ensureMsgf(false, TEXT("UUMGSequencePlayer shouldn't be doing anything except wrap a widget animation runner."));
}

void UUMGSequencePlayer::Tick(float DeltaTime)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->Tick(DeltaTime);
	}
}

void UUMGSequencePlayer::Play(float StartAtTime, int32 InNumLoopsToPlay, EUMGSequencePlayMode::Type InPlayMode, float InPlaybackSpeed, bool bInRestoreState)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		FWidgetAnimationStatePlayParams PlayParams;
		PlayParams.StartAtTime = StartAtTime;
		PlayParams.NumLoopsToPlay = InNumLoopsToPlay;
		PlayParams.PlayMode = InPlayMode;
		PlayParams.PlaybackSpeed = InPlaybackSpeed;
		PlayParams.bRestoreState = bInRestoreState;
		AnimState->Play(PlayParams);
	}
}

void UUMGSequencePlayer::PlayTo(float StartAtTime, float EndAtTime, int32 InNumLoopsToPlay, EUMGSequencePlayMode::Type InPlayMode, float InPlaybackSpeed, bool bInRestoreState)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		FWidgetAnimationStatePlayParams PlayParams;
		PlayParams.StartAtTime = StartAtTime;
		PlayParams.EndOffset = EndAtTime;
		PlayParams.NumLoopsToPlay = InNumLoopsToPlay;
		PlayParams.PlayMode = InPlayMode;
		PlayParams.PlaybackSpeed = InPlaybackSpeed;
		PlayParams.bRestoreState = bInRestoreState;
		AnimState->Play(PlayParams);
	}
}

void UUMGSequencePlayer::Pause()
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->Pause();
	}
}

void UUMGSequencePlayer::Reverse()
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->Reverse();
	}
}

void UUMGSequencePlayer::Stop()
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->Stop();
	}
}

void UUMGSequencePlayer::SetCurrentTime(float InTime)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->SetCurrentTime(InTime);
	}
}

FQualifiedFrameTime UUMGSequencePlayer::GetCurrentTime() const
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->GetCurrentTime();
	}
	return FQualifiedFrameTime();
}

const UWidgetAnimation* UUMGSequencePlayer::GetAnimation() const
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->GetAnimation();
	}
	return nullptr;
}

FName UUMGSequencePlayer::GetUserTag() const
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->GetUserTag();
	}
	return NAME_None;
}

void UUMGSequencePlayer::SetUserTag(FName InUserTag)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->SetUserTag(InUserTag);
	}
}

void UUMGSequencePlayer::SetNumLoopsToPlay(int32 InNumLoopsToPlay)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->SetNumLoopsToPlay(InNumLoopsToPlay);
	}
}

void UUMGSequencePlayer::SetPlaybackSpeed(float InPlaybackSpeed)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->SetPlaybackSpeed(InPlaybackSpeed);
	}
}

bool UUMGSequencePlayer::IsPlayingForward() const
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->IsPlayingForward();
	}
	return true;
}

bool UUMGSequencePlayer::IsStopping() const
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->IsStopping();
	}
	return false;
}

FMovieSceneRootEvaluationTemplateInstance& UUMGSequencePlayer::GetEvaluationTemplate()
{
	return RootTemplateInstance;
}

UMovieSceneEntitySystemLinker* UUMGSequencePlayer::ConstructEntitySystemLinker()
{
	ensureMsgf(false, TEXT("This legacy player should never have to construct a linker."));
	return nullptr;
}

UObject* UUMGSequencePlayer::AsUObject()
{
	return this;
}

EMovieScenePlayerStatus::Type UUMGSequencePlayer::GetPlaybackStatus() const
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->GetPlaybackStatus();
	}
	return EMovieScenePlayerStatus::Stopped;
}

void UUMGSequencePlayer::SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus)
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		AnimState->SetPlaybackStatus(InPlaybackStatus);
	}
}

IMovieScenePlaybackClient* UUMGSequencePlayer::GetPlaybackClient()
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->GetSharedPlaybackState()->FindCapability<IMovieScenePlaybackClient>();
	}
	return nullptr;
}

FMovieSceneSpawnRegister& UUMGSequencePlayer::GetSpawnRegister()
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		if (FMovieSceneSpawnRegister* SpawnRegister = AnimState->GetSharedPlaybackState()->FindCapability<FMovieSceneSpawnRegister>())
		{
			return *SpawnRegister;
		}
	}
	return IMovieScenePlayer::GetSpawnRegister();
}

UObject* UUMGSequencePlayer::GetPlaybackContext() const
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->GetUserWidget();
	}
	return nullptr;
}

void UUMGSequencePlayer::InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState)
{
	ensureMsgf(false, TEXT("The legacy player should never initialize sequences: it only wraps already initialized ones."));
}

void UUMGSequencePlayer::RemoveEvaluationData()
{
	if (FWidgetAnimationState* AnimState = WidgetAnimationHandle.GetAnimationState())
	{
		return AnimState->RemoveEvaluationData();
	}
}

void UUMGSequencePlayer::TearDown()
{
}

void UUMGSequencePlayer::BeginDestroy()
{
	Super::BeginDestroy();
}

UUMGSequencePlayer::FOnSequenceFinishedPlaying& UUMGSequencePlayer::OnSequenceFinishedPlaying()
{
	return OnSequenceFinishedPlayingEvent;
}

void UUMGSequencePlayer::BroadcastSequenceFinishedPlaying()
{
	OnSequenceFinishedPlayingEvent.Broadcast(*this);
}

