// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/UMGSequenceTickManager.h"
#include "Animation/UMGSequencePlayer.h"
#include "Blueprint/UserWidget.h"
#include "Engine/World.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UMGSequenceTickManager)

DECLARE_CYCLE_STAT(TEXT("Flush End of Frame Animations"), MovieSceneEval_FlushEndOfFrameAnimations, STATGROUP_MovieSceneEval);


namespace UE::UMG
{

	static TAutoConsoleVariable<int32> CVarUMGMaxAnimationLatentActions(
		TEXT("Widget.MaxAnimationLatentActions"),
		100,
		TEXT("Defines the maximum number of latent actions that can be run in one frame."),
		ECVF_Default
	);
	int32 GFlushUMGAnimationsAtEndOfFrame = 1;
	static FAutoConsoleVariableRef CVarUMGAnimationsAtEndOfFrame(
		TEXT("UMG.FlushAnimationsAtEndOfFrame"),
		GFlushUMGAnimationsAtEndOfFrame,
		TEXT("Whether to automatically flush any outstanding animations at the end of the frame, or just wait until next frame."),
		ECVF_Default
	);

	float GAnimationBudgetMs = 0.0f;
	FAutoConsoleVariableRef CVarAnimationBudgetMs(
		TEXT("UMG.AnimationBudgetMs"),
		GAnimationBudgetMs,
		TEXT("(Default: 0.0) EXPERIMENTAL: A per-frame animation budget to use for evaluation of all UMG animations this frame.")
	);
} // namespace UE::UMG

UUMGSequenceTickManager::UUMGSequenceTickManager(const FObjectInitializer& Init)
	: Super(Init)
	, bIsTicking(false)
{
}

void UUMGSequenceTickManager::Initialize(UObject* Owner)
{
	Linker = UMovieSceneEntitySystemLinker::FindOrCreateLinker(Owner, UE::MovieScene::EEntitySystemLinkerRole::UMG, TEXT("UMGAnimationEntitySystemLinker"));
	check(Linker);
	Runner = Linker->GetRunner();

	FSlateApplication& SlateApp = FSlateApplication::Get();
	FDelegateHandle PreTickHandle = SlateApp.OnPreTick().AddUObject(this, &UUMGSequenceTickManager::TickWidgetAnimations);
	check(PreTickHandle.IsValid());
	SlateApplicationPreTickHandle = PreTickHandle;

	FDelegateHandle PostTickHandle = SlateApp.OnPostTick().AddUObject(this, &UUMGSequenceTickManager::HandleSlatePostTick);
	check(PostTickHandle.IsValid());
	SlateApplicationPostTickHandle = PostTickHandle;
}

void UUMGSequenceTickManager::AddWidget(UUserWidget* InWidget)
{
	// This is functionally the same as OnWidgetTicked, but they remain
	// separate functions to convey the semantic difference
	TWeakObjectPtr<UUserWidget> WeakWidget = InWidget;

	if (FSequenceTickManagerWidgetData* WidgetData = WeakUserWidgetData.Find(WeakWidget))
	{
		WidgetData->bIsTicking = true;
	}
	else if (bIsTicking)
	{
		PendingUserWidgets.Add(InWidget);
	}
	else
	{
		WeakUserWidgetData.Add(WeakWidget, FSequenceTickManagerWidgetData());
	}
}

void UUMGSequenceTickManager::RemoveWidget(UUserWidget* InWidget)
{
	ClearLatentActions(InWidget);
	TWeakObjectPtr<UUserWidget> WeakWidget = InWidget;
	WeakUserWidgetData.Remove(WeakWidget);
}

void UUMGSequenceTickManager::OnWidgetTicked(UUserWidget* InWidget)
{
	if (FSequenceTickManagerWidgetData* WidgetData = WeakUserWidgetData.Find(InWidget))
	{
		WidgetData->bIsTicking = true;
	}
	else if (bIsTicking)
	{
		PendingUserWidgets.Add(InWidget);
	}
	else
	{
		WeakUserWidgetData.Add(InWidget, FSequenceTickManagerWidgetData());
	}
}

void UUMGSequenceTickManager::BeginDestroy()
{
	if (SlateApplicationPreTickHandle.IsValid())
	{
		if (FSlateApplication::IsInitialized())
		{
			FSlateApplication& SlateApp = FSlateApplication::Get();
			SlateApp.OnPreTick().Remove(SlateApplicationPreTickHandle);
			SlateApplicationPreTickHandle.Reset();

			SlateApp.OnPostTick().Remove(SlateApplicationPostTickHandle);
			SlateApplicationPostTickHandle.Reset();
		}
	}

	Super::BeginDestroy();
}

void UUMGSequenceTickManager::TickWidgetAnimations(float DeltaSeconds)
{
	if (bIsTicking)
	{
		return;
	}

	if (IsUnreachable() || HasAnyFlags(RF_BeginDestroyed) || Linker == nullptr || Linker->IsUnreachable() || Linker->HasAnyFlags(RF_BeginDestroyed))
	{
		// Speculatively ignore any kinds of updates if any of the required objects are in the process of being torn down
		return;
	}

	// Don't tick the animation if inside of a PostLoad
	if (FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return;
	}

	TGuardValue<bool> IsTickingGuard(bIsTicking, true);

	// Tick all animations in all active widgets.
	//
	// In the main code path (the one where animations are just chugging along), the UMG sequence players
	// will queue evaluations on the global sequencer ECS linker. In some specific cases, though (pausing,
	// stopping, etc.), we might see some blocking (immediate) evaluations running here.
	//
	// The WidgetData have one frame delay (they are updated at the end of the frame).
	// This may delay the animation update by one frame.
	const bool bIsCurrentlyEvaluating = Runner->IsCurrentlyEvaluating();
	{
		SCOPE_CYCLE_UOBJECT(ContextScope, this);

		// Process animations for visible widgets
		for (auto WidgetIter = WeakUserWidgetData.CreateIterator(); WidgetIter; ++WidgetIter)
		{
			UUserWidget* UserWidget = WidgetIter.Key().Get();
			FSequenceTickManagerWidgetData& WidgetData = WidgetIter.Value();

			const bool bRemoveWidget = TickWidgetAnimations(DeltaSeconds, UserWidget, WidgetData, bIsCurrentlyEvaluating);
			if (bRemoveWidget)
			{
				WidgetIter.RemoveCurrent();
			}
		}
	}

	// We may have seen new animations start on new widgets during the previous loop. To prevent the
	// WeakUserWidgetData map from being modified during iteration, any such newly animated widgets would
	// have been placed in PendingUserWidgets. We need to process them now until nothing new is added.
	while (PendingUserWidgets.Num() > 0)
	{
		TArray<TObjectPtr<UUserWidget>> NewUserWidgets;
		Swap(PendingUserWidgets, NewUserWidgets);

		for (TObjectPtr<UUserWidget> NewWidget : NewUserWidgets)
		{
			FSequenceTickManagerWidgetData NewWidgetData;

			const bool bRemoveWidget = TickWidgetAnimations(DeltaSeconds, NewWidget, NewWidgetData, bIsCurrentlyEvaluating);
			if (!bRemoveWidget)
			{
				WeakUserWidgetData.Add(NewWidget, NewWidgetData);
			}
		}
	}
	ensure(PendingUserWidgets.IsEmpty());

	ForceFlush();

	if (!Runner->IsCurrentlyEvaluating())
	{
		for (auto WidgetIter = WeakUserWidgetData.CreateIterator(); WidgetIter; ++WidgetIter)
		{
			UUserWidget* UserWidget = WidgetIter.Key().Get();
			ensureMsgf(UserWidget, TEXT("Widget became null during animation tick!"));

			if (UserWidget)
			{
				// If this widget no longer has any animations playing, it doesn't need to be ticked any more
				if (!UserWidget->IsAnyAnimationPlaying())
				{
					UserWidget->UpdateCanTick();
					UserWidget->AnimationTickManager = nullptr;
					WidgetIter.RemoveCurrent();
				}
			}
			else
			{
				WidgetIter.RemoveCurrent();
			}
		}
	}

	WeakUserWidgetData.Shrink();
}

bool UUMGSequenceTickManager::TickWidgetAnimations(
		float DeltaSeconds, 
		UUserWidget* UserWidget, 
		FSequenceTickManagerWidgetData& UserWidgetData,
		bool bIsCurrentlyEvaluating)
{
	UserWidgetData.bActionsAndAnimationTicked = false;

	if (!UserWidget)
	{
		// Remove this widget.
		return true;
	}
	else if (!UserWidget->IsConstructed())
	{
		if (!bIsCurrentlyEvaluating)
		{
			// Tear down any animations that are not currently being stopped.
			UserWidget->ConditionalTearDownAnimations();
			UserWidget->UpdateCanTick();

			// If there are no more animations playing, we can remove this widget altogether.
			if (!UserWidget->IsAnyAnimationPlaying())
			{
				// Resetting the animation tick manager is ok here because TearDownAnimations will always 
				// clear out all animations.
				UserWidget->AnimationTickManager = nullptr;

				// Remove this widget.
				return true;
			}
		}
	}
	else if (!UserWidgetData.bIsTicking)
	{
		// If this widget has not told us it is ticking, we disable animations for that widget.
		// Once it ticks again, the animation will be updated naturally, and doesn't need anything re-enabling.
		// 
		// @todo: There is a chance that relative animations hitting this code path will resume with
		// different relative bases due to the way the ecs data is destroyed and re-created.
		// In order to fix this we would have to annex that data instead of destroying it.
		if (!bIsCurrentlyEvaluating)
		{
			UserWidget->DisableAnimations();

			// Do not null out UUserWidget::AnimationTickManager because although we removed animation _data_
			// the animations themselves are still playing. As such any UUMGSequencePlayers may hold a reference 
			// to this tick manager's linker, and therefore also need to keep this tick manager alive since the 
			// linker is not outered to this tick manager.

			// Remove this widget.
			return true;
		}
	}
	else
	{
		SCOPE_CYCLE_UOBJECT(WidgetContextScope, UserWidget);

#if WITH_EDITOR
		const bool bTickAnimations = !UserWidget->IsDesignTime();
#else
		const bool bTickAnimations = true;
#endif
		if (bTickAnimations && UserWidget->IsVisible())
		{
			UserWidget->TickActionsAndAnimation(DeltaSeconds);
			UserWidgetData.bActionsAndAnimationTicked = true;
		}

		// Assume this widget will no longer tick, until we're told otherwise by way of OnWidgetTicked.
		UserWidgetData.bIsTicking = false;
	}

	// Don't remove this widget.
	return false;
}

void UUMGSequenceTickManager::ForceFlush()
{
	Runner->Flush(UE::UMG::GAnimationBudgetMs);
	RunLatentActions();
}

void UUMGSequenceTickManager::HandleSlatePostTick(float DeltaSeconds)
{
	// Early out if inside a PostLoad
	if (FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return;
	}

	// Only tick widgets at the end of the frame if our runner has completely finished, and we still have updates
	if (UE::UMG::GFlushUMGAnimationsAtEndOfFrame && Runner->HasQueuedUpdates() && !Runner->IsCurrentlyEvaluating())
	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_FlushEndOfFrameAnimations);

		Runner->Flush();
		RunLatentActions();
	}
}

void UUMGSequenceTickManager::AddLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate)
{
	LatentActionManager.AddLatentAction(Delegate);
}

void UUMGSequenceTickManager::ClearLatentActions(UObject* Object)
{
	LatentActionManager.ClearLatentActions(Object);
}

void UUMGSequenceTickManager::RunLatentActions()
{
	if (!this->Runner->IsCurrentlyEvaluating())
	{
		int32 UpdateCount   = this->Runner->GetQueuedUpdateCount();
		uint64 SystemSerial = this->Linker->EntityManager.GetSystemSerial();

		LatentActionManager.RunLatentActions([this, &SystemSerial, &UpdateCount]
		{
			int32  NewUpdateCount  = this->Runner->GetQueuedUpdateCount();
			uint64 NewSystemSerial = this->Linker->EntityManager.GetSystemSerial();
			if (NewUpdateCount != UpdateCount || NewSystemSerial != SystemSerial)
			{
				UpdateCount = NewUpdateCount;
				SystemSerial = NewSystemSerial;

				this->Runner->Flush();
			}
		});
	}
}

UUMGSequenceTickManager* UUMGSequenceTickManager::Get(UObject* PlaybackContext)
{
	const TCHAR* TickManagerName = TEXT("GlobalUMGSequenceTickManager");

	// The tick manager is owned by GEngine to ensure that it is kept alive for widgets that do not belong to
	// a world, but still require animations to be ticked. Ultimately this class could become an engine subsystem
	// but that would mean it is still around and active even if there are no animations playing, which is less
	// than ideal
	UObject* Owner = GEngine;
	if (!ensure(Owner))
	{
		// If (in the hopefully impossible event) there is no engine, use the previous method of a World as a fallback.
		// This will at least ensure we do not crash at the callsite due to a null tick manager
		check(PlaybackContext != nullptr && PlaybackContext->GetWorld() != nullptr);
		Owner = PlaybackContext->GetWorld();
	}

	UUMGSequenceTickManager* TickManager = FindObject<UUMGSequenceTickManager>(Owner, TickManagerName);
	if (!TickManager)
	{
		TickManager = NewObject<UUMGSequenceTickManager>(Owner, TickManagerName);
		TickManager->Initialize(Owner);
	}
	return TickManager;
}


