// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/WidgetAnimation.h"
#include "UObject/Package.h"
#include "Components/Visual.h"
#include "Blueprint/UserWidget.h"
#include "MovieScene.h"
#include "Components/PanelSlot.h"
#include "IMovieScenePlayer.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneMaterialParameterCollectionTrack.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "UObject/SequencerObjectVersion.h"

#if WITH_EDITOR
#include "Styling/AppStyle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetAnimation)


#define LOCTEXT_NAMESPACE "UWidgetAnimation"

#if WITH_EDITOR
UWidgetAnimation::FFixupWidgetDynamicBindingsEvent UWidgetAnimation::FixupWidgetDynamicBindingsEvent;
#endif

/* UWidgetAnimation structors
 *****************************************************************************/

UWidgetAnimation::UWidgetAnimation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MovieScene(nullptr)
{
	bParentContextsAreSignificant = false;
	bLegacyFinishOnStop = true;
}

/* UObject interface
 *****************************************************************************/

void UWidgetAnimation::PostLoad()
{
	if (GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::FinishUMGEvaluation)
	{
		bLegacyFinishOnStop = false;
	}

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LevelSequenceUpgradeDynamicBindings)
	{
		bool bConvertedDynamicBinding = false;
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
			if (Possessable.DynamicBinding_DEPRECATED.Function)
			{
				bConvertedDynamicBinding = true;

				for (FWidgetAnimationBinding& Binding : AnimationBindings)
				{
					if (Binding.AnimationGuid == Possessable.GetGuid())
					{
						Binding.DynamicBinding = Possessable.DynamicBinding_DEPRECATED;
						Possessable.DynamicBinding_DEPRECATED = FMovieSceneDynamicBinding();
						break;
					}
				}
			}
		}
#if WITH_EDITOR
		if (bConvertedDynamicBinding && FixupWidgetDynamicBindingsEvent.IsBound())
		{
			FixupWidgetDynamicBindingsEvent.Broadcast(this);
		}
#endif
	}

	Super::PostLoad();
}


/* UWidgetAnimation interface
 *****************************************************************************/

#if WITH_EDITOR

UWidgetAnimation* UWidgetAnimation::GetNullAnimation()
{
	static UWidgetAnimation* NullAnimation = nullptr;

	if (!NullAnimation)
	{
		NullAnimation = NewObject<UWidgetAnimation>(GetTransientPackage(), NAME_None);
		NullAnimation->AddToRoot();
		NullAnimation->MovieScene = NewObject<UMovieScene>(NullAnimation, FName("No Animation"));
		NullAnimation->MovieScene->AddToRoot();

		NullAnimation->MovieScene->SetDisplayRate(FFrameRate(20, 1));
	}

	return NullAnimation;
}

void UWidgetAnimation::SetDisplayLabel(const FString& InDisplayLabel)
{
	DisplayLabel = InDisplayLabel;
}

FText UWidgetAnimation::GetDisplayName() const
{
	const bool bHasDisplayLabel = !DisplayLabel.IsEmpty();
	return bHasDisplayLabel ? FText::FromString(DisplayLabel) : Super::GetDisplayName();
}

ETrackSupport UWidgetAnimation::IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{
	if (InTrackClass == UMovieSceneAudioTrack::StaticClass() ||
		InTrackClass == UMovieSceneEventTrack::StaticClass() ||
		InTrackClass == UMovieSceneMaterialParameterCollectionTrack::StaticClass() ||
		InTrackClass == UMovieSceneTimeWarpTrack::StaticClass())
	{
		return ETrackSupport::Supported;
	}

	return Super::IsTrackSupportedImpl(InTrackClass);
}

bool UWidgetAnimation::IsFilterSupportedImpl(const FString& InFilterName) const
{
	static const TArray<FString> SupportedFilters = {
		TEXT("Audio"),
		TEXT("Event"),
		TEXT("Keyed"),
		TEXT("Folder"),
		TEXT("Group"),
		TEXT("TimeDilation"),
		TEXT("TimeWarp"),
		TEXT("Unbound")
	};
	return SupportedFilters.Contains(InFilterName);
}

const FSlateBrush* UWidgetAnimation::GetCustomBrushForBinding(FGuid BindingID) const
{
	for (const FWidgetAnimationBinding& Binding : AnimationBindings)
	{
		if (Binding.AnimationGuid == BindingID)
		{
			if (Binding.DynamicBinding.Function)
			{
				return FAppStyle::GetBrush("Sequencer.DynamicBindingIconOverlay");
			}
		}
	}
	return nullptr;
}


void UWidgetAnimation::IterateDynamicBindings(TFunction<void(const FGuid&, FMovieSceneDynamicBinding&)> InCallback)
{
	for (FWidgetAnimationBinding& AnimationBinding : AnimationBindings)
	{
		InCallback(AnimationBinding.AnimationGuid, AnimationBinding.DynamicBinding);
	}
}

#endif

float UWidgetAnimation::GetStartTime() const
{
	return static_cast<float>(MovieScene->GetPlaybackRange().GetLowerBoundValue() / MovieScene->GetTickResolution());
}

float UWidgetAnimation::GetEndTime() const
{
	return static_cast<float>(MovieScene->GetPlaybackRange().GetUpperBoundValue() / MovieScene->GetTickResolution());
}

void UWidgetAnimation::BindToAnimationStarted(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate)
{
	if (ensure(Widget))
	{
		Widget->BindToAnimationStarted(this, Delegate);
	}
}

void UWidgetAnimation::UnbindFromAnimationStarted(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate)
{
	if (ensure(Widget))
	{
		Widget->UnbindFromAnimationStarted(this, Delegate);
	}
}

void UWidgetAnimation::UnbindAllFromAnimationStarted(UUserWidget* Widget)
{
	if (ensure(Widget))
	{
		Widget->UnbindAllFromAnimationStarted(this);
	}
}

void UWidgetAnimation::BindToAnimationFinished(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate)
{
	if (ensure(Widget))
	{
		Widget->BindToAnimationFinished(this, Delegate);
	}
}

void UWidgetAnimation::UnbindFromAnimationFinished(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate)
{
	if (ensure(Widget))
	{
		Widget->UnbindFromAnimationFinished(this, Delegate);
	}
}

void UWidgetAnimation::UnbindAllFromAnimationFinished(UUserWidget* Widget)
{
	if (ensure(Widget))
	{
		Widget->UnbindAllFromAnimationFinished(this);
	}
}


/* UMovieSceneAnimation overrides
 *****************************************************************************/


void UWidgetAnimation::BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context)
{
	UUserWidget* PreviewWidget = CastChecked<UUserWidget>(Context);

	// If it's the Root Widget
	if (&PossessedObject == PreviewWidget)
	{
		FWidgetAnimationBinding NewBinding;
		{
			NewBinding.AnimationGuid = ObjectId;
			NewBinding.WidgetName = PossessedObject.GetFName();
			NewBinding.bIsRootWidget = true;
		}

		AnimationBindings.Add(NewBinding);
		return;
	}
	
	UPanelSlot* PossessedSlot = Cast<UPanelSlot>(&PossessedObject);

	if ((PossessedSlot != nullptr) && (PossessedSlot->Content != nullptr))
	{
		// Save the name of the widget containing the slots. This is the object
		// to look up that contains the slot itself (the thing we are animating).
		FWidgetAnimationBinding NewBinding;
		{
			NewBinding.AnimationGuid = ObjectId;
			NewBinding.SlotWidgetName = PossessedSlot->GetFName();
			NewBinding.WidgetName = PossessedSlot->Content->GetFName();
			NewBinding.bIsRootWidget = false;
		}

		AnimationBindings.Add(NewBinding);
	}
	else if (PossessedSlot == nullptr)
	{
		FWidgetAnimationBinding NewBinding;
		{
			NewBinding.AnimationGuid = ObjectId;
			NewBinding.WidgetName = PossessedObject.GetFName();
			NewBinding.bIsRootWidget = false;
		}

		AnimationBindings.Add(NewBinding);
	}
}


bool UWidgetAnimation::CanPossessObject(UObject& Object, UObject* InPlaybackContext) const
{
	if (InPlaybackContext == nullptr)
	{
		return false;
	}

	UUserWidget* PreviewWidget = CastChecked<UUserWidget>(InPlaybackContext);

	if (&Object == PreviewWidget)
	{
		return true;
	}

	UPanelSlot* Slot = Cast<UPanelSlot>(&Object);

	if ((Slot != nullptr) && (Slot->Content == nullptr))
	{
		// can't possess empty slots.
		return false;
	}

	return (Object.IsA<UVisual>() && Object.IsIn(PreviewWidget));
}

void UWidgetAnimation::LocateBoundObjects(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& ResolveParams, TSharedPtr<const FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	if (ResolveParams.Context == nullptr)
	{
		return;
	}

	UUserWidget* PreviewWidget = CastChecked<UUserWidget>(ResolveParams.Context);
	if (PreviewWidget->WidgetTree == nullptr)
	{
		return;
	}

	for (const FWidgetAnimationBinding& Binding : AnimationBindings)
	{
		if (Binding.AnimationGuid == ObjectId)
		{
			UObject* FoundObject = Binding.FindRuntimeObject(*PreviewWidget->WidgetTree, *PreviewWidget, this, SharedPlaybackState);

			if (FoundObject)
			{
				OutObjects.Add(FoundObject);
			}
		}
	}
}

UMovieScene* UWidgetAnimation::GetMovieScene() const
{
	return MovieScene;
}

UObject* UWidgetAnimation::CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID)
{
	// Widget animations do not create separate director instances, but just re-use the UUserWidget from the playback context
	UUserWidget* WidgetContext = CastChecked<UUserWidget>(SharedPlaybackState->GetPlaybackContext(), ECastCheckedType::NullAllowed);
	if (!WidgetContext)
	{
		return nullptr;
	}

	return WidgetContext;
}

UObject* UWidgetAnimation::GetParentObject(UObject* Object) const
{
	UPanelSlot* Slot = Cast<UPanelSlot>(Object);

	if (Slot != nullptr)
	{
		// The slot is actually the child of the panel widget in the hierarchy,
		// but we want it to show up as a sub-object of the widget it contains
		// in the timeline so we return the content's GUID.
		return Slot->Content;
	}

	return nullptr;
}

void UWidgetAnimation::UnbindPossessableObjects(const FGuid& ObjectId)
{
	// mark dirty
	Modify();

	// remove animation bindings
	AnimationBindings.RemoveAll([&](const FWidgetAnimationBinding& Binding) {
		return Binding.AnimationGuid == ObjectId;
	});
}

void UWidgetAnimation::RemoveBinding(const UObject& PossessedObject)
{
	Modify();

	FName WidgetName = PossessedObject.GetFName();
	FName SlotWidgetName = NAME_None;

	const UPanelSlot* PossessedSlot = Cast<UPanelSlot>(&PossessedObject);

	if ((PossessedSlot != nullptr) && (PossessedSlot->Content != nullptr))
	{
		SlotWidgetName = PossessedSlot->GetFName();
		WidgetName = PossessedSlot->Content->GetFName();
	}

	AnimationBindings.RemoveAll([&](const FWidgetAnimationBinding& Binding) {
		return Binding.WidgetName.IsEqual(WidgetName) && Binding.SlotWidgetName.IsEqual(SlotWidgetName);
	});
}

void UWidgetAnimation::RemoveBinding(const FWidgetAnimationBinding& Binding)
{
	Modify();

	AnimationBindings.Remove(Binding);
}

bool UWidgetAnimation::IsPostLoadThreadSafe() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE

