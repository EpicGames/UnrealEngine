// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/WidgetAnimationBinding.h"
#include "UObject/Object.h"
#include "Components/Widget.h"
#include "Blueprint/WidgetTree.h"
#include "MovieSceneDynamicBindingInvoker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetAnimationBinding)


/* FWidgetAnimationBinding interface
 *****************************************************************************/

UObject* FWidgetAnimationBinding::FindRuntimeObject(const UWidgetTree& WidgetTree, UUserWidget& UserWidget ) const
{	
	return FindRuntimeObject(WidgetTree, UserWidget, nullptr, nullptr);
}

UObject* FWidgetAnimationBinding::FindRuntimeObject(const UWidgetTree& WidgetTree, UUserWidget& UserWidget, const UMovieSceneSequence* Sequence, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	if (Sequence && SharedPlaybackState && DynamicBinding.Function)
	{
		FMovieSceneDynamicBindingResolveResult ResolveResult = FMovieSceneDynamicBindingInvoker::ResolveDynamicBinding(SharedPlaybackState.ToSharedRef(), const_cast<UMovieSceneSequence*>(Sequence), MovieSceneSequenceID::Root, AnimationGuid, DynamicBinding);
		for (TObjectPtr<UObject> Object : ResolveResult.Objects)
		{
			if (Object.Get())
			{
				return Object.Get();
			}
		}
	}

	if (bIsRootWidget)
	{
		return &UserWidget;
	}

	UObject* FoundObject = WidgetTree.FindWidget(*WidgetName.ToString());

	if (FoundObject && (SlotWidgetName != NAME_None))
	{
		// if we were animating the slot, look up the slot that contains the widget 
		UWidget* WidgetObject = CastChecked<UWidget>(FoundObject);

		if (WidgetObject->Slot)
		{
			FoundObject = WidgetObject->Slot;
		}
	}

	return FoundObject;
}