// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/WidgetAnimationDelegateBinding.h"
#include "UMGPrivate.h"
#include "Blueprint/UserWidget.h"
#include "Animation/WidgetAnimation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetAnimationDelegateBinding)


UWidgetAnimationDelegateBinding::UWidgetAnimationDelegateBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UWidgetAnimationDelegateBinding::BindDynamicDelegates(UObject* InInstance) const
{
	if (UUserWidget* InUserWidget = Cast<UUserWidget>(InInstance))
	{
		for (int32 BindIndex = 0; BindIndex < WidgetAnimationDelegateBindings.Num(); ++BindIndex)
		{
			const FBlueprintWidgetAnimationDelegateBinding& Binding = WidgetAnimationDelegateBindings[BindIndex];

			FObjectProperty* AnimationProp = FindFProperty<FObjectProperty>(InUserWidget->GetClass(), Binding.AnimationToBind);
			if (AnimationProp)
			{
				UWidgetAnimation* AnimationPropData = Cast<UWidgetAnimation>(AnimationProp->GetObjectPropertyValue_InContainer(InUserWidget));
				if (ensure(AnimationPropData))
				{
					FWidgetAnimationDynamicEvent Delegate;
					Delegate.BindUFunction(InUserWidget, Binding.FunctionNameToBind);

					InUserWidget->BindToAnimationEvent(AnimationPropData, Delegate, Binding.Action, Binding.UserTag);
				}
			}
			else
			{
				UE_LOGF(LogUMG, Error, "Unable to find Animation %ls on Widget Class '%ls', was the animation deleted?", *Binding.AnimationToBind.ToString(), *InUserWidget->GetClass()->GetName());
			}
		}
	}
}

