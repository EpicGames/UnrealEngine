// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/WidgetAnimationHandle.h"

#include "Animation/UMGSequencePlayer.h"
#include "Animation/WidgetAnimationState.h"
#include "Blueprint/UserWidget.h"
#include "EdGraph/EdGraphPin.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetAnimationHandle)

FWidgetAnimationHandle::FWidgetAnimationHandle()
{
}

FWidgetAnimationHandle::FWidgetAnimationHandle(TSharedPtr<FWidgetAnimationState> InState)
	: WeakState(InState)
{
}

bool FWidgetAnimationHandle::IsValid() const
{
	return WeakState.IsValid();
}

UUMGSequencePlayer* FWidgetAnimationHandle::GetSequencePlayer() const
{
	if (FWidgetAnimationState* State = GetAnimationState())
	{
		return State->GetOrCreateLegacyPlayer();
	}
	return nullptr;
}

FWidgetAnimationState* FWidgetAnimationHandle::GetAnimationState() const
{
	return WeakState.Pin().Get();
}

TSharedPtr<FWidgetAnimationState> FWidgetAnimationHandle::PinAnimationState() const
{
	return WeakState.Pin();
}

FName FWidgetAnimationHandle::GetUserTag() const
{
	if (FWidgetAnimationState* State = GetAnimationState())
	{
		return State->GetUserTag();
	}
	return NAME_None;
}

void FWidgetAnimationHandle::SetUserTag(FName InUserTag)
{
	if (FWidgetAnimationState* State = GetAnimationState())
	{
		State->SetUserTag(InUserTag);
	}
}

FName UWidgetAnimationHandleFunctionLibrary::GetUserTag(const FWidgetAnimationHandle& Target)
{
	return Target.GetUserTag();
}

void UWidgetAnimationHandleFunctionLibrary::SetUserTag(FWidgetAnimationHandle& Target, FName InUserTag)
{
	Target.SetUserTag(InUserTag);
}

