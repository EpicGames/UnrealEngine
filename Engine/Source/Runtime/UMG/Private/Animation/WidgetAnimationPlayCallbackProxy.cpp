// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/WidgetAnimationPlayCallbackProxy.h"
#include "Animation/UMGSequencePlayer.h"
#include "Engine/World.h"
#include "TimerManager.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Containers/Ticker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetAnimationPlayCallbackProxy)

#define LOCTEXT_NAMESPACE "UMG"

UWidgetAnimationPlayCallbackProxy* UWidgetAnimationPlayCallbackProxy::CreatePlayAnimationProxyObject(
		UUMGSequencePlayer*& Result,
		UUserWidget* Widget,
		UWidgetAnimation* InAnimation,
		float StartAtTime,
		int32 NumLoopsToPlay,
		EUMGSequencePlayMode::Type PlayMode,
		float PlaybackSpeed)
{
	FWidgetAnimationHandle Unused;
	UWidgetAnimationPlayCallbackProxy* Proxy = NewPlayAnimationProxyObject(Unused, Widget, InAnimation, StartAtTime, NumLoopsToPlay, PlayMode, PlaybackSpeed);
	Result = Proxy->WidgetAnimationHandle.GetSequencePlayer();
	return Proxy;
}

UWidgetAnimationPlayCallbackProxy* UWidgetAnimationPlayCallbackProxy::NewPlayAnimationProxyObject(
		FWidgetAnimationHandle& Result,
		UUserWidget* Widget,
		UWidgetAnimation* InAnimation,
		float StartAtTime,
		int32 NumLoopsToPlay,
		EUMGSequencePlayMode::Type PlayMode,
		float PlaybackSpeed)
{
	UWidgetAnimationPlayCallbackProxy* Proxy = NewObject<UWidgetAnimationPlayCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->ExecutePlayAnimation(Widget, InAnimation, StartAtTime, NumLoopsToPlay, PlayMode, PlaybackSpeed);
	Result = Proxy->WidgetAnimationHandle;
	return Proxy;
}

UWidgetAnimationPlayCallbackProxy* UWidgetAnimationPlayCallbackProxy::CreatePlayAnimationTimeRangeProxyObject(
		UUMGSequencePlayer*& Result,
		UUserWidget* Widget,
		UWidgetAnimation* InAnimation,
		float StartAtTime,
		float EndAtTime,
		int32 NumLoopsToPlay,
		EUMGSequencePlayMode::Type PlayMode,
		float PlaybackSpeed)
{
	FWidgetAnimationHandle Unused;
	UWidgetAnimationPlayCallbackProxy* Proxy = NewPlayAnimationTimeRangeProxyObject(Unused, Widget, InAnimation, StartAtTime, EndAtTime, NumLoopsToPlay, PlayMode, PlaybackSpeed);
	Result = Proxy->WidgetAnimationHandle.GetSequencePlayer();
	return Proxy;
}

UWidgetAnimationPlayCallbackProxy* UWidgetAnimationPlayCallbackProxy::NewPlayAnimationTimeRangeProxyObject(
		FWidgetAnimationHandle& Result,
		UUserWidget* Widget,
		UWidgetAnimation* InAnimation,
		float StartAtTime,
		float EndAtTime,
		int32 NumLoopsToPlay,
		EUMGSequencePlayMode::Type PlayMode,
		float PlaybackSpeed)
{
	UWidgetAnimationPlayCallbackProxy* Proxy = NewObject<UWidgetAnimationPlayCallbackProxy>();
	Proxy->SetFlags(RF_StrongRefOnFrame);
	Proxy->ExecutePlayAnimationTimeRange(Widget, InAnimation, StartAtTime, EndAtTime, NumLoopsToPlay, PlayMode, PlaybackSpeed);
	Result = Proxy->WidgetAnimationHandle;
	return Proxy;
}

UWidgetAnimationPlayCallbackProxy::UWidgetAnimationPlayCallbackProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UWidgetAnimationPlayCallbackProxy::ExecutePlayAnimation(UUserWidget* Widget, UWidgetAnimation* InAnimation, float StartAtTime, int32 NumLoopsToPlay, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed)
{
	if (!Widget)
	{
		return;
	}

	WidgetAnimationHandle = Widget->PlayAnimation(InAnimation, StartAtTime, NumLoopsToPlay, PlayMode, PlaybackSpeed);
	FWidgetAnimationState* State = WidgetAnimationHandle.GetAnimationState();
	if (State)
	{
		OnFinishedHandle = State->GetOnWidgetAnimationFinished().AddUObject(this, &UWidgetAnimationPlayCallbackProxy::OnSequenceFinished);
	}
}

void UWidgetAnimationPlayCallbackProxy::ExecutePlayAnimationTimeRange(UUserWidget* Widget, UWidgetAnimation* InAnimation, float StartAtTime, float EndAtTime, int32 NumLoopsToPlay, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed)
{
	if (!Widget)
	{
		return;
	}

	WidgetAnimationHandle = Widget->PlayAnimationTimeRange(InAnimation, StartAtTime, EndAtTime, NumLoopsToPlay, PlayMode, PlaybackSpeed);
	FWidgetAnimationState* State = WidgetAnimationHandle.GetAnimationState();
	if (State)
	{
		OnFinishedHandle = State->GetOnWidgetAnimationFinished().AddUObject(this, &UWidgetAnimationPlayCallbackProxy::OnSequenceFinished);
	}
}

void UWidgetAnimationPlayCallbackProxy::OnSequenceFinished(FWidgetAnimationState& State)
{
	State.GetOnWidgetAnimationFinished().Remove(OnFinishedHandle);

	// We delay the Finish broadcast to next frame.
	FTSTicker::FDelegateHandle TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UWidgetAnimationPlayCallbackProxy::OnAnimationFinished));
}


bool UWidgetAnimationPlayCallbackProxy::OnAnimationFinished(float /*DeltaTime*/)
{
	Finished.Broadcast();

	// Returning false, disable the ticker.
	return false;
}

#undef LOCTEXT_NAMESPACE

