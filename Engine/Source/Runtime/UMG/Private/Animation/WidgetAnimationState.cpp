// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/WidgetAnimationState.h"

#include "Animation/UMGSequencePlayer.h"
#include "Animation/UMGSequenceTickManager.h"
#include "Animation/WidgetAnimation.h"
#include "Animation/WidgetAnimationHandle.h"
#include "Blueprint/UserWidget.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "Evaluation/SequenceDirectorPlaybackCapability.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieSceneLegacyPlayer.h"
#include "MovieSceneTimeHelpers.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UMGPrivate.h"

namespace UE::UMG
{

	bool GVarAnimationDetailedLogging = false;
	FAutoConsoleVariableRef CVarAnimationDetailedLogging(
		TEXT("UMG.AnimationDetailedLogging"),
		GVarAnimationDetailedLogging,
		TEXT("(Default: false) Whether to print out detailed information about UMG animations.")
	);

	bool GVarAnimationMarkers = false;
	FAutoConsoleVariableRef CVarAnimationMarkers(
		TEXT("UMG.AnimationMarkers"),
		GVarAnimationMarkers,
		TEXT("(Default: false) Whether to emit profiling frame markers for starting and stopping UMG animations.")
	);

	struct FUMGLegacyPlayerProvider : public UE::MovieScene::ILegacyPlayerProviderPlaybackCapability
	{
		FWidgetAnimationState* State = nullptr;

		FUMGLegacyPlayerProvider(FWidgetAnimationState* InState)
			: State(InState)
		{
			check(State);
		}

		virtual IMovieScenePlayer* CreateLegacyPlayer(TSharedRef<UE::MovieScene::FSharedPlaybackState> InSharedPlaybackState) override
		{
			return State->GetOrCreateLegacyPlayer();
		}
	};

} // namespace UE::UMG

FWidgetAnimationState::FWidgetAnimationState()
{
	UserTag = NAME_None;
	BlockedDeltaTimeCompensation = 0.f;
	bRestoreState = false;
	bIsBeginningPlay = false;
	bIsStopping = false;
	bIsPendingDelete = false;
}

void FWidgetAnimationState::Initialize(UWidgetAnimation* InAnimation, UUserWidget* InUserWidget)
{
	using namespace UE::UMG;
	using namespace UE::MovieScene;

	check(InAnimation && InUserWidget);

	Animation = InAnimation;
	WeakUserWidget = InUserWidget;

	check(InUserWidget->AnimationTickManager);

	const bool bNeedsLegacyPlayer = NeedsLegacyPlayer();
	const bool bNeedsPrivateLinker = EnumHasAnyFlags(InAnimation->GetFlags(), EMovieSceneSequenceFlags::BlockingEvaluation);

	UMovieSceneEntitySystemLinker* Linker = InUserWidget->AnimationTickManager->GetLinker();
	if (bNeedsPrivateLinker)
	{
		Linker = UMovieSceneEntitySystemLinker::CreateLinker(InUserWidget->GetWorld(), UE::MovieScene::EEntitySystemLinkerRole::UMG);
		PrivateLinker = Linker;
	}

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	FRootInstanceHandle RootInstanceHandle = InstanceRegistry->AllocateRootInstance(*InAnimation, InUserWidget);
	{
		FSequenceInstance& RootInstance = InstanceRegistry->MutateInstance(RootInstanceHandle);
		TSharedRef<FSharedPlaybackState> SharedPlaybackState = RootInstance.GetSharedPlaybackState();
		WeakPlaybackState = SharedPlaybackState;

		SharedPlaybackState->AddCapability<FMovieSceneEvaluationState>();

		// The UUserWidget that owns us is its own Sequence Director Blueprint, so don't create a cycle of 
		// strong GC references: create the cache right away and initialize it to use weak references.
		const bool bUseStrongDirectorCache = false;
		SharedPlaybackState->AddCapability<FSequenceDirectorPlaybackCapability>(bUseStrongDirectorCache);

		if (bNeedsLegacyPlayer)
		{
			GetOrCreateLegacyPlayer();
			check(LegacyPlayer);
			SharedPlaybackState->AddCapability<FPlayerIndexPlaybackCapability>(LegacyPlayer->GetUniqueIndex());
		}
		else
		{
			SharedPlaybackState->AddCapability<FUMGLegacyPlayerProvider>(this);
		}

		RootInstance.Initialize();
	}

	PlaybackManager.Initialize(InAnimation);
	PlaybackManager.SetDissectLooping(EMovieSceneLoopDissection::DissectOne);

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		UE_LOGF(LogUMG, Log, "Animation: Initializing '%ls' playing on '%ls' (%p), instance [%d]%ls", 
				*GetNameSafe(Animation), *GetNameSafe(InUserWidget), InUserWidget, RootInstanceHandle.InstanceID,
				(LegacyPlayer ? TEXT(", with legacy player") : TEXT("")));
	}
#endif
}

bool FWidgetAnimationState::NeedsLegacyPlayer() const
{
	UUserWidget* UserWidget = WeakUserWidget.Get();
	if (!UserWidget)
	{
		return false;
	}

	const ERequiresLegacyPlayer Requirement = UserWidget->GetLegacyPlayerRequirement();
	if (Requirement == ERequiresLegacyPlayer::AutoDetect)
	{
		// Require a legacy player if the widget has a native C++ sub-class. This is because that
		// sub-class could override one of the virtual methods that takes a legacy player (there is
		// no way for us to detect that at compile time, sadly).
		// If we only have Blueprint sub-classes, they can't use the legacy player except via
		// lazy-creation on the FWidgetAnimationHandle, so we don't need to create one right away.
		UClass* CurrentClass = UserWidget->GetClass();
		while (CurrentClass && CurrentClass != UUserWidget::StaticClass())
		{
			if (EnumHasAnyFlags(CurrentClass->GetClassFlags(), CLASS_Native))
			{
				return true;
			}

			CurrentClass = CurrentClass->GetSuperClass();
		}

		return false;
	}
	else
	{
		return Requirement == ERequiresLegacyPlayer::Yes;
	}
}

void FWidgetAnimationState::FlushIfPrivateLinker()
{
	using namespace UE::MovieScene;

	if (PrivateLinker)
	{
		TSharedPtr<FMovieSceneEntitySystemRunner> EntitySystemRunner = PrivateLinker->GetRunner();
		EntitySystemRunner->Flush();
	}
}

void FWidgetAnimationState::Tick(float InDeltaSeconds)
{
	using namespace UE::MovieScene;

	UUserWidget* UserWidget = WeakUserWidget.Get();
	TSharedPtr<FSharedPlaybackState> SharedPlaybackState = WeakPlaybackState.Pin();
	if (!UserWidget || !SharedPlaybackState)
	{
#if !NO_LOGGING
		if (UE::UMG::GVarAnimationDetailedLogging)
		{
			UE_LOGF(LogUMG, Log, "Animation: Ticking '%ls' on '%ls' (%p) aborted, invalid state.",
					*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget);
		}
#endif
		return;
	}

	TSharedPtr<FMovieSceneEntitySystemRunner> EntitySystemRunner = SharedPlaybackState->GetRunner();
	if (!EntitySystemRunner)
	{
#if !NO_LOGGING
		if (UE::UMG::GVarAnimationDetailedLogging)
		{
			UE_LOGF(LogUMG, Log, "Animation: Ticking '%ls' on '%ls' (%p) aborted, shared playback state has no runner.",
					*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget);
		}
#endif
		return;
	}

	if (bIsStopping || bIsBeginningPlay)
	{
		return;
	}

	if (EntitySystemRunner->IsCurrentlyEvaluating())
	{
		BlockedDeltaTimeCompensation += InDeltaSeconds;
		return;
	}

	const EMovieScenePlayerStatus::Type PreviousPlaybackStatus = PlaybackManager.GetPlaybackStatus();
	if (PreviousPlaybackStatus == EMovieScenePlayerStatus::Playing)
	{
		// Update root transform in case it has changed.
		if (const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy())
		{
			PlaybackManager.SetPlaybackTimeTransform(Hierarchy->GetRootTransform());
		}

		// Get the evaluation contexts for this tick.
		FMovieScenePlaybackManager::FContexts TickContexts;
		PlaybackManager.Update(InDeltaSeconds, TickContexts);

		const EMovieScenePlayerStatus::Type NextPlaybackStatus = PlaybackManager.GetPlaybackStatus();
		const bool bNeedsFinalUpdate = (NextPlaybackStatus == EMovieScenePlayerStatus::Stopped);

		// Queue up the evalutions as needed. If we finished playback, setup our OnStopped callback
		// to tear things down.
		for (int32 Index = 0; Index < TickContexts.Num(); ++Index)
		{
			FMovieSceneContext& TickContext(TickContexts[Index]);

			FSimpleDelegate OnFlushDelegate;
			ERunnerUpdateFlags UpdateFlags = ERunnerUpdateFlags::None;
			if (bNeedsFinalUpdate && Index == TickContexts.Num() - 1)
			{
				OnFlushDelegate = FSimpleDelegate::CreateSP(this, &FWidgetAnimationState::OnStopped);
				UpdateFlags = ERunnerUpdateFlags::Flush;
				bIsStopping = true;
			}
			EntitySystemRunner->QueueUpdate(
					TickContext, SharedPlaybackState->GetRootInstanceHandle(), 
					MoveTemp(OnFlushDelegate),
					UpdateFlags);

#if !NO_LOGGING
			if (UE::UMG::GVarAnimationDetailedLogging)
			{
				UE_LOGF(LogUMG, Log, "Animation: Ticking '%ls' on '%ls' (%p), Time=%ls, Status=%ls, Direction=%ls (update %d)",
						*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget,
						*LexToString(TickContext.GetTime()),
						*UEnum::GetValueAsString(TickContext.GetStatus()),
						((TickContext.GetDirection() == EPlayDirection::Forwards) ? TEXT("Forwards") : TEXT("Backwards")),
						Index);
			}
#endif
		}

#if !NO_LOGGING
		if (UE::UMG::GVarAnimationDetailedLogging)
		{
			UE_LOGF(LogUMG, Log, "Animation: Ticking done '%ls' on '%ls' (%p), NumLoopsCompleted=%d, PlaybackSpeed=%f",
					*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget,
					PlaybackManager.GetNumLoopsCompleted(),
					PlaybackManager.GetPlayRate());
		}
#endif

		FlushIfPrivateLinker();
	}
	else
	{
#if !NO_LOGGING
		if (UE::UMG::GVarAnimationDetailedLogging)
		{
			UE_LOGF(LogUMG, Log, "Animation: Ticking '%ls' on '%ls' (%p) skipped, state not playing.",
					*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget);
		}
#endif
	}
}

void FWidgetAnimationState::OnBegunPlay()
{
	using namespace UE::MovieScene;

	bIsBeginningPlay = false;

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		UUserWidget* UserWidget = WeakUserWidget.Get();
		UE_LOGF(LogUMG, Log, "Animation: OnBegunPlay '%ls' on '%ls' (%p)",
				*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget);
	}
#endif
}

void FWidgetAnimationState::OnStopped()
{
	using namespace UE::MovieScene;

	TSharedPtr<FSharedPlaybackState> SharedPlaybackState = WeakPlaybackState.Pin();
	if (SharedPlaybackState)
	{
		TSharedPtr<FMovieSceneEntitySystemRunner> EntitySystemRunner = SharedPlaybackState->GetRunner();
		const bool bNeedsFinalFlush = EntitySystemRunner->QueueFinalUpdate(SharedPlaybackState->GetRootInstanceHandle());

		// Even if our request to Finish the instance was queued, we can wait until the next flush for those effects to be seen
		// This will most likely happen immediately anyway since the runner will keep looping until its queue is empty,
		// And we are already inside an active evaluation

		if (bNeedsFinalFlush)
		{
			FlushIfPrivateLinker();
		}
	}

	if (SharedPlaybackState && bRestoreState)
	{
		SharedPlaybackState->GetPreAnimatedState().RestorePreAnimatedState();
	}

	bIsStopping = false;
	bIsPendingDelete = true;

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UUserWidget* UserWidget = WeakUserWidget.Get();
		UE_LOGF(LogUMG, Log, "Animation: OnStopped '%ls' on '%ls' (%p) %ls%ls",
				*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget,
				OnWidgetAnimationFinishedEvent.IsBound() ? TEXT(", OnWidgetAnimationFinishedEvent bound") : TEXT(""),
				(LegacyPlayer && LegacyPlayer->OnSequenceFinishedPlaying().IsBound()) ? TEXT(", OnSequenceFinishedPlaying bound") : TEXT(""));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

	UUserWidget* UserWidget = WeakUserWidget.Get();
	if (UserWidget)
	{
		if (UE::UMG::GVarAnimationMarkers && Animation)
		{
			CSV_EVENT_GLOBAL(TEXT("Stop Animation [%s::%s]"), *GetNameSafe(UserWidget), *GetNameSafe(Animation));
		}

		UserWidget->BroadcastAnimationFinishedPlaying(*this);
	}

	OnWidgetAnimationFinishedEvent.Broadcast(*this);

	if (LegacyPlayer)
	{
		LegacyPlayer->BroadcastSequenceFinishedPlaying();
	}
}

UUMGSequencePlayer* FWidgetAnimationState::GetOrCreateLegacyPlayer()
{
	if (!LegacyPlayer)
	{
		if (UUserWidget* UserWidget = WeakUserWidget.Get())
		{
			LegacyPlayer = NewObject<UUMGSequencePlayer>(UserWidget);
			LegacyPlayer->InitSequencePlayer(*this);
		}
	}
	return LegacyPlayer;
}

UUMGSequencePlayer* FWidgetAnimationState::GetLegacyPlayer() const
{
	return LegacyPlayer;
}

FWidgetAnimationHandle FWidgetAnimationState::GetAnimationHandle() const
{
	FWidgetAnimationState* MutableThis = const_cast<FWidgetAnimationState*>(this);
	return FWidgetAnimationHandle(SharedThis(MutableThis));
}

bool FWidgetAnimationState::IsPlayingForward() const
{
	return PlaybackManager.IsPlayingForward();
}

EMovieScenePlayerStatus::Type FWidgetAnimationState::GetPlaybackStatus() const
{
	return PlaybackManager.GetPlaybackStatus();
}

void FWidgetAnimationState::SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus)
{
	PlaybackManager.SetPlaybackStatus(InPlaybackStatus);
}

FQualifiedFrameTime FWidgetAnimationState::GetCurrentTime() const
{
	const FFrameTime CurrentPosition = PlaybackManager.GetCurrentTime();
	return FQualifiedFrameTime(CurrentPosition, PlaybackManager.GetDisplayRate());
}

void FWidgetAnimationState::SetCurrentTime(float InTime)
{
	const FFrameTime JumpFrameTime = PlaybackManager.GetDisplayRate().AsFrameTime(InTime);
	PlaybackManager.SetCurrentTime(JumpFrameTime);
}

void FWidgetAnimationState::SetNumLoopsToPlay(int32 InNumLoopsToPlay)
{
	// For UMG animations, we treat a whole ping-pong as a loop.
	if (PlayMode == EUMGSequencePlayMode::PingPong)
	{
		PlaybackManager.SetNumLoopsToPlay(2 * InNumLoopsToPlay);
	}
	else
	{
		PlaybackManager.SetNumLoopsToPlay(InNumLoopsToPlay);
	}
}

void FWidgetAnimationState::SetPlaybackSpeed(float InPlaybackSpeed)
{
	PlaybackManager.SetPlayRate(InPlaybackSpeed);
}

void FWidgetAnimationState::Play(const FWidgetAnimationStatePlayParams& PlayParams)
{
	using namespace UE::MovieScene;

	TSharedPtr<FSharedPlaybackState> SharedPlaybackState = WeakPlaybackState.Pin();
	if (!ensure(SharedPlaybackState))
	{
		return;
	}

	UUserWidget* UserWidget = WeakUserWidget.Get();

	if (UE::UMG::GVarAnimationMarkers && Animation && UserWidget)
	{
		CSV_EVENT_GLOBAL(TEXT("Play Animation [%s::%s]"), *GetNameSafe(UserWidget), *GetNameSafe(Animation));
	}

	bRestoreState = PlayParams.bRestoreState;
	if (bRestoreState)
	{
		SharedPlaybackState->GetPreAnimatedState().EnableGlobalPreAnimatedStateCapture();
	}

	bIsBeginningPlay = true;

	const FFrameRate DisplayRate = PlaybackManager.GetDisplayRate();

	PlaybackManager.SetStartOffset(FFrameTime(0));
	if (PlayParams.StartOffset.IsSet())
	{
		const FFrameTime StartOffsetTime = DisplayRate.AsFrameTime(PlayParams.StartOffset.GetValue());
		PlaybackManager.SetStartOffset(StartOffsetTime);
	}

	PlaybackManager.SetEndOffset(FFrameTime(0));
	if (PlayParams.EndOffset.IsSet())
	{
		const FFrameTime EndOffsetTime = DisplayRate.AsFrameTime(PlayParams.EndOffset.GetValue());
		PlaybackManager.SetEndOffset(PlaybackManager.GetEndOffset() + EndOffsetTime);
	}

	PlaybackManager.ClearPlaybackEndTime();
	if (PlayParams.EndAtTime.IsSet())
	{
		const FFrameTime EndFrameTime = DisplayRate.AsFrameTime(PlayParams.EndAtTime.GetValue());
		PlaybackManager.SetPlaybackEndTime(EndFrameTime);
	}

	PlayMode = PlayParams.PlayMode;
	if (PlayMode == EUMGSequencePlayMode::Reverse)
	{
		// When playing in reverse, subtract the start time from the end.
		const FFrameTime StartFrameTime = DisplayRate.AsFrameTime(PlayParams.StartAtTime);
		const FFrameTime ReverseStartFrameTime = PlaybackManager.GetEffectiveEndTime() - StartFrameTime;
		PlaybackManager.SetCurrentTime(ReverseStartFrameTime);
	}
	else
	{
		const FFrameTime StartFrameTime = DisplayRate.AsFrameTime(PlayParams.StartAtTime);
		PlaybackManager.SetCurrentTime(StartFrameTime);
	}

	PlaybackManager.SetPlayDirection(EPlayDirection::Forwards);
	PlaybackManager.SetPingPongPlayback(false);
	switch (PlayMode)
	{
		case EUMGSequencePlayMode::Forward:
			break;
		case EUMGSequencePlayMode::PingPong:
			PlaybackManager.SetPingPongPlayback(true);
			break;
		case EUMGSequencePlayMode::Reverse:
			PlaybackManager.SetPlayDirection(EPlayDirection::Backwards);
			break;
	}

	// For UMG animations, a whole ping-pong is a single loop.
	PlaybackManager.SetNumLoopsToPlay(
			(PlayMode == EUMGSequencePlayMode::PingPong) ?
				2 * PlayParams.NumLoopsToPlay :
				PlayParams.NumLoopsToPlay);
	PlaybackManager.ResetNumLoopsCompleted();

	PlaybackManager.SetPlayRate(PlayParams.PlaybackSpeed);
	PlaybackManager.SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	// Setup time warping.
	if (const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy())
	{
		PlaybackManager.SetPlaybackTimeTransform(Hierarchy->GetRootTransform());
		PlaybackManager.SetTransformPlaybackTime(true);
	}
	else
	{
		PlaybackManager.SetTransformPlaybackTime(false);
	}

	// TODO: we shouldn't have to queue an update right away but we preserve the old behavior for now.
	TSharedPtr<FMovieSceneEntitySystemRunner> Runner = SharedPlaybackState->GetRunner();
	if (Runner)
	{
		// Use UpdateToNextTick here so that we don't re-evaluate the first tick a second time when
		// we start calling Tick().
		const FMovieSceneContext FirstContext = PlaybackManager.UpdateToNextTick();

		Runner->QueueUpdate(
			FirstContext,
			SharedPlaybackState->GetRootInstanceHandle(),
			FSimpleDelegate::CreateSP(this, &FWidgetAnimationState::OnBegunPlay),
			ERunnerUpdateFlags::Flush);
	}

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		UE_LOGF(LogUMG, Log, "Animation: Play '%ls' on '%ls' (%p), StartTime=%ls, NumLoopsToPlay=%d, PlaybackSpeed=%f, PlayMode=%ls%ls",
				*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget,
				*LexToString(PlaybackManager.GetCurrentTime()),
				PlaybackManager.GetNumLoopsToPlay(),
				PlaybackManager.GetPlayRate(),
				PlaybackManager.GetPlayDirection() == EPlayDirection::Forwards ? TEXT("Forwards") : TEXT("Backwards"),
				PlaybackManager.IsPingPongPlayback() ? TEXT(", PingPong") : TEXT(""));
	}
#endif

	FlushIfPrivateLinker();
}

void FWidgetAnimationState::Stop()
{
	using namespace UE::MovieScene;

	if (PlaybackManager.GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped)
	{
		return;
	}

	PlaybackManager.SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);

	PlaybackManager.SetCurrentTimeOffset(0);

	bool bFlushPrivateLinker = false;

	if (TSharedPtr<FSharedPlaybackState> SharedPlaybackState = WeakPlaybackState.Pin())
	{
		const FRootInstanceHandle& RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
		UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
		const FSequenceInstance& RootInstance = Linker->GetInstanceRegistry()->GetInstance(RootInstanceHandle);
		if (RootInstance.HasEverUpdated())
		{
			TSharedPtr<FMovieSceneEntitySystemRunner> Runner = SharedPlaybackState->GetRunner();

			const FMovieSceneContext ReturnToStartContext = PlaybackManager.UpdateAtCurrentTime();
			Runner->QueueUpdate(
					ReturnToStartContext,
					SharedPlaybackState->GetRootInstanceHandle(),
					FSimpleDelegate::CreateSP(this, &FWidgetAnimationState::OnStopped),
					ERunnerUpdateFlags::Flush);

			bIsStopping = true;
			bFlushPrivateLinker = true;
		}
		else
		{
			OnStopped();
		}
	}
	else
	{
		OnStopped();
	}

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		UUserWidget* UserWidget = WeakUserWidget.Get();
		UE_LOGF(LogUMG, Log, "Animation: Stop '%ls' on '%ls' (%p)",
				*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget);
	}
#endif

	if (bFlushPrivateLinker)
	{
		FlushIfPrivateLinker();
	}
}

void FWidgetAnimationState::Pause()
{
	using namespace UE::MovieScene;

	// Should be Paused but old behavior was to set to Stopped.
	PlaybackManager.SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);

	// Evaluate the sequence at its current time, with a status of 'Stopped' to ensure that animated state 
	// pauses correctly. (ie. audio sounds should stop/pause)
	if (TSharedPtr<FSharedPlaybackState> SharedPlaybackState = WeakPlaybackState.Pin())
	{
		TSharedPtr<FMovieSceneEntitySystemRunner> Runner = SharedPlaybackState->GetRunner();

		const FMovieSceneContext PauseContext = PlaybackManager.UpdateAtCurrentTime();
		Runner->QueueUpdate(
				PauseContext,
				SharedPlaybackState->GetRootInstanceHandle(),
				ERunnerUpdateFlags::Flush);
	}

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		UUserWidget* UserWidget = WeakUserWidget.Get();
		UE_LOGF(LogUMG, Log, "Animation: Pause '%ls' on '%ls' (%p), PauseTime=%ls",
				*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget,
				*LexToString(PlaybackManager.GetCurrentTime()));
	}
#endif

	FlushIfPrivateLinker();
}

void FWidgetAnimationState::Reverse()
{
	PlaybackManager.ReversePlayDirection();
}

void FWidgetAnimationState::RemoveEvaluationData()
{
	using namespace UE::MovieScene;

	if (TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = WeakPlaybackState.Pin())
	{
		UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker();
		FRootInstanceHandle RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
		FSequenceInstance& RootInstance = Linker->GetInstanceRegistry()->MutateInstance(RootInstanceHandle);

		TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
		if (Runner->IsCurrentlyEvaluating())
		{
			Runner->FlushOutstanding();
		}

		RootInstance.Ledger.UnlinkEverything(Linker);
		RootInstance.InvalidateCachedData();
	}

	if (LegacyPlayer)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LegacyPlayer->GetEvaluationTemplate().ResetDirectorInstances();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		UUserWidget* UserWidget = WeakUserWidget.Get();
		UE_LOGF(LogUMG, Log, "Animation: RemoveEvaluationData '%ls' on '%ls' (%p)",
				*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget);
	}
#endif
}

void FWidgetAnimationState::TearDown()
{
	using namespace UE::MovieScene;

	FRootInstanceHandle RootInstanceHandle;
	UMovieSceneEntitySystemLinker* EntitySystemLinker = nullptr;

	if (TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = WeakPlaybackState.Pin())
	{
		RootInstanceHandle = SharedPlaybackState->GetRootInstanceHandle();
		EntitySystemLinker = SharedPlaybackState->GetLinker();
	}

	if (LegacyPlayer)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = LegacyPlayer->GetEvaluationTemplate();
		EvaluationTemplate.TearDown();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	LegacyPlayer = nullptr;

	WeakPlaybackState.Reset();

	if (EntitySystemLinker && RootInstanceHandle.IsValid())
	{
		EntitySystemLinker->DestroyInstanceImmediately(RootInstanceHandle);
	}

#if !NO_LOGGING
	if (UE::UMG::GVarAnimationDetailedLogging)
	{
		UUserWidget* UserWidget = WeakUserWidget.Get();
		UE_LOGF(LogUMG, Log, "Animation: TearDown '%ls' on '%ls' (%p)",
				*GetNameSafe(Animation), *GetNameSafe(UserWidget), UserWidget);
	}
#endif
}

bool FWidgetAnimationState::IsValid() const
{
	return WeakPlaybackState.IsValid();
}

void FWidgetAnimationState::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Animation);
	Collector.AddReferencedObject(LegacyPlayer);
	Collector.AddReferencedObject(PrivateLinker);
}

