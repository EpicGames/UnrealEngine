// Copyright Epic Games, Inc. All Rights Reserved.


#include "Animation/WidgetMaterialTrackUtilities.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "Materials/MaterialInterface.h"
#include "Styling/SlateBrush.h"
#include "Components/Widget.h"
#include "Fonts/SlateFontInfo.h"


template<typename T> struct TMaterialStructType
{
	static FName GetTypeName()
	{
		static_assert(!sizeof(T), "TMaterialStructType trait must be specialized for this type.");
		return NAME_None;
	}
};


template<> struct TMaterialStructType<FSlateBrush>
{
	static FName GetTypeName()
	{
		static const FName TypeName = "SlateBrush";
		return TypeName;
	}

	static FString GetPropertyName()
	{
		// Note: usign a custom name here.  FSlateBrush::ResourceObject is not a descriptive name
		return NSLOCTEXT("WidgetMaterialTrackUtilities", "BrushMaterialName", "Brush Material").ToString();
	}

	static UMaterialInterface* GetMaterial(void* Data)
	{
		FSlateBrush* Brush = (FSlateBrush*)Data;
		return Cast<UMaterialInterface>(Brush->GetResourceObject());
	}

	static void SetMaterial(void* Data, UMaterialInterface* Material)
	{
		FSlateBrush* Brush = (FSlateBrush*)Data;
		Brush->SetResourceObject(Material);
	}
};

template<> struct TMaterialStructType<FSlateFontInfo>
{
	static FName GetTypeName()
	{
		static const FName TypeName = "SlateFontInfo";
		return TypeName;
	}

	static FString GetPropertyName()
	{
		return GET_MEMBER_NAME_STRING_CHECKED(FSlateFontInfo, FontMaterial);
	}

	static UMaterialInterface* GetMaterial(void* Data)
	{
		FSlateFontInfo* Font = (FSlateFontInfo*)Data;
		return Cast<UMaterialInterface>(Font->FontMaterial);
	}

	static void SetMaterial(void* Data, UMaterialInterface* Material)
	{
		FSlateFontInfo* Font = (FSlateFontInfo*)Data;
		Font->FontMaterial = Material;
	}
};

template<> struct TMaterialStructType<FFontOutlineSettings>
{
	static FName GetTypeName()
	{
		static const FName TypeName = "FontOutlineSettings";
		return TypeName;
	}

	static FString GetPropertyName()
	{
		return GET_MEMBER_NAME_STRING_CHECKED(FFontOutlineSettings, OutlineMaterial);
	}

	static UMaterialInterface* GetMaterial(void* Data)
	{
		FFontOutlineSettings* OutlineSettings = (FFontOutlineSettings*)Data;
		return Cast<UMaterialInterface>(OutlineSettings->OutlineMaterial);
	}

	static void SetMaterial(void* Data, UMaterialInterface* Material)
	{
		FFontOutlineSettings* OutlineSettings = (FFontOutlineSettings*)Data;
		OutlineSettings->OutlineMaterial = Material;
	}
};

UMaterialInterface* FWidgetMaterialHandle::GetMaterial() const
{
	if(TypeName == TMaterialStructType<FSlateFontInfo>::GetTypeName())
	{
		return TMaterialStructType<FSlateFontInfo>::GetMaterial(Data);
	}
	else if(TypeName == TMaterialStructType<FSlateBrush>::GetTypeName())
	{
		return TMaterialStructType<FSlateBrush>::GetMaterial(Data);
	}
	else if(TypeName == TMaterialStructType<FFontOutlineSettings>::GetTypeName())
	{
		return TMaterialStructType<FFontOutlineSettings>::GetMaterial(Data);
	}
	else
	{
		return nullptr;
	}
}

void FWidgetMaterialHandle::SetMaterial(UMaterialInterface* InMaterial, UWidget* OwnerWidget)
{
	if(TypeName == TMaterialStructType<FSlateFontInfo>::GetTypeName())
	{
		TMaterialStructType<FSlateFontInfo>::SetMaterial(Data, InMaterial);
	}
	else if(TypeName == TMaterialStructType<FSlateBrush>::GetTypeName())
	{
		TMaterialStructType<FSlateBrush>::SetMaterial(Data, InMaterial);
	}
	else if(TypeName == TMaterialStructType<FFontOutlineSettings>::GetTypeName())
	{
		TMaterialStructType<FFontOutlineSettings>::SetMaterial(Data, InMaterial);
	}

	TSharedPtr<SWidget> RawWidget = OwnerWidget->GetCachedWidget();
	if (RawWidget.IsValid())
	{
		RawWidget->Invalidate(EInvalidateWidget::LayoutAndVolatility);
		OwnerWidget->SynchronizeProperties();
	}
}


FWidgetMaterialHandle GetPropertyValueByPath(void* DataObject, UStruct* PropertySource, TArrayView<const FName> PropertyPath, int32 PathIndex )
{
	if ( DataObject != nullptr && PathIndex < PropertyPath.Num() )
	{
		for ( TFieldIterator<FProperty> PropertyIterator( PropertySource ); PropertyIterator; ++PropertyIterator )
		{
			FProperty* Property = *PropertyIterator;
			if ( Property != nullptr && Property->GetFName() == PropertyPath[PathIndex] )
			{
				// Only struct properties are relevant for the search.
				FStructProperty* StructProperty = CastField<FStructProperty>( Property );
				if ( StructProperty == nullptr )
				{
					return FWidgetMaterialHandle();
				}

				if ( PathIndex == PropertyPath.Num() - 1 )
				{
					const FName StructName = StructProperty->Struct->GetFName();
					if (StructName == TMaterialStructType<FSlateFontInfo>::GetTypeName() ||
						StructName == TMaterialStructType<FSlateBrush>::GetTypeName() ||
						StructName == TMaterialStructType<FFontOutlineSettings>::GetTypeName() )
					{
						FWidgetMaterialHandle Handle(StructName, StructProperty->ContainerPtrToValuePtr<void>(DataObject));
						return Handle;
					}
					else
					{
						return FWidgetMaterialHandle();
					}
				}
				else
				{
					return GetPropertyValueByPath(Property->ContainerPtrToValuePtr<void>( DataObject ), StructProperty->Struct, PropertyPath, PathIndex + 1 );
				}
			}
		}
	}
	return FWidgetMaterialHandle();
}

FWidgetMaterialHandle WidgetMaterialTrackUtilities::GetMaterialHandle(UWidget* Widget, TArrayView<const FName> BrushPropertyNamePath)
{
	return GetPropertyValueByPath(Widget, Widget->GetClass(), BrushPropertyNamePath, 0);
}

void GetMaterialBrushPropertyPathsRecursive(void* DataObject, UStruct* PropertySource, TArray<FProperty*>& PropertyPath, TArray<FWidgetMaterialPropertyPath>& MaterialBrushPropertyPaths)
{
	if ( DataObject != nullptr )
	{
		for ( TFieldIterator<FProperty> PropertyIterator( PropertySource ); PropertyIterator; ++PropertyIterator )
		{
			FProperty* Property = *PropertyIterator;
			if ( Property != nullptr && Property->HasAnyPropertyFlags( CPF_Deprecated ) == false )
			{
				PropertyPath.Add( Property );

				FStructProperty* StructProperty = CastField<FStructProperty>( Property );
				if ( StructProperty != nullptr )
				{
					const FName StructName = StructProperty->Struct->GetFName();
					void* Data = Property->ContainerPtrToValuePtr<void>(DataObject);

					UMaterialInterface* MaterialInterface = nullptr;

					FString PropertyName;
					if(StructName == TMaterialStructType<FSlateFontInfo>::GetTypeName())
					{
						MaterialInterface = TMaterialStructType<FSlateFontInfo>::GetMaterial(Data);
						PropertyName = TMaterialStructType<FSlateFontInfo>::GetPropertyName();
					}
					else if(StructName == TMaterialStructType<FSlateBrush>::GetTypeName())
					{
						MaterialInterface =TMaterialStructType<FSlateBrush>::GetMaterial(Data);
						PropertyName = TMaterialStructType<FSlateBrush>::GetPropertyName();
					}
					else if(StructName == TMaterialStructType<FFontOutlineSettings>::GetTypeName())
					{
						MaterialInterface =TMaterialStructType<FFontOutlineSettings>::GetMaterial(Data);
						PropertyName = TMaterialStructType<FFontOutlineSettings>::GetPropertyName();

					}

					if(MaterialInterface)
					{
						MaterialBrushPropertyPaths.Emplace(PropertyPath, PropertyName);
					}
				
					GetMaterialBrushPropertyPathsRecursive( StructProperty->ContainerPtrToValuePtr<void>( DataObject ), StructProperty->Struct, PropertyPath, MaterialBrushPropertyPaths);
				}

				PropertyPath.RemoveAt( PropertyPath.Num() - 1 );
			}
		}
	}
}


void WidgetMaterialTrackUtilities::GetMaterialBrushPropertyPaths( UWidget* Widget, TArray<FWidgetMaterialPropertyPath>& MaterialBrushPropertyPaths )
{
	TArray<FProperty*> PropertyPath;
	GetMaterialBrushPropertyPathsRecursive( Widget, Widget->GetClass(), PropertyPath, MaterialBrushPropertyPaths );
}

FName WidgetMaterialTrackUtilities::GetTrackNameFromPropertyNamePath( TArrayView<const FName> PropertyNamePath )
{
	if ( PropertyNamePath.Num() == 0 )
	{
		return FName();
	}

	FString TrackName = PropertyNamePath[0].ToString();
	for ( int32 i = 1; i < PropertyNamePath.Num(); i++ )
	{
		TrackName.AppendChar( '.' );
		TrackName.Append( PropertyNamePath[i].ToString() );
	}

	return FName( *TrackName );
}

