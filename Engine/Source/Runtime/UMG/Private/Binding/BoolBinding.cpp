// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/BoolBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoolBinding)

#define LOCTEXT_NAMESPACE "UMG"

UBoolBinding::UBoolBinding()
{
}

bool UBoolBinding::IsSupportedSource(FProperty* Property) const
{
	return IsConcreteTypeCompatibleWithReflectedType<bool>(Property);
}

bool UBoolBinding::IsSupportedDestination(FProperty* Property) const
{
	return IsConcreteTypeCompatibleWithReflectedType<bool>(Property);
}

bool UBoolBinding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		bool Value = false;
		if ( SourcePath.GetValue<bool>(Source, Value) )
		{
			return Value;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

