// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/BrushBinding.h"
#include "Engine/Texture2D.h"
#include "Brushes/SlateNoResource.h"
#include "Blueprint/WidgetBlueprintLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrushBinding)

#define LOCTEXT_NAMESPACE "UMG"

UBrushBinding::UBrushBinding()
{
}

bool UBrushBinding::IsSupportedDestination(FProperty* Property) const
{
	return IsConcreteTypeCompatibleWithReflectedType<FSlateBrush>(Property);
}

bool UBrushBinding::IsSupportedSource(FProperty* Property) const
{
	if ( IsConcreteTypeCompatibleWithReflectedType<UObject*>(Property) )
	{
		if ( FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property) )
		{
			return ObjectProperty->PropertyClass->IsChildOf(UTexture2D::StaticClass());
		}
	}

	return IsConcreteTypeCompatibleWithReflectedType<FSlateBrush>(Property);
}

FSlateBrush UBrushBinding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		if ( bConversion.Get(EConversion::None) == EConversion::None )
		{
			FSlateBrush Value;
			if ( SourcePath.GetValue<FSlateBrush>(Source, Value) )
			{
				bConversion = EConversion::None;
				return Value;
			}
		}

		if ( bConversion.Get(EConversion::Texture) == EConversion::Texture )
		{
			UObject* Value;
			if ( SourcePath.GetValue<UObject*>(Source, Value) )
			{
				if ( UTexture2D* Texture = Cast<UTexture2D>(Value) )
				{
					bConversion = EConversion::Texture;
					return GetDefault<UWidgetBlueprintLibrary>()->MakeBrushFromTexture(Texture);
				}
			}
		}
	}

	return FSlateNoResource();
}

#undef LOCTEXT_NAMESPACE

