// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/CheckedStateBinding.h"
#include "Styling/SlateTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CheckedStateBinding)

#define LOCTEXT_NAMESPACE "UMG"

UCheckedStateBinding::UCheckedStateBinding()
{
}

bool UCheckedStateBinding::IsSupportedSource(FProperty* Property) const
{
	return IsSupportedDestination(Property) || IsConcreteTypeCompatibleWithReflectedType<bool>(Property);
}

bool UCheckedStateBinding::IsSupportedDestination(FProperty* Property) const
{
	static const FName CheckBoxStateEnum("ECheckBoxState");

	if ( FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property) )
	{
		return EnumProperty->GetEnum()->GetFName() == CheckBoxStateEnum;
	}
	else if ( FByteProperty* ByteProperty = CastField<FByteProperty>(Property) )
	{
		if ( ByteProperty->IsEnum() )
		{
			return ByteProperty->Enum->GetFName() == CheckBoxStateEnum;
		}
	}

	return false;
}

ECheckBoxState UCheckedStateBinding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		if ( bConversion.Get(EConversion::None) == EConversion::None )
		{
			uint8 Value = 0;
			if ( SourcePath.GetValue<uint8>(Source, Value) )
			{
				bConversion = EConversion::None;
				return static_cast<ECheckBoxState>(Value);
			}
		}

		if ( bConversion.Get(EConversion::Bool) == EConversion::Bool )
		{
			bool Value = false;
			if ( SourcePath.GetValue<bool>(Source, Value) )
			{
				bConversion = EConversion::Bool;
				return Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
	}

	return ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE

