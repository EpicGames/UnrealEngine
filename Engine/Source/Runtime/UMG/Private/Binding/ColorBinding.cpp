// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/ColorBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ColorBinding)

#define LOCTEXT_NAMESPACE "UMG"

UColorBinding::UColorBinding()
{
}

bool UColorBinding::IsSupportedDestination(FProperty* Property) const
{
	return IsSupportedSource(Property);
}

bool UColorBinding::IsSupportedSource(FProperty* Property) const
{
	return
		IsConcreteTypeCompatibleWithReflectedType<FSlateColor>(Property) ||
		IsConcreteTypeCompatibleWithReflectedType<FLinearColor>(Property);
}

void UColorBinding::Bind(FProperty* Property, FScriptDelegate* Delegate)
{
	if ( IsConcreteTypeCompatibleWithReflectedType<FSlateColor>(Property) )
	{
		static const FName BinderFunction("GetSlateValue");
		Delegate->BindUFunction(this, BinderFunction);
	}
	else if(IsConcreteTypeCompatibleWithReflectedType<FLinearColor>(Property))
	{
		static const FName BinderFunction("GetLinearValue");
		Delegate->BindUFunction(this, BinderFunction);
	}
}

FSlateColor UColorBinding::GetSlateValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		if ( !bNeedsConversion.Get(false) )
		{
			FSlateColor SlateColor;
			if ( SourcePath.GetValue<FSlateColor>(Source, SlateColor) )
			{
				bNeedsConversion = false;
				return SlateColor;
			}
		}

		if ( bNeedsConversion.Get(true) )
		{
			FLinearColor LinearValue(ForceInitToZero);
			if ( SourcePath.GetValue<FLinearColor>(Source, LinearValue) )
			{
				bNeedsConversion = true;
				return FSlateColor(LinearValue);
			}
		}
	}

	return FSlateColor();
}

FLinearColor UColorBinding::GetLinearValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		if ( !bNeedsConversion.Get(false) )
		{
			FLinearColor LinearValue(ForceInitToZero);
			if ( SourcePath.GetValue<FLinearColor>(Source, LinearValue) )
			{
				bNeedsConversion = false;
				return LinearValue;
			}
		}

		if ( bNeedsConversion.Get(true) )
		{
			FSlateColor SlateColor;
			if ( SourcePath.GetValue<FSlateColor>(Source, SlateColor) )
			{
				bNeedsConversion = true;
				return SlateColor.GetSpecifiedColor();
			}
		}
	}

	return FLinearColor();
}

#undef LOCTEXT_NAMESPACE

