// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/DynamicPropertyPath.h"
#include "PropertyPathHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicPropertyPath)

FDynamicPropertyPath::FDynamicPropertyPath()
{
}

FDynamicPropertyPath::FDynamicPropertyPath(const FString& Path)
	: FCachedPropertyPath(Path)
	{
}

FDynamicPropertyPath::FDynamicPropertyPath(const TArray<FString>& PropertyChain)
	: FCachedPropertyPath(PropertyChain)
{
}

