// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/FloatBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatBinding)

#define LOCTEXT_NAMESPACE "UMG"

UFloatBinding::UFloatBinding()
{
}

bool UFloatBinding::IsSupportedDestination(FProperty* Property) const
{
	return IsSupportedSource(Property);
}

bool UFloatBinding::IsSupportedSource(FProperty* Property) const
{
	return IsConcreteTypeCompatibleWithReflectedType<float>(Property) || IsConcreteTypeCompatibleWithReflectedType<double>(Property);
}

float UFloatBinding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		// Since we can bind to either a float or double, we need to perform a narrowing conversion where necessary.
		// If this isn't a property, then we're assuming that a function is used to extract the float value.

		float FloatValue = 0.0f;

		if (SourcePath.Resolve(Source))
		{
			double DoubleValue = 0.0;
			if (SourcePath.GetValue<float>(Source, FloatValue))
			{
				return FloatValue;
			}
			else if (SourcePath.GetValue<double>(Source, DoubleValue))
			{
				FloatValue = static_cast<float>(DoubleValue);
				return FloatValue;
			}
		}
	}

	return 0.0f;
}

#undef LOCTEXT_NAMESPACE

