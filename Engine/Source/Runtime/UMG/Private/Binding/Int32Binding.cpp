// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/Int32Binding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Int32Binding)

#define LOCTEXT_NAMESPACE "UMG"

UInt32Binding::UInt32Binding()
{
}

bool UInt32Binding::IsSupportedDestination(FProperty* Property) const
{
	return IsSupportedSource(Property);
}

bool UInt32Binding::IsSupportedSource(FProperty* Property) const
{
	return IsConcreteTypeCompatibleWithReflectedType<int32>(Property);
}

int32 UInt32Binding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		int32 Value = 0;
		if ( SourcePath.GetValue<int32>(Source, Value) )
		{
			return Value;
		}
	}

	return 0;
}

#undef LOCTEXT_NAMESPACE

