// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/MouseCursorBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MouseCursorBinding)

#define LOCTEXT_NAMESPACE "UMG"

UMouseCursorBinding::UMouseCursorBinding()
{
}

bool UMouseCursorBinding::IsSupportedSource(FProperty* Property) const
{
	return IsSupportedDestination(Property);
}

bool UMouseCursorBinding::IsSupportedDestination(FProperty* Property) const
{
	static const FName MouseCursorEnum("EMouseCursor");
	
	if ( FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property) )
	{
		return EnumProperty->GetEnum()->GetFName() == MouseCursorEnum;
	}
	else if ( FByteProperty* ByteProperty = CastField<FByteProperty>(Property) )
	{
		if ( ByteProperty->IsEnum() )
		{
			return ByteProperty->Enum->GetFName() == MouseCursorEnum;
		}
	}

	return false;
}

EMouseCursor::Type UMouseCursorBinding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		uint8 Value = 0;
		if ( SourcePath.GetValue<uint8>(Source, Value) )
		{
			return static_cast<EMouseCursor::Type>( Value );
		}
	}

	return EMouseCursor::Default;
}

#undef LOCTEXT_NAMESPACE

