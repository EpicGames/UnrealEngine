// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/PropertyBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBinding)

#define LOCTEXT_NAMESPACE "UMG"

DEFINE_STAT(STAT_UMGBinding);

UPropertyBinding::UPropertyBinding()
{
}

bool UPropertyBinding::IsSupportedSource(FProperty* Property) const
{
	return false;
}

bool UPropertyBinding::IsSupportedDestination(FProperty* Property) const
{
	return false;
}

void UPropertyBinding::Bind(FProperty* Property, FScriptDelegate* Delegate)
{
	static const FName BinderFunction("GetValue");
	Delegate->BindUFunction(this, BinderFunction);
}

#undef LOCTEXT_NAMESPACE

