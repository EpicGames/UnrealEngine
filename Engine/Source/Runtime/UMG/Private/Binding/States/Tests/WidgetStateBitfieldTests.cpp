// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Binding/States/WidgetStateSettings.h"
#include "Binding/States/WidgetStateBitfield.h"
#include "Binding/States/WidgetStateRegistration.h"
#include "Components/CheckBox.h"

/**
 * Coverage:
 *
 * OperatorBool
 * OperatorBitwiseAnd (Intersection)
 * OperatorBitwiseOr (Union)
 * OperatorBitwiseNot
 * HasFlag
 *
 * Note: Checkbox include is only needed if you want to use 'UWidgetCheckedStateRegistration' pre-defined
 * bitfields. It is possible to just re-create these yourself if you don't want to include checkbox.
 * Or more generally, a specific module that you may / may not know exists.
 */

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogWidgetStateBitfieldTest, Log, All);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOperatorBoolTest, "Slate.WidgetState.OperatorBoolTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FOperatorBoolTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FWidgetStateBitfield Test = {};
		GPassing &= TestEqual(TEXT("Unset is false"), Test, false);
	}

	{
		FWidgetStateBitfield Test = {};

		Test.SetBinaryStateSlow(FName("Pressed"), true);
		GPassing &= TestEqual(TEXT("Any binary is true"), Test, true);

		GPassing &= TestEqual(TEXT("Has binary is true"), Test.HasBinaryStates(), true);

		Test.SetBinaryStateSlow(FName("Pressed"), false);
		GPassing &= TestEqual(TEXT("No binary is false"), Test, false);

		GPassing &= TestEqual(TEXT("Has binary is false"), Test.HasBinaryStates(), false);
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOperatorBitwiseAndTest, "Slate.WidgetState.OperatorBitwiseAndTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FOperatorBitwiseAndTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FWidgetStateBitfield Pressed = {};
		FWidgetStateBitfield Hovered = {};
		FWidgetStateBitfield PressedHovered = {};

		Pressed.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);
		Hovered.SetBinaryState(UWidgetHoveredStateRegistration::Bit, true);
		PressedHovered.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);
		PressedHovered.SetBinaryState(UWidgetHoveredStateRegistration::Bit, true);

		GPassing &= TestEqual(TEXT("Pressed & Hovered is false"), Pressed.Intersect(Hovered), false);
		GPassing &= TestEqual(TEXT("Hovered & Pressed is false"), Hovered.Intersect(Pressed), false);

		GPassing &= TestEqual(TEXT("PressedHovered & Pressed is true"), PressedHovered.Intersect(Pressed), true);
		GPassing &= TestEqual(TEXT("Pressed & PressedHovered is true"), Pressed.Intersect(PressedHovered), true);

		GPassing &= TestEqual(TEXT("PressedHovered & Hovered is true"), PressedHovered.Intersect(Hovered), true);
		GPassing &= TestEqual(TEXT("Hovered & PressedHovered is true"), Hovered.Intersect(PressedHovered), true);

		GPassing &= TestEqual(TEXT("PressedHovered & PressedHovered is true"), PressedHovered.Intersect(PressedHovered), true);
		GPassing &= TestEqual(TEXT("PressedHovered & PressedHovered is true"), PressedHovered.Intersect(PressedHovered), true);

		GPassing &= TestEqual(TEXT("Bit: Pressed & Hovered is false"), UWidgetPressedStateRegistration::Bit.Intersect(UWidgetHoveredStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Bit: Hovered & Pressed is false"), UWidgetHoveredStateRegistration::Bit.Intersect(UWidgetPressedStateRegistration::Bit), false);

		GPassing &= TestEqual(TEXT("Bit: PressedHovered & Pressed is true"), PressedHovered.Intersect(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Bit: Pressed & PressedHovered is true"), UWidgetPressedStateRegistration::Bit.Intersect(PressedHovered), true);

		GPassing &= TestEqual(TEXT("Bit: PressedHovered & Hovered is true"), PressedHovered.Intersect(UWidgetHoveredStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Bit: Hovered & PressedHovered is true"), UWidgetHoveredStateRegistration::Bit.Intersect(PressedHovered), true);
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOperatorBitwiseOrTest, "Slate.WidgetState.OperatorBitwiseOrTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FOperatorBitwiseOrTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FWidgetStateBitfield Pressed = {};
		FWidgetStateBitfield Hovered = {};
		FWidgetStateBitfield PressedHovered = {};

		Pressed.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);
		Hovered.SetBinaryState(UWidgetHoveredStateRegistration::Bit, true);
		PressedHovered.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);
		PressedHovered.SetBinaryState(UWidgetHoveredStateRegistration::Bit, true);

		GPassing &= TestEqual(TEXT("Pressed | Hovered is PressedHovered"), Pressed.Union(Hovered), PressedHovered);
		GPassing &= TestEqual(TEXT("Hovered | Pressed is PressedHovered"), Hovered.Union(Pressed), PressedHovered);

		GPassing &= TestEqual(TEXT("PressedHovered | Pressed is PressedHovered"), PressedHovered.Union(Pressed), PressedHovered);
		GPassing &= TestEqual(TEXT("Pressed | PressedHovered is PressedHovered"), Pressed.Union(PressedHovered), PressedHovered);

		GPassing &= TestEqual(TEXT("PressedHovered | Hovered is PressedHovered"), PressedHovered.Union(Hovered), PressedHovered);
		GPassing &= TestEqual(TEXT("Hovered | PressedHovered is PressedHovered"), Hovered.Union(PressedHovered), PressedHovered);

		GPassing &= TestEqual(TEXT("Bit: Pressed | Hovered is PressedHovered"), UWidgetPressedStateRegistration::Bit.Union(UWidgetHoveredStateRegistration::Bit), PressedHovered);
		GPassing &= TestEqual(TEXT("Bit: Hovered | Pressed is PressedHovered"), UWidgetHoveredStateRegistration::Bit.Union(UWidgetPressedStateRegistration::Bit), PressedHovered);
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOperatorBitwiseNotTest, "Slate.WidgetState.OperatorBitwiseNotTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FOperatorBitwiseNotTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;
	{
		FWidgetStateBitfield Test = {};
		GPassing &= TestEqual(TEXT("~{} is true"), ~Test, true);
	}

	{
		FWidgetStateBitfield Pressed = {};

		Pressed.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);

		GPassing &= TestEqual(TEXT("~Pressed is true"), ~Pressed, true);

		GPassing &= TestEqual(TEXT("Pressed & ~Pressed is false"), Pressed.Intersect(~Pressed), false);
		GPassing &= TestEqual(TEXT("Pressed does not have any flags ~Pressed"), Pressed.HasAnyFlags(~Pressed), false);
		GPassing &= TestEqual(TEXT("~Pressed does not have any flags Pressed"), (~Pressed).HasAnyFlags(Pressed), false);
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FHasFlagTest, "Slate.WidgetState.HasFlagTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FHasFlagTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FWidgetStateBitfield Test = {};

		Test.SetBinaryStateSlow(FName("Pressed"), true);
		GPassing &= TestEqual(TEXT("Pressed any flag true"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all flag true"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed any binary flag true"), Test.HasAnyBinaryFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all binary flag true"), Test.HasAllBinaryFlags(UWidgetPressedStateRegistration::Bit), true);

		Test.SetBinaryStateSlow(FName("Pressed"), false);
		GPassing &= TestEqual(TEXT("Pressed any flag false"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed all flag false"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed any binary flag false"), Test.HasAnyBinaryFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed all binary flag false"), Test.HasAllBinaryFlags(UWidgetPressedStateRegistration::Bit), false);

		Test.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);
		GPassing &= TestEqual(TEXT("Pressed any flag true"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all flag true"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed any binary flag true"), Test.HasAnyBinaryFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all binary flag true"), Test.HasAllBinaryFlags(UWidgetPressedStateRegistration::Bit), true);

		Test.SetBinaryState(UWidgetPressedStateRegistration::Bit, false);
		GPassing &= TestEqual(TEXT("Pressed any flag false"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed all flag false"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed any binary flag false"), Test.HasAnyBinaryFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed all binary flag false"), Test.HasAllBinaryFlags(UWidgetPressedStateRegistration::Bit), false);

		// Slow cache results
		uint8 PressedIndex = UWidgetStateSettings::Get()->GetBinaryStateIndex(FName("Pressed"));

		Test.SetBinaryState(PressedIndex, true);
		GPassing &= TestEqual(TEXT("Pressed any flag true"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all flag true"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed any binary flag true"), Test.HasAnyBinaryFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all binary flag true"), Test.HasAllBinaryFlags(UWidgetPressedStateRegistration::Bit), true);

		Test.SetBinaryState(PressedIndex, false);
		GPassing &= TestEqual(TEXT("Pressed any flag false"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed all flag false"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed any binary flag false"), Test.HasAnyBinaryFlags(UWidgetPressedStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Pressed all binary flag false"), Test.HasAllBinaryFlags(UWidgetPressedStateRegistration::Bit), false);
	}

	{
		FWidgetStateBitfield Test = {};
		FWidgetStateBitfield PressedHovered = {};

		PressedHovered.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);
		PressedHovered.SetBinaryState(UWidgetHoveredStateRegistration::Bit, true);

		Test.SetBinaryState(UWidgetPressedStateRegistration::Bit, true);
		Test.SetBinaryState(UWidgetHoveredStateRegistration::Bit, true);
		GPassing &= TestEqual(TEXT("Pressed any flag true"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all flag true"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Hovered any flag true"), Test.HasAnyFlags(UWidgetHoveredStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Hovered all flag true"), Test.HasAllFlags(UWidgetHoveredStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("PressedHovered any flag true"), Test.HasAnyFlags(PressedHovered), true);
		GPassing &= TestEqual(TEXT("PressedHovered all flag true"), Test.HasAllFlags(PressedHovered), true);

		Test.SetBinaryState(UWidgetHoveredStateRegistration::Bit, false);
		GPassing &= TestEqual(TEXT("Pressed any flag true"), Test.HasAnyFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Pressed all flag true"), Test.HasAllFlags(UWidgetPressedStateRegistration::Bit), true);
		GPassing &= TestEqual(TEXT("Hovered any flag true"), Test.HasAnyFlags(UWidgetHoveredStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("Hovered all flag true"), Test.HasAllFlags(UWidgetHoveredStateRegistration::Bit), false);
		GPassing &= TestEqual(TEXT("PressedHovered any flag true"), Test.HasAnyFlags(PressedHovered), true);
		GPassing &= TestEqual(TEXT("PressedHovered all flag true"), Test.HasAllFlags(PressedHovered), false);
	}

	return GPassing;
}

#endif // WITH_DEV_AUTOMATION_TESTS