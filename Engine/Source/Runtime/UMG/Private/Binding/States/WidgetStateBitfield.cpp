// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Binding/States/WidgetStateBitfield.h"

#include "Binding/States/WidgetStateSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetStateBitfield)

FWidgetStateBitfield::FWidgetStateBitfield()
{
}

FWidgetStateBitfield::FWidgetStateBitfield(const FName InStateName)
{
	SetBinaryStateSlow(InStateName, true);
}

FWidgetStateBitfield::FWidgetStateBitfield(const FName InStateName, const uint8 InValue)
{
}

FWidgetStateBitfield FWidgetStateBitfield::operator~() const
{
	FWidgetStateBitfield Temp = *this;
	Temp.NegateStates();
	return Temp;
}

FWidgetStateBitfield FWidgetStateBitfield::Intersect(const FWidgetStateBitfield& Rhs) const
{
	FWidgetStateBitfield Temp;
	Temp.BinaryStates = BinaryStates & Rhs.BinaryStates;
	return Temp;
}

FWidgetStateBitfield FWidgetStateBitfield::Union(const FWidgetStateBitfield& Rhs) const
{
	FWidgetStateBitfield Temp;
	Temp.BinaryStates = BinaryStates | Rhs.BinaryStates;
	return Temp;
}

FWidgetStateBitfield::operator bool() const
{
	// We have to check 'IsEnumStateUsed' here since otherwise a completely empty statefield would pass
	return BinaryStates != 0;
}

bool FWidgetStateBitfield::HasBinaryStates() const
{
	return BinaryStates != 0;
}

bool FWidgetStateBitfield::HasEnumStates() const
{
	return false;
}

bool FWidgetStateBitfield::HasEmptyUsedEnumStates() const
{
	return false;
}

bool FWidgetStateBitfield::HasAnyFlags(const FWidgetStateBitfield& InBitfield) const
{
	return HasAnyBinaryFlags(InBitfield);
}

bool FWidgetStateBitfield::HasAllFlags(const FWidgetStateBitfield& InBitfield) const
{
	return HasAllBinaryFlags(InBitfield);
}

bool FWidgetStateBitfield::HasAnyBinaryFlags(const FWidgetStateBitfield& InBitfield) const
{
	return (BinaryStates & InBitfield.BinaryStates) != 0;
}

bool FWidgetStateBitfield::HasAllBinaryFlags(const FWidgetStateBitfield& InBitfield) const
{
	return (BinaryStates & InBitfield.BinaryStates) == InBitfield.BinaryStates;
}

bool FWidgetStateBitfield::HasAnyEnumFlags(const FWidgetStateBitfield& InBitfield) const
{
	return false;
}

bool FWidgetStateBitfield::HasAllEnumFlags(const FWidgetStateBitfield& InBitfield) const
{
	return false;
}

void FWidgetStateBitfield::SetState(const FWidgetStateBitfield& InBitfield)
{
	*this = InBitfield;
}

void FWidgetStateBitfield::NegateStates()
{
	NegateBinaryStates();
}

void FWidgetStateBitfield::SetBinaryState(uint8 BinaryStateIndex, bool BinaryStateValue)
{
	BinaryStates = (BinaryStates & ~(static_cast<uint64>(1) << BinaryStateIndex)) | (static_cast<uint64>(BinaryStateValue) << BinaryStateIndex);
}

void FWidgetStateBitfield::SetBinaryState(const FWidgetStateBitfield& BinaryStateBitfield, bool BinaryStateValue)
{
	if (BinaryStateValue)
	{
		BinaryStates = BinaryStates | BinaryStateBitfield.BinaryStates;
	}
	else
	{
		BinaryStates = BinaryStates & ~(BinaryStateBitfield.BinaryStates);
	}
}

void FWidgetStateBitfield::SetBinaryStateSlow(FName BinaryStateName, bool BinaryStateValue)
{
	uint8 BinaryStateIndex = UWidgetStateSettings::Get()->GetBinaryStateIndex(BinaryStateName);
	SetBinaryState(BinaryStateIndex, BinaryStateValue);
}

void FWidgetStateBitfield::NegateBinaryStates()
{
	BinaryStates = ~BinaryStates;
}

void FWidgetStateBitfield::SetEnumState(uint8 EnumStateIndex, uint8 EnumStateValue)
{
}

void FWidgetStateBitfield::SetEnumState(const FWidgetStateBitfield& EnumStateBitfield)
{
}

void FWidgetStateBitfield::SetEnumStateSlow(FName EnumStateName, uint8 EnumStateValue)
{
}

void FWidgetStateBitfield::ClearEnumState(const FWidgetStateBitfield& EnumStateBitfield)
{
}

void FWidgetStateBitfield::ClearEnumState(uint8 EnumStateIndex)
{
}

void FWidgetStateBitfield::ClearEnumState(FName EnumStateName)
{
}

void FWidgetStateBitfield::NegateEnumStates()
{
}
