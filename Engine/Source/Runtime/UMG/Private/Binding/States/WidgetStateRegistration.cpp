// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/States/WidgetStateRegistration.h"

#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetStateRegistration)

FName UWidgetBinaryStateRegistration::GetStateName() const
{ 
	return FName(); 
};

bool UWidgetBinaryStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const 
{ 
	return false; 
}

void UWidgetBinaryStateRegistration::InitializeStaticBitfields() const
{
}

FWidgetStateBitfield UWidgetHoveredStateRegistration::Bit;

FName UWidgetHoveredStateRegistration::GetStateName() const
{
	return StateName;
};

bool UWidgetHoveredStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const
{
	return false;
}

void UWidgetHoveredStateRegistration::InitializeStaticBitfields() const
{
	Bit = FWidgetStateBitfield(GetStateName());
}

FWidgetStateBitfield UWidgetPressedStateRegistration::Bit;

FName UWidgetPressedStateRegistration::GetStateName() const
{
	return StateName;
};

bool UWidgetPressedStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const
{
	return false;
}

void UWidgetPressedStateRegistration::InitializeStaticBitfields() const
{
	Bit = FWidgetStateBitfield(GetStateName());
}

FWidgetStateBitfield UWidgetDisabledStateRegistration::Bit;

FName UWidgetDisabledStateRegistration::GetStateName() const
{
	return StateName;
};

bool UWidgetDisabledStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const
{
	return !InWidget->GetIsEnabled();
}

void UWidgetDisabledStateRegistration::InitializeStaticBitfields() const
{
	Bit = FWidgetStateBitfield(GetStateName());
}

FWidgetStateBitfield UWidgetSelectedStateRegistration::Bit;

FName UWidgetSelectedStateRegistration::GetStateName() const
{
	return StateName;
};

bool UWidgetSelectedStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const
{
	return false;
}

void UWidgetSelectedStateRegistration::InitializeStaticBitfields() const
{
	Bit = FWidgetStateBitfield(GetStateName());
}

FName UWidgetEnumStateRegistration::GetStateName() const 
{ 
	return FName(); 
}

bool UWidgetEnumStateRegistration::GetRegisteredWidgetUsesState(const UWidget* InWidget) const 
{ 
	return false; 
}

uint8 UWidgetEnumStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const 
{ 
	return 0; 
}

void UWidgetEnumStateRegistration::InitializeStaticBitfields() const
{
}