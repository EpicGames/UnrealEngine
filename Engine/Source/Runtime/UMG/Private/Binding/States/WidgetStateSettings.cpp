// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/States/WidgetStateSettings.h"

#include "Binding/States/WidgetStateBitfield.h"
#include "Binding/States/WidgetStateRegistration.h"

#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetStateSettings)

void UWidgetStateSettings::PostInitProperties()
{
	Super::PostInitProperties();

	{
		TArray<UClass*> BinaryStateClasses;
		GetDerivedClasses(UWidgetBinaryStateRegistration::StaticClass(), BinaryStateClasses, false);

		BinaryStateRegistrationCDOs.Empty(BinaryStateClasses.Num());
		BinaryStateMap.Empty(BinaryStateClasses.Num());
		BinaryStates.Empty(BinaryStateClasses.Num());

		uint8 BinaryStateIndex = 0;
		for (UClass* BinaryStateClass : BinaryStateClasses)
		{
			// CDO shouldn't ever be nullptr in non-compilation contexts
			UWidgetBinaryStateRegistration* BinaryStateRegistrationCDO = CastChecked<UWidgetBinaryStateRegistration>(BinaryStateClass->GetDefaultObject());
			BinaryStateRegistrationCDOs.Add(BinaryStateRegistrationCDO);

			FName BinaryStateName = BinaryStateRegistrationCDO->GetStateName();
			BinaryStateMap.Add(BinaryStateName, BinaryStateIndex);
			BinaryStates.Add(BinaryStateName);
			BinaryStateRegistrationCDO->InitializeStaticBitfields();

			BinaryStateIndex++;
		}
	}

	{
		TArray<UClass*> EnumStateClasses;
		GetDerivedClasses(UWidgetEnumStateRegistration::StaticClass(), EnumStateClasses, false);

		EnumStateRegistrationCDOs.Empty(EnumStateClasses.Num());
		EnumStateMap.Empty(EnumStateClasses.Num());
		EnumStates.Empty(EnumStateClasses.Num());

		uint8 EnumStateIndex = 0;
		for (UClass* EnumStateClass : EnumStateClasses)
		{
			// CDO shouldn't ever be nullptr in non-compilation contexts
			UWidgetEnumStateRegistration* EnumStateRegistrationCDO = CastChecked<UWidgetEnumStateRegistration>(EnumStateClass->GetDefaultObject());
			EnumStateRegistrationCDOs.Add(EnumStateRegistrationCDO);

			FName EnumStateName = EnumStateRegistrationCDO->GetStateName();
			EnumStateMap.Add(EnumStateName, EnumStateIndex);
			EnumStates.Add(EnumStateName);
			EnumStateRegistrationCDO->InitializeStaticBitfields();

			EnumStateIndex++;
		}
	}
}

void UWidgetStateSettings::GetAllStateNames(TArray<FName>& OutStateNames) const
{
	OutStateNames.Reset(BinaryStates.Num() + EnumStates.Num());
	OutStateNames.Append(BinaryStates);
	OutStateNames.Append(EnumStates);
}

void UWidgetStateSettings::GetBinaryStateNames(TArray<FName>& OutBinaryStateNames) const
{
	OutBinaryStateNames = BinaryStates;
}

void UWidgetStateSettings::GetEnumStateNames(TArray<FName>& OutEnumStateNames) const
{
	OutEnumStateNames = EnumStates;
}

uint8 UWidgetStateSettings::GetBinaryStateIndex(const FName InBinaryStateName) const
{
	return BinaryStateMap[InBinaryStateName];
}

uint8 UWidgetStateSettings::GetEnumStateIndex(const FName InEnumStateName) const
{
	return EnumStateMap[InEnumStateName];
}

FName UWidgetStateSettings::GetBinaryStateName(const uint8 InBinaryStateIndex) const
{
	return BinaryStates[InBinaryStateIndex];
}

FName UWidgetStateSettings::GetEnumStateName(const uint8 InEnumStateIndex) const
{
	return EnumStates[InEnumStateIndex];
}

FWidgetStateBitfield UWidgetStateSettings::GetInitialRegistrationBitfield(const UWidget* InWidget) const
{
	FWidgetStateBitfield InitBitfield = {};

	uint8 BinaryStateIndex = 0;
	for (const TObjectPtr<UWidgetBinaryStateRegistration>& BinaryStateRegistrationCDO : BinaryStateRegistrationCDOs)
	{
		bool InitBinaryState = BinaryStateRegistrationCDO->GetRegisteredWidgetState(InWidget);
		InitBitfield.SetBinaryState(BinaryStateIndex, InitBinaryState);
		BinaryStateIndex++;

		ensureMsgf(BinaryStateIndex <= 64, TEXT("Num Binary states should not exceed 64, as it is stored in a uint64. Remove 'UWidgetBinaryStateRegistration' subclasses from your project to address."));
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	uint8 EnumStateIndex = 0;
	for (const TObjectPtr<UWidgetEnumStateRegistration>& EnumStateRegistrationCDO : EnumStateRegistrationCDOs)
	{
		bool InitUsesEnumState = EnumStateRegistrationCDO->GetRegisteredWidgetUsesState(InWidget);
		if (InitUsesEnumState)
		{
			uint8 InitEnumState = EnumStateRegistrationCDO->GetRegisteredWidgetState(InWidget);
			InitBitfield.SetEnumState(EnumStateIndex, InitEnumState);
		}
		else
		{
			InitBitfield.ClearEnumState(EnumStateIndex);
		}
		EnumStateIndex++;
	}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	return InitBitfield;
}
