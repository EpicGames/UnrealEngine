// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/TextBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextBinding)

#define LOCTEXT_NAMESPACE "UMG"

bool UTextBinding::IsSupportedDestination(FProperty* Property) const
{
	return IsConcreteTypeCompatibleWithReflectedType<FText>(Property);
}

bool UTextBinding::IsSupportedSource(FProperty* Property) const
{
	return
		IsConcreteTypeCompatibleWithReflectedType<FText>(Property) ||
		IsConcreteTypeCompatibleWithReflectedType<FString>(Property) ||
		IsConcreteTypeCompatibleWithReflectedType<int32>(Property) ||
		IsConcreteTypeCompatibleWithReflectedType<float>(Property);
}

void UTextBinding::Bind(FProperty* Property, FScriptDelegate* Delegate)
{
	if ( IsConcreteTypeCompatibleWithReflectedType<FText>(Property) )
	{
		static const FName BinderFunction("GetTextValue");
		Delegate->BindUFunction(this, BinderFunction);
	}
	else if ( IsConcreteTypeCompatibleWithReflectedType<FString>(Property) )
	{
		static const FName BinderFunction("GetStringValue");
		Delegate->BindUFunction(this, BinderFunction);
	}
}

FText UTextBinding::GetTextValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		if (NeedsConversion.Get(EConversion::None) == EConversion::None)
		{
			FText TextValue = FText::GetEmpty();
			if ( SourcePath.GetValue<FText>(Source, TextValue) )
			{
				NeedsConversion = EConversion::None;
				return TextValue;
			}
		}

		if (NeedsConversion.Get(EConversion::String) == EConversion::String)
		{
			FString StringValue;
			if (SourcePath.GetValue<FString>(Source, StringValue))
			{
				NeedsConversion = EConversion::String;
				return FText::FromString(StringValue);
			}
		}
		if (NeedsConversion.Get(EConversion::Integer) == EConversion::Integer)
		{
			int32 IntegerValue;
			if (SourcePath.GetValue<int32>(Source, IntegerValue))
			{
				NeedsConversion = EConversion::Integer;
				return FText::AsNumber(IntegerValue);
			}
		}
		if (NeedsConversion.Get(EConversion::Float) == EConversion::Float)
		{
			float FloatValue;
			if (SourcePath.GetValue<float>(Source, FloatValue))
			{
				NeedsConversion = EConversion::Float;
				return FText::AsNumber(FloatValue);
			}
		}
	}

	return FText::GetEmpty();
}

FString UTextBinding::GetStringValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if(UObject* Source = SourceObject.Get())
	{
		if (NeedsConversion.Get(EConversion::None) == EConversion::None)
		{
			FString StringValue;
			if ( SourcePath.GetValue<FString>(Source, StringValue) )
			{
				NeedsConversion = EConversion::None;
				return StringValue;
			}
		}

		if (NeedsConversion.Get(EConversion::Words) == EConversion::Words)
		{
			FText TextValue = FText::GetEmpty();
			if (SourcePath.GetValue<FText>(Source, TextValue))
			{
				NeedsConversion = EConversion::Words;
				return TextValue.ToString();
			}
		}
		if (NeedsConversion.Get(EConversion::Integer) == EConversion::Integer)
		{
			int32 IntegerValue;
			if (SourcePath.GetValue<int32>(Source, IntegerValue))
			{
				NeedsConversion = EConversion::Integer;
				return FString::FromInt(IntegerValue);
			}
		}
		if (NeedsConversion.Get(EConversion::Float) == EConversion::Float)
		{
			float FloatValue;
			if (SourcePath.GetValue<float>(Source, FloatValue))
			{
				NeedsConversion = EConversion::Float;
				return FString::SanitizeFloat(FloatValue);
			}
		}
	}

	return FString();
}

#undef LOCTEXT_NAMESPACE

