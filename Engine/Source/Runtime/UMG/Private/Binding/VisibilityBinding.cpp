// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/VisibilityBinding.h"
#include "Components/SlateWrapperTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisibilityBinding)

#define LOCTEXT_NAMESPACE "UMG"

UVisibilityBinding::UVisibilityBinding()
{
}

bool UVisibilityBinding::IsSupportedSource(FProperty* Property) const
{
	return IsSupportedDestination(Property);
}

bool UVisibilityBinding::IsSupportedDestination(FProperty* Property) const
{
	static const FName VisibilityEnum("ESlateVisibility");

	if ( FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property) )
	{
		return EnumProperty->GetEnum()->GetFName() == VisibilityEnum;
	}
	else if ( FByteProperty* ByteProperty = CastField<FByteProperty>(Property) )
	{
		if ( ByteProperty->IsEnum() )
		{
			return ByteProperty->Enum->GetFName() == VisibilityEnum;
		}
	}

	return false;
}

ESlateVisibility UVisibilityBinding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		uint8 Value = 0;
		if ( SourcePath.GetValue<uint8>(Source, Value) )
		{
			return static_cast<ESlateVisibility>( Value );
		}
	}

	return ESlateVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE

