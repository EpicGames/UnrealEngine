// Copyright Epic Games, Inc. All Rights Reserved.

#include "Binding/WidgetBinding.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBinding)

#define LOCTEXT_NAMESPACE "UMG"

UWidgetBinding::UWidgetBinding()
{
}

bool UWidgetBinding::IsSupportedDestination(FProperty* Property) const
{
	return IsSupportedSource(Property);
}

bool UWidgetBinding::IsSupportedSource(FProperty* Property) const
{
	if ( IsConcreteTypeCompatibleWithReflectedType<UObject*>(Property) )
	{
		if ( FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property) )
		{
			return ObjectProperty->PropertyClass->IsChildOf(UWidget::StaticClass());
		}
	}

	return false;
}

UWidget* UWidgetBinding::GetValue() const
{
	//SCOPE_CYCLE_COUNTER(STAT_UMGBinding);

	if ( UObject* Source = SourceObject.Get() )
	{
		UObject* Value = nullptr;
		if ( SourcePath.GetValue<UObject*>(Source, Value) )
		{
			if ( UWidget* Widget = Cast<UWidget>(Value) )
			{
				return Widget;
			}
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

