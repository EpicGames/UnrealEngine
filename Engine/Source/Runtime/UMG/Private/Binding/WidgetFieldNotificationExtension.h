// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/Visual.h"
#include "Extensions/UserWidgetExtension.h"
#include "FieldNotificationDelegate.h"

#include "WidgetFieldNotificationExtension.generated.h"

class UVisual;

UCLASS(MinimalAPI, Transient, NotBlueprintType)
class UWidgetFieldNotificationExtension : public UUserWidgetExtension
{
	GENERATED_BODY()

public:
	using FDelegate = UE::FieldNotification::FFieldMulticastDelegate::FDelegate;
	using FFieldId = UE::FieldNotification::FFieldId;
	using FRemoveFromResult = UE::FieldNotification::FFieldMulticastDelegate::FRemoveFromResult;
	using FRemoveAllResult = UE::FieldNotification::FFieldMulticastDelegate::FRemoveAllResult;

public:
	FDelegateHandle AddFieldValueChangedDelegate(const UVisual* InObject, FFieldId InFieldId, FDelegate InNewDelegate)
	{
		return Delegates.Add(InObject, InFieldId, MoveTemp(InNewDelegate));
	}

	FDelegateHandle AddFieldValueChangedDelegate(const UVisual* InObject, FFieldId InFieldId, const FFieldValueChangedDynamicDelegate& InDelegate)
	{
		return Delegates.Add(InObject, InFieldId, InDelegate);
	}

	FRemoveFromResult RemoveFieldValueChangedDelegate(const UVisual* InObject, FFieldId InFieldId, FDelegateHandle InDelegateHandle)
	{
		return Delegates.RemoveFrom(InObject, InFieldId, InDelegateHandle);
	}

	FRemoveFromResult RemoveFieldValueChangedDelegate(const UVisual* InObject, FFieldId InFieldId, const FFieldValueChangedDynamicDelegate& InDelegate)
	{
		return Delegates.RemoveFrom(InObject, InFieldId, InDelegate);
	}

	FRemoveAllResult RemoveAllFieldValueChangedDelegates(const UVisual* InObject, FDelegateUserObjectConst InUserObject)
	{
		return Delegates.RemoveAll(InObject, InUserObject);
	}

	FRemoveAllResult RemoveAllFieldValueChangedDelegates(const UVisual* InObject, FFieldId InFieldId, FDelegateUserObjectConst InUserObject)
	{
		return Delegates.RemoveAll(InObject, InFieldId, InUserObject);
	}

	void BroadcastFieldValueChanged(UVisual* InObject, UE::FieldNotification::FFieldId InFieldId)
	{
		Delegates.Broadcast(InObject, InFieldId);
	}

private:
	UE::FieldNotification::FFieldMulticastDelegate Delegates;
};
