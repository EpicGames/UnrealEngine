// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/AsyncTaskDownloadImage.h"
#include "Modules/ModuleManager.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DDynamic.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Interfaces/IHttpResponse.h"
#include "HttpModule.h"
#include "TextureResource.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AsyncTaskDownloadImage)


//----------------------------------------------------------------------//
// UAsyncTaskDownloadImage
//----------------------------------------------------------------------//

UAsyncTaskDownloadImage::UAsyncTaskDownloadImage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if ( HasAnyFlags(RF_ClassDefaultObject) == false )
	{
		AddToRoot();
	}
}

UAsyncTaskDownloadImage* UAsyncTaskDownloadImage::DownloadImage(FString URL)
{
	UAsyncTaskDownloadImage* DownloadTask = NewObject<UAsyncTaskDownloadImage>();
	DownloadTask->Start(URL);

	return DownloadTask;
}

void UAsyncTaskDownloadImage::Start(FString URL)
{
#if !UE_SERVER
	// Create the Http request and add to pending request list
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->OnProcessRequestComplete().BindUObject(this, &UAsyncTaskDownloadImage::HandleImageRequest);
	HttpRequest->SetURL(URL);
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();
#else
	// On the server we don't execute fail or success we just don't fire the request.
	RemoveFromRoot();
#endif
}

void UAsyncTaskDownloadImage::HandleImageRequest(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
#if !UE_SERVER

	RemoveFromRoot();

	if ( bSucceeded && HttpResponse.IsValid() && EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()) &&
		 HttpResponse->GetContentLength() > 0 && HttpResponse->GetContent().Num() > 0 )
	{
		IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		TSharedPtr<IImageWrapper> ImageWrappers[3] =
		{
			ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG),
			ImageWrapperModule.CreateImageWrapper(EImageFormat::JPEG),
			ImageWrapperModule.CreateImageWrapper(EImageFormat::BMP),
		};

		for ( auto ImageWrapper : ImageWrappers )
		{
			if ( ImageWrapper.IsValid() && 
				 ImageWrapper->SetCompressed(HttpResponse->GetContent().GetData(), HttpResponse->GetContent().Num()) &&
				 ImageWrapper->GetWidth() <= TNumericLimits<int32>::Max() && 
				 ImageWrapper->GetHeight() <= TNumericLimits<int32>::Max())
			{
				
				TArray64<uint8> RawData;
				const ERGBFormat InFormat = ERGBFormat::BGRA;
				if ( ImageWrapper->GetRaw(InFormat, 8, RawData) )
				{					
					if ( UTexture2DDynamic* Texture = UTexture2DDynamic::Create(static_cast<int32>(ImageWrapper->GetWidth()), static_cast<int32>(ImageWrapper->GetHeight())) )
					{
						Texture->SRGB = true;
						Texture->UpdateResource();

						FTexture2DDynamicResource* TextureResource = static_cast<FTexture2DDynamicResource*>(Texture->GetResource());
						if (TextureResource)
						{
							ENQUEUE_RENDER_COMMAND(FWriteRawDataToTexture)(
								[TextureResource, RawData = MoveTemp(RawData)](FRHICommandListImmediate& RHICmdList)
								{
									TextureResource->WriteRawToTexture_RenderThread(RawData);
								});
						}
						OnSuccess.Broadcast(Texture);
						return;
					}
				}
			}
		}
	}

	OnFail.Broadcast(nullptr);

#endif
}

