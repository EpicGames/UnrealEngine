// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/GameViewportSubsystem.h"

#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetLayoutLibrary.h"
#include "Components/Widget.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Templates/Tuple.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameViewportSubsystem)

/*
 *
 */
namespace UE::UMG::Private
{
	TPair<FMargin, bool> CalculateOffsetArgument(const FGameViewportWidgetSlot& Slot)
	{
		// If the size is zero, and we're not stretched, then use the desired size.
		FVector2D FinalSize = FVector2D(Slot.Offsets.Right, Slot.Offsets.Bottom);
		bool bUseAutoSize = FinalSize.IsZero() && !Slot.Anchors.IsStretchedVertical() && !Slot.Anchors.IsStretchedHorizontal();
		return TPair<FMargin, bool>(Slot.Offsets, bUseAutoSize);
	}
}
 
 /*
 *
 */
void UGameViewportSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	
	//FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &ThisClass::HandleLevelRemovedFromWorld);
	FWorldDelegates::OnWorldCleanup.AddUObject(this, &ThisClass::HandleWorldCleanup);
	FWorldDelegates::OnWorldBeginTearDown.AddUObject(this, &ThisClass::HandleRemoveWorld);
	FWorldDelegates::OnPreWorldFinishDestroy.AddUObject(this, &ThisClass::HandleRemoveWorld);
}

void UGameViewportSubsystem::Deinitialize()
{
	//FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FWorldDelegates::OnPreWorldFinishDestroy.RemoveAll(this);
	FWorldDelegates::OnWorldBeginTearDown.RemoveAll(this);
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);

	if (ViewportWidgets.Num() > 0)
	{
		using FElement = TTuple<UWidget*, TWeakPtr<SConstraintCanvas>, TWeakObjectPtr<ULocalPlayer>>;
		FMemMark Mark(FMemStack::Get());
		TArray<FElement, TMemStackAllocator<>> TmpViewportWidgets;
		TmpViewportWidgets.Reserve(ViewportWidgets.Num());
		for (auto& Itt : ViewportWidgets)
		{
			if (UWidget* Widget = Itt.Key.ResolveObjectPtr())
			{
				TmpViewportWidgets.Emplace(Widget, Itt.Value.FullScreenWidget, Itt.Value.LocalPlayer);
			}
		}
		ViewportWidgets.Empty();

		for (FElement& Element : TmpViewportWidgets)
		{
			if (Element.Get<0>()->bIsManagedByGameViewportSubsystem)
			{
				RemoveWidgetInternal(Element.Get<0>(), Element.Get<1>(), Element.Get<2>());
			}
		}
	}

	Super::Deinitialize();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UGameViewportSubsystem* UGameViewportSubsystem::Get(UWorld* World)
{
	return Get();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UGameViewportSubsystem* UGameViewportSubsystem::Get()
{
	return GEngine->GetEngineSubsystem<UGameViewportSubsystem>();
}

bool UGameViewportSubsystem::IsWidgetAdded(const UWidget* Widget) const
{
	if (!Widget)
	{
		FFrame::KismetExecutionMessage(TEXT("The Widget is invalid."), ELogVerbosity::Warning);
		return false;
	}

	if (Widget->bIsManagedByGameViewportSubsystem)
	{
		TObjectKey<UWidget> WidgetKey = Widget;
		if (const FSlotInfo* FoundSlot = ViewportWidgets.Find(WidgetKey))
		{
			return FoundSlot->FullScreenWidget.IsValid();
		}
	}

	return false;
}

bool UGameViewportSubsystem::AddWidget(UWidget* Widget, FGameViewportWidgetSlot Slot)
{
	return AddToScreen(Widget, nullptr, Slot);
}

bool UGameViewportSubsystem::AddWidgetForPlayer(UWidget* Widget, ULocalPlayer* Player, FGameViewportWidgetSlot Slot)
{
	if (!Player)
	{
		FFrame::KismetExecutionMessage(TEXT("The Player is invalid."), ELogVerbosity::Warning);
		return false;
	}
	return AddToScreen(Widget, Player, Slot);
}

bool UGameViewportSubsystem::AddToScreen(UWidget* Widget, ULocalPlayer* Player, FGameViewportWidgetSlot& Slot)
{
	if (!Widget)
	{
		FFrame::KismetExecutionMessage(TEXT("The Widget is invalid."), ELogVerbosity::Warning);
		return false;
	}

	if (UPanelWidget* ParentPanel = Widget->GetParent())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("The widget '%s' already has a parent widget. It can't also be added to the viewport!"), *Widget->GetName()), ELogVerbosity::Warning);
		return false;
	}

	// Add the widget to the current worlds viewport.
	UWorld* World = Widget->GetWorld();
	if (!World || !World->IsGameWorld())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("The widget '%s' does not have a World."), *Widget->GetName()), ELogVerbosity::Warning);
		return false;
	}

	UGameViewportClient* ViewportClient = World->GetGameViewport();
	if (!ViewportClient)
	{
		FFrame::KismetExecutionMessage(TEXT("No game viewport was found."), ELogVerbosity::Warning);
		return false;
	}

	SConstraintCanvas::FSlot* RawSlot = nullptr;
	TSharedPtr<SConstraintCanvas> FullScreenCanvas;
	{
		TObjectKey<UWidget> WidgetKey = Widget;
		FSlotInfo& SlotInfo = ViewportWidgets.FindOrAdd(WidgetKey);
		Widget->bIsManagedByGameViewportSubsystem = true;

		if (SlotInfo.FullScreenWidget.IsValid())
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("The widget '%s' was already added to the screen."), *Widget->GetName()), ELogVerbosity::Warning);
			return false;
		}

		TPair<FMargin, bool> OffsetArgument = UE::UMG::Private::CalculateOffsetArgument(Slot);
		FullScreenCanvas = SNew(SConstraintCanvas)
			+ SConstraintCanvas::Slot()
			.Offset(OffsetArgument.Get<0>())
			.AutoSize(OffsetArgument.Get<1>())
			.Anchors(Slot.Anchors)
			.Alignment(Slot.Alignment)
			.Expose(RawSlot);

		SlotInfo.Slot = Slot;
		SlotInfo.LocalPlayer = Player;
		SlotInfo.FullScreenWidget = FullScreenCanvas;
		SlotInfo.FullScreenWidgetSlot = RawSlot;
	}

	check(RawSlot);
	RawSlot->AttachWidget(Widget->TakeWidget());

	if (Player)
	{
		ViewportClient->AddViewportWidgetForPlayer(Player, FullScreenCanvas.ToSharedRef(), Slot.ZOrder);
	}
	else
	{
		// We add 10 to the zorder when adding to the viewport to avoid 
		// displaying below any built-in controls, like the virtual joysticks on mobile builds.
		ViewportClient->AddViewportWidgetContent(FullScreenCanvas.ToSharedRef(), Slot.ZOrder + 10);
	}
	OnWidgetAdded.Broadcast(Widget, Player);

	return true;
}

void UGameViewportSubsystem::RemoveWidget(UWidget* Widget)
{
	if (Widget && Widget->bIsManagedByGameViewportSubsystem)
	{
		FSlotInfo SlotInfo;
		TObjectKey<UWidget> WidgetKey = Widget;
		ViewportWidgets.RemoveAndCopyValue(WidgetKey, SlotInfo);
		RemoveWidgetInternal(Widget, SlotInfo.FullScreenWidget, SlotInfo.LocalPlayer);

		OnWidgetRemoved.Broadcast(Widget);
	}
}

void UGameViewportSubsystem::RemoveWidgetInternal(UWidget* Widget, const TWeakPtr<SConstraintCanvas>& FullScreenWidget, const TWeakObjectPtr<ULocalPlayer>& LocalPlayer)
{
	Widget->bIsManagedByGameViewportSubsystem = false;
	if (TSharedPtr<SWidget> WidgetHost = FullScreenWidget.Pin())
	{
		// If this is a game world remove the widget from the current world's viewport.
		UWorld* World = Widget->GetWorld();
		if (World && World->IsGameWorld())
		{
			if (UGameViewportClient* ViewportClient = World->GetGameViewport())
			{
				TSharedRef<SWidget> WidgetHostRef = WidgetHost.ToSharedRef();
				ViewportClient->RemoveViewportWidgetContent(WidgetHostRef);

				// We may no longer have access to our owning player if the player controller was destroyed
				// Passing nullptr to RemoveViewportWidgetForPlayer will search all player layers for this widget
				ViewportClient->RemoveViewportWidgetForPlayer(LocalPlayer.Get(), WidgetHostRef);
			}
		}
	}
}

FGameViewportWidgetSlot UGameViewportSubsystem::GetWidgetSlot(const UWidget* Widget) const
{
	if (Widget && Widget->bIsManagedByGameViewportSubsystem)
	{
		TObjectKey<UWidget> WidgetKey = Widget;
		const FSlotInfo* SlotInfo = ViewportWidgets.Find(WidgetKey);
		ensureMsgf(SlotInfo != nullptr, TEXT("The bIsManagedByGameViewportSubsystem should matches the ViewportWidgets state."));
		if (SlotInfo)
		{
			return SlotInfo->Slot;
		}
	}
	return FGameViewportWidgetSlot();
}

void UGameViewportSubsystem::SetWidgetSlot(UWidget* Widget, FGameViewportWidgetSlot Slot)
{
	if (Widget && !Widget->HasAnyFlags(RF_BeginDestroyed))
	{
		TObjectKey<UWidget> WidgetKey = Widget;
		FSlotInfo& SlotInfo = ViewportWidgets.FindOrAdd(WidgetKey);
		Widget->bIsManagedByGameViewportSubsystem = true;
		SlotInfo.Slot = Slot;
		if (TSharedPtr<SConstraintCanvas> WidgetHost = SlotInfo.FullScreenWidget.Pin())
		{
			check(SlotInfo.FullScreenWidgetSlot);
			TPair<FMargin, bool> OffsetArgument = UE::UMG::Private::CalculateOffsetArgument(Slot);
			SlotInfo.FullScreenWidgetSlot->SetOffset(OffsetArgument.Get<0>());
			SlotInfo.FullScreenWidgetSlot->SetAutoSize(OffsetArgument.Get<1>());
			SlotInfo.FullScreenWidgetSlot->SetAnchors(Slot.Anchors);
			SlotInfo.FullScreenWidgetSlot->SetAlignment(Slot.Alignment);
			SlotInfo.FullScreenWidgetSlot->SetZOrder(static_cast<float>(Slot.ZOrder));
			WidgetHost->Invalidate(EInvalidateWidgetReason::Layout);
		}
	}
}

FGameViewportWidgetSlot UGameViewportSubsystem::SetWidgetSlotPosition(FGameViewportWidgetSlot Slot, const UWidget* Widget, FVector2D Position, bool bRemoveDPIScale)
{
	if (bRemoveDPIScale)
	{
		const float Scale = UWidgetLayoutLibrary::GetViewportScale(Widget);
		Position /= Scale;
	}

	const FVector2f NewPosition = UE::Slate::CastToVector2f(Position);
	Slot.Offsets.Left = NewPosition.X;
	Slot.Offsets.Top = NewPosition.Y;
	Slot.Anchors = FAnchors(0.f, 0.f);

	return Slot;
}

FGameViewportWidgetSlot UGameViewportSubsystem::SetWidgetSlotDesiredSize(FGameViewportWidgetSlot Slot, FVector2D Size)
{
	const FVector2f NewSize = UE::Slate::CastToVector2f(Size);
	Slot.Offsets.Right = NewSize.X;
	Slot.Offsets.Bottom = NewSize.Y;
	Slot.Anchors = FAnchors(0.f, 0.f);
	return Slot;
}

void UGameViewportSubsystem::HandleWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResoures)
{
	HandleRemoveWorld(InWorld);
}

void UGameViewportSubsystem::HandleRemoveWorld(UWorld* InWorld)
{
	TArray<UWidget*, TInlineAllocator<16>> WidgetsToRemove;
	for (FViewportWidgetList::TIterator Itt = ViewportWidgets.CreateIterator(); Itt; ++Itt)
	{
		if (UWidget* Widget = Itt.Key().ResolveObjectPtr())
		{
			if (Itt.Value().Slot.bAutoRemoveOnWorldRemoved && InWorld == Widget->GetWorld())
			{
				WidgetsToRemove.Add(Widget);
			}
		}
		else
		{
			Itt.RemoveCurrent();
		}
	}

	for (UWidget* Widget : WidgetsToRemove)
	{
		Widget->RemoveFromParent();
	}
}

void UGameViewportSubsystem::HandleLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	// If the InLevel is null, it's a signal that the entire world is about to disappear, so
	// go ahead and remove this widget from the viewport, it could be holding onto too many
	// dangerous actor references that won't carry over into the next world.
	if (InLevel == nullptr)
	{
		HandleRemoveWorld(InWorld);
	}
}
