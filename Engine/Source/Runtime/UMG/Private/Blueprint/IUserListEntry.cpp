// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/IUserListEntry.h"
#include "Slate/SObjectTableRow.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/DragDropOperation.h"
#include "Components/ListViewBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IUserListEntry)

TMap<TWeakObjectPtr<const UUserWidget>, TWeakPtr<const IObjectTableRow>> IObjectTableRow::ObjectRowsByUserWidget;

UUserListEntry::UUserListEntry(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

void IUserListEntry::ReleaseEntry(UUserWidget& ListEntryWidget)
{
	if (IUserListEntry* NativeImplementation = Cast<IUserListEntry>(&ListEntryWidget))
	{
		NativeImplementation->NativeOnEntryReleased();
	}
	else if (ListEntryWidget.Implements<UUserListEntry>())
	{
		Execute_BP_OnEntryReleased(&ListEntryWidget);
	}
}

void IUserListEntry::UpdateItemSelection(UUserWidget& ListEntryWidget, bool bIsSelected)
{
	if (IUserListEntry* NativeImplementation = Cast<IUserListEntry>(&ListEntryWidget))
	{
		NativeImplementation->NativeOnItemSelectionChanged(bIsSelected);
	}
	else if (ListEntryWidget.Implements<UUserListEntry>())
	{
		Execute_BP_OnItemSelectionChanged(&ListEntryWidget, bIsSelected);
	}
}

void IUserListEntry::UpdateItemExpansion(UUserWidget& ListEntryWidget, bool bIsExpanded)
{
	if (IUserListEntry* NativeImplementation = Cast<IUserListEntry>(&ListEntryWidget))
	{
		NativeImplementation->NativeOnItemExpansionChanged(bIsExpanded);
	}
	else if (ListEntryWidget.Implements<UUserListEntry>())
	{
		Execute_BP_OnItemExpansionChanged(&ListEntryWidget, bIsExpanded);
	}
}

UMG_API void IUserListEntry::UpdateEntryDropIndicator(UUserWidget& ListEntryWidget, TOptional<EItemDropZone> DropZone)
{
	if (ListEntryWidget.Implements<UUserListEntry>() && DropZone.IsSet())
	{
		Execute_BP_OnUpdateEntryDropIndicator(&ListEntryWidget, UDragDropOperation::ConvertSlateDropZoneToUMG(DropZone));
	}
}

UMG_API void IUserListEntry::UpdateEntryDragOverState(UUserWidget& ListEntryWidget, bool bIsDraggingOver)
{
	if (ListEntryWidget.Implements<UUserListEntry>())
	{
		Execute_BP_OnEntryDragOverChanged(&ListEntryWidget, bIsDraggingOver);
	}
}

UMG_API void IUserListEntry::EndEntryDropOperation(UUserWidget& ListEntryWidget, bool bSuccess)
{
	if (ListEntryWidget.Implements<UUserListEntry>())
	{
		Execute_BP_OnEndEntryDropOperation(&ListEntryWidget, bSuccess);
	}
}

UMG_API void IUserListEntry::HandleEntryDragged(UUserWidget& ListEntryWidget, UDragDropOperation* Operation)
{
	if (ListEntryWidget.Implements<UUserListEntry>())
	{
		Execute_BP_OnEntryDragged(&ListEntryWidget, Operation);
	}
}

UMG_API void IUserListEntry::HandleEntryDropped(UUserWidget& ListEntryWidget, UDragDropOperation* Operation)
{
	if (ListEntryWidget.Implements<UUserListEntry>())
	{
		Execute_BP_OnEntryDropped(&ListEntryWidget, Operation);
	}
}

bool IUserListEntry::IsListItemSelected() const
{
	return UUserListEntryLibrary::IsListItemSelected(Cast<UUserWidget>(const_cast<IUserListEntry*>(this)));
}

bool IUserListEntry::IsListItemExpanded() const
{
	return UUserListEntryLibrary::IsListItemExpanded(Cast<UUserWidget>(const_cast<IUserListEntry*>(this)));
}

UListViewBase* IUserListEntry::GetOwningListView() const
{
	return UUserListEntryLibrary::GetOwningListView(Cast<UUserWidget>(const_cast<IUserListEntry*>(this)));
}

void IUserListEntry::NativeOnEntryReleased()
{
	Execute_BP_OnEntryReleased(Cast<UObject>(this));
}

void IUserListEntry::NativeOnItemSelectionChanged(bool bIsSelected)
{
	Execute_BP_OnItemSelectionChanged(Cast<UObject>(this), bIsSelected);
}

void IUserListEntry::NativeOnItemExpansionChanged(bool bIsExpanded)
{
	Execute_BP_OnItemExpansionChanged(Cast<UObject>(this), bIsExpanded);
}

bool UUserListEntryLibrary::IsListItemSelected(TScriptInterface<IUserListEntry> UserListEntry)
{
	TSharedPtr<const IObjectTableRow> SlateRow = IObjectTableRow::ObjectRowFromUserWidget(Cast<UUserWidget>(UserListEntry.GetObject()));
	if (SlateRow.IsValid())
	{
		return SlateRow->IsItemSelected();
	}
	return false;
}

bool UUserListEntryLibrary::IsListItemExpanded(TScriptInterface<IUserListEntry> UserListEntry)
{
	TSharedPtr<const IObjectTableRow> SlateRow = IObjectTableRow::ObjectRowFromUserWidget(Cast<UUserWidget>(UserListEntry.GetObject()));
	if (SlateRow.IsValid())
	{
		return SlateRow->IsItemExpanded();
	}
	return false;
}

UListViewBase* UUserListEntryLibrary::GetOwningListView(TScriptInterface<IUserListEntry> UserListEntry)
{
	TSharedPtr<const IObjectTableRow> SlateRow = IObjectTableRow::ObjectRowFromUserWidget(Cast<UUserWidget>(UserListEntry.GetObject()));
	if (SlateRow.IsValid())
	{
		return SlateRow->GetOwningListView();
	}
	return nullptr;
}

