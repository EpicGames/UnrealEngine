// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/IUserObjectListEntry.h"
#include "Blueprint/UserWidget.h"
#include "Components/ListView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IUserObjectListEntry)

UUserObjectListEntry::UUserObjectListEntry(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

void IUserObjectListEntry::NativeOnListItemObjectSet(UObject* ListItemObject)
{
	Execute_OnListItemObjectSet(Cast<UObject>(this), ListItemObject);
}

UObject* IUserObjectListEntry::GetListItemObjectInternal() const
{
	return UUserObjectListEntryLibrary::GetListItemObject(Cast<UUserWidget>(const_cast<IUserObjectListEntry*>(this)));
}

void IUserObjectListEntry::SetListItemObject(UUserWidget& ListEntryWidget, UObject* ListItemObject)
{
	if (IUserObjectListEntry* NativeImplementation = Cast<IUserObjectListEntry>(&ListEntryWidget))
	{
		NativeImplementation->NativeOnListItemObjectSet(ListItemObject);
	}
	else if (ListEntryWidget.Implements<UUserObjectListEntry>())
	{
		Execute_OnListItemObjectSet(&ListEntryWidget, ListItemObject);
	}
}

UObject* UUserObjectListEntryLibrary::GetListItemObject(TScriptInterface<IUserObjectListEntry> UserObjectListEntry)
{
	if (UUserWidget* EntryWidget = Cast<UUserWidget>(UserObjectListEntry.GetObject()))
	{
		const UListView* OwningListView = Cast<UListView>(UUserListEntryLibrary::GetOwningListView(EntryWidget));
		if (const TObjectPtrWrapTypeOf<UObject*>* ListItem = OwningListView ? OwningListView->ItemFromEntryWidget(*EntryWidget) : nullptr)
		{
			return *ListItem;
		}
	}
	return nullptr;
}

int32 UUserObjectListEntryLibrary::GetListItemIndex(TScriptInterface<IUserObjectListEntry> UserObjectListEntry)
{
	if (UUserWidget* EntryWidget = Cast<UUserWidget>(UserObjectListEntry.GetObject()))
	{
		if (const UListView* OwningListView = Cast<UListView>(UUserListEntryLibrary::GetOwningListView(EntryWidget)))
		{
			if (UObject* Item = UUserObjectListEntryLibrary::GetListItemObject(UserObjectListEntry))
			{
				return OwningListView->GetIndexForItem(Item);
			}
		}
	}
	return INDEX_NONE;
}

bool UUserObjectListEntryLibrary::IsFirstWidget(TScriptInterface<IUserObjectListEntry> UserObjectListEntry)
{
	return UUserObjectListEntryLibrary::GetListItemIndex(UserObjectListEntry) == 0;
}

bool UUserObjectListEntryLibrary::IsLastWidget(TScriptInterface<IUserObjectListEntry> UserObjectListEntry)
{
	if (UUserWidget* EntryWidget = Cast<UUserWidget>(UserObjectListEntry.GetObject()))
	{
		if (const UListView* OwningListView = Cast<UListView>(UUserListEntryLibrary::GetOwningListView(EntryWidget)))
		{
			if (UObject* Item = UUserObjectListEntryLibrary::GetListItemObject(UserObjectListEntry))
			{
				const int32 Index = OwningListView->GetIndexForItem(Item);
				return Index != INDEX_NONE && Index == OwningListView->GetNumItems() - 1;
			}
		}
	}
	return false;
}
