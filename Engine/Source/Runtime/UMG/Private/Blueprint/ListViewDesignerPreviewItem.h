// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ListViewDesignerPreviewItem.generated.h"

// Empty dummy UObject used as the table view item during design time
// Allows rough design-time previewing of how list contents will lay out
UCLASS(Transient, Within = ListView)
class UListViewDesignerPreviewItem : public UObject
{
	GENERATED_BODY()
};