// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/UserWidgetPool.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserWidgetPool)

FUserWidgetPool::FUserWidgetPool(UWidget& InOwningWidget)
	: OwningWidget(&InOwningWidget)
{}

FUserWidgetPool::~FUserWidgetPool()
{
	ResetPool();
}

void FUserWidgetPool::SetWorld(UWorld* InOwningWorld)
{
	OwningWorld = InOwningWorld;
}

void FUserWidgetPool::SetDefaultPlayerController(APlayerController* InDefaultPlayerController)
{
	DefaultPlayerController = InDefaultPlayerController;
}

void FUserWidgetPool::RebuildWidgets()
{
	for (UUserWidget* Widget : ActiveWidgets)
	{
		CachedSlateByWidgetObject.Add(Widget, Widget->TakeWidget());
	}
}

void FUserWidgetPool::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects<UUserWidget>(ActiveWidgets, OwningWidget.Get());
	Collector.AddReferencedObjects<UUserWidget>(InactiveWidgets, OwningWidget.Get());
}

void FUserWidgetPool::Release(UUserWidget* Widget, bool bReleaseSlate)
{
	if (Widget != nullptr)
	{
		const int32 ActiveWidgetIdx = ActiveWidgets.Find(Widget);
		if (ActiveWidgetIdx != INDEX_NONE)
		{
			InactiveWidgets.Push(Widget);
			ActiveWidgets.RemoveAt(ActiveWidgetIdx);

			if (bReleaseSlate)
			{
				CachedSlateByWidgetObject.Remove(Widget);
			}
		}
	}
}

void FUserWidgetPool::Release(TArray<UUserWidget*> Widgets, bool bReleaseSlate)
{
	for (UUserWidget* Widget : Widgets)
	{
		Release(Widget, bReleaseSlate);
	}
}

void FUserWidgetPool::ReleaseAll(bool bReleaseSlate)
{
	InactiveWidgets.Append(ActiveWidgets);
	ActiveWidgets.Empty();

	if (bReleaseSlate)
	{
		CachedSlateByWidgetObject.Reset();
	}
}

void FUserWidgetPool::ResetPool()
{
	InactiveWidgets.Reset();
	ActiveWidgets.Reset();
	CachedSlateByWidgetObject.Reset();
}

void FUserWidgetPool::ReleaseInactiveSlateResources()
{
	for (UUserWidget* InactiveWidget : InactiveWidgets)
	{
		CachedSlateByWidgetObject.Remove(InactiveWidget);
	}
}

void FUserWidgetPool::ReleaseAllSlateResources()
{
	CachedSlateByWidgetObject.Reset();
}

