// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/BackgroundBlur.h"
#include "HAL/IConsoleManager.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElements.h"
#include "Brushes/SlateNoResource.h"
#include "Widgets/Layout/SBackgroundBlur.h"
#include "Components/BackgroundBlurSlot.h"
#include "UObject/EditorObjectVersion.h"
#include "ObjectEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BackgroundBlur)

#define LOCTEXT_NAMESPACE "UMG"

UBackgroundBlur::UBackgroundBlur(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Padding(0.f, 0.f)
	, bApplyAlphaToBlur(true)
	, BlurStrength(0.f)
	, bOverrideAutoRadiusCalculation(false)
	, BlurRadius(0)
	, CornerRadius(0,0,0,0)
	, LowQualityFallbackBrush(FSlateNoResource())
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UBackgroundBlur::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyBackgroundBlur.Reset();
}

UClass* UBackgroundBlur::GetSlotClass() const
{
	return UBackgroundBlurSlot::StaticClass();
}

TSharedRef<SWidget> UBackgroundBlur::RebuildWidget()
{
	MyBackgroundBlur = SNew(SBackgroundBlur);

	if ( GetChildrenCount() > 0 )
	{
		Cast<UBackgroundBlurSlot>(GetContentSlot())->BuildSlot(MyBackgroundBlur.ToSharedRef());
	}
	
	return MyBackgroundBlur.ToSharedRef();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void UBackgroundBlur::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if(MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetPadding(Padding);
		MyBackgroundBlur->SetHAlign(HorizontalAlignment);
		MyBackgroundBlur->SetVAlign(VerticalAlignment);

		MyBackgroundBlur->SetApplyAlphaToBlur(bApplyAlphaToBlur);
		MyBackgroundBlur->SetBlurRadius(bOverrideAutoRadiusCalculation ? BlurRadius : TOptional<int32>());
		MyBackgroundBlur->SetBlurStrength(BlurStrength);
		MyBackgroundBlur->SetLowQualityBackgroundBrush(&LowQualityFallbackBrush);
		MyBackgroundBlur->SetCornerRadius(CornerRadius);
	}
}

void UBackgroundBlur::OnSlotAdded(UPanelSlot* InSlot)
{
	UBackgroundBlurSlot* BackgroundBlurSlot = CastChecked<UBackgroundBlurSlot>(InSlot);
	BackgroundBlurSlot->SetPadding(Padding);
	BackgroundBlurSlot->SetHorizontalAlignment(HorizontalAlignment);
	BackgroundBlurSlot->SetVerticalAlignment(VerticalAlignment);

	// Add the child to the live slot if it already exists
	if (MyBackgroundBlur.IsValid())
	{
		// Construct the underlying slot
		BackgroundBlurSlot->BuildSlot(MyBackgroundBlur.ToSharedRef());
	}
}
	
void UBackgroundBlur::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetContent(SNullWidget::NullWidget);
	}
}

void UBackgroundBlur::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetPadding(InPadding);
	}
}

FMargin UBackgroundBlur::GetPadding() const
{
	return Padding;
}

void UBackgroundBlur::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetHAlign(InHorizontalAlignment);
	}
}

EHorizontalAlignment UBackgroundBlur::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UBackgroundBlur::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetVAlign(InVerticalAlignment);
	}
}

EVerticalAlignment UBackgroundBlur::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UBackgroundBlur::SetApplyAlphaToBlur(bool bInApplyAlphaToBlur)
{
	bApplyAlphaToBlur = bInApplyAlphaToBlur;
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetApplyAlphaToBlur(bInApplyAlphaToBlur);
	}
}

bool UBackgroundBlur::GetApplyAlphaToBlur() const
{
	return bApplyAlphaToBlur;
}

void UBackgroundBlur::SetOverrideAutoRadiusCalculation(bool InOverrideAutoRadiusCalculation)
{
	bOverrideAutoRadiusCalculation = InOverrideAutoRadiusCalculation;
	if (MyBackgroundBlur.IsValid())
	{
		// When set to false, it needs to reset the optional value and it will use the Blur strength to calculate the blur radius
		MyBackgroundBlur->SetBlurRadius(bOverrideAutoRadiusCalculation ? BlurRadius : TOptional<int32>());
	}	
}

bool UBackgroundBlur::GetOverrideAutoRadiusCalculation() const
{
	return bOverrideAutoRadiusCalculation;
}

void UBackgroundBlur::SetBlurRadius(int32 InBlurRadius)
{
	BlurRadius = InBlurRadius;
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetBlurRadius(InBlurRadius);
		bOverrideAutoRadiusCalculation = true;
	}
}

int32 UBackgroundBlur::GetBlurRadius() const
{
	return BlurRadius;
}

void UBackgroundBlur::SetBlurStrength(float InStrength)
{
	BlurStrength = InStrength;
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetBlurStrength(InStrength);
	}
}

float UBackgroundBlur::GetBlurStrength() const
{
	return BlurStrength;
}

void UBackgroundBlur::SetCornerRadius(FVector4 InCornerRadius)
{
	CornerRadius = InCornerRadius;
	if (MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetCornerRadius(InCornerRadius);
	}
}

FVector4 UBackgroundBlur::GetCornerRadius() const
{
	return CornerRadius;
}

void UBackgroundBlur::SetLowQualityFallbackBrush(const FSlateBrush& InBrush)
{
	LowQualityFallbackBrush = InBrush;
	if(MyBackgroundBlur.IsValid())
	{
		MyBackgroundBlur->SetLowQualityBackgroundBrush(&LowQualityFallbackBrush);
	}
}

FSlateBrush UBackgroundBlur::GetLowQualityFallbackBrush() const
{
	return LowQualityFallbackBrush;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UBackgroundBlur::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
}

void UBackgroundBlur::PostLoad()
{
	Super::PostLoad();

	if ( GetLinkerCustomVersion(FEditorObjectVersion::GUID) < FEditorObjectVersion::AddedBackgroundBlurContentSlot )
	{
		//Convert existing slot to new background blur slot slot.
		if ( UPanelSlot* PanelSlot = GetContentSlot() )
		{
			if ( PanelSlot->IsA<UBackgroundBlurSlot>() == false )
			{
				UBackgroundBlurSlot* BlurSlot = NewObject<UBackgroundBlurSlot>(this);
				BlurSlot->Content = PanelSlot->Content;
				BlurSlot->Content->Slot = BlurSlot;
				BlurSlot->Parent = this;
				Slots[0] = BlurSlot;

				// We don't want anyone considering this panel slot for anything, so mark it pending kill.  Otherwise
				// it will confuse the pass we do when doing template validation when it finds it outered to the blur widget.
				PanelSlot->MarkAsGarbage();
			}
		}
	}
}

#if WITH_EDITOR

void UBackgroundBlur::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static bool IsReentrant = false;

	if (!IsReentrant)
	{
		IsReentrant = true;

		if (PropertyChangedEvent.Property)
		{
			static const FName PaddingName("Padding");
			static const FName HorizontalAlignmentName("HorizontalAlignment");
			static const FName VerticalAlignmentName("VerticalAlignment");

			FName PropertyName = PropertyChangedEvent.Property->GetFName();

			if (UBackgroundBlurSlot* BlurSlot = Cast<UBackgroundBlurSlot>(GetContentSlot()))
			{
				if (PropertyName == PaddingName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, PaddingName, BlurSlot, PaddingName);
				}
				else if (PropertyName == HorizontalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, HorizontalAlignmentName, BlurSlot, HorizontalAlignmentName);
				}
				else if (PropertyName == VerticalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, VerticalAlignmentName, BlurSlot, VerticalAlignmentName);
				}
			}
		}

		IsReentrant = false;
	}
}

const FText UBackgroundBlur::GetPaletteCategory()
{
	return LOCTEXT("SpecialFX", "Special Effects");
}

#endif


#undef LOCTEXT_NAMESPACE

