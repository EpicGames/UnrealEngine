// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/BackgroundBlurSlot.h"
#include "ObjectEditorUtils.h"
#include "Widgets/Layout/SBackgroundBlur.h"
#include "Components/BackgroundBlur.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BackgroundBlurSlot)

/////////////////////////////////////////////////////
// UBackgroundBlurSlot

UBackgroundBlurSlot::UBackgroundBlurSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Padding = FMargin(4.f, 2.f);

	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UBackgroundBlurSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	BackgroundBlur.Reset();
}

void UBackgroundBlurSlot::BuildSlot(TSharedRef<SBackgroundBlur> InBackgroundBlur)
{
	BackgroundBlur = InBackgroundBlur;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BackgroundBlur->SetPadding(Padding);
	BackgroundBlur->SetHAlign(HorizontalAlignment);
	BackgroundBlur->SetVAlign(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	BackgroundBlur->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UBackgroundBlurSlot::GetPadding() const
{
	return Padding;
}

void UBackgroundBlurSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	CastChecked<UBackgroundBlur>(Parent)->SetPadding(InPadding);
}

EHorizontalAlignment UBackgroundBlurSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UBackgroundBlurSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	CastChecked<UBackgroundBlur>(Parent)->SetHorizontalAlignment(InHorizontalAlignment);
}

EVerticalAlignment UBackgroundBlurSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UBackgroundBlurSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	CastChecked<UBackgroundBlur>(Parent)->SetVerticalAlignment(InVerticalAlignment);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UBackgroundBlurSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ( BackgroundBlur.IsValid() )
	{
		SetPadding(Padding);
		SetHorizontalAlignment(HorizontalAlignment);
		SetVerticalAlignment(VerticalAlignment);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

void UBackgroundBlurSlot::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static bool IsReentrant = false;

	if ( !IsReentrant )
	{
		IsReentrant = true;

		if ( PropertyChangedEvent.Property )
		{
			static const FName PaddingName("Padding");
			static const FName HorizontalAlignmentName("HorizontalAlignment");
			static const FName VerticalAlignmentName("VerticalAlignment");

			FName PropertyName = PropertyChangedEvent.Property->GetFName();

			if ( UBackgroundBlur* ParentBackgroundBlur = CastChecked<UBackgroundBlur>(Parent) )
			{
				if (PropertyName == PaddingName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, PaddingName, ParentBackgroundBlur, PaddingName);
				}
				else if (PropertyName == HorizontalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, HorizontalAlignmentName, ParentBackgroundBlur, HorizontalAlignmentName);
				}
				else if (PropertyName == VerticalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, VerticalAlignmentName, ParentBackgroundBlur, VerticalAlignmentName);
				}
			}
		}

		IsReentrant = false;
	}
}

#endif

