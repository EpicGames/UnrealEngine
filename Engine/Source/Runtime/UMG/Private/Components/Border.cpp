// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Border.h"
#include "Slate/SlateBrushAsset.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Components/BorderSlot.h"
#include "ObjectEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Border)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UBorder

UBorder::UBorder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ContentColorAndOpacity = FLinearColor::White;
	BrushColor = FLinearColor::White;

	Padding = FMargin(4, 2);

	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;

	DesiredSizeScale = FVector2D(1, 1);

	bShowEffectWhenDisabled = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UBorder::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyBorder.Reset();
}

TSharedRef<SWidget> UBorder::RebuildWidget()
{
	MyBorder = SNew(SBorder)
		.FlipForRightToLeftFlowDirection(bFlipForRightToLeftFlowDirection);
	
	if ( GetChildrenCount() > 0 )
	{
		Cast<UBorderSlot>(GetContentSlot())->BuildSlot(MyBorder.ToSharedRef());
	}

	return MyBorder.ToSharedRef();
}

void UBorder::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyBorder.IsValid())
	{
		return;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TAttribute<FLinearColor> ContentColorAndOpacityBinding = PROPERTY_BINDING(FLinearColor, ContentColorAndOpacity);
	TAttribute<FSlateColor> BrushColorBinding = OPTIONAL_BINDING_CONVERT(FLinearColor, BrushColor, FSlateColor, ConvertLinearColorToSlateColor);
	TAttribute<const FSlateBrush*> ImageBinding = OPTIONAL_BINDING_CONVERT(FSlateBrush, Background, const FSlateBrush*, ConvertImage);
	
	MyBorder->SetPadding(Padding);
	MyBorder->SetBorderBackgroundColor(BrushColorBinding);
	MyBorder->SetColorAndOpacity(ContentColorAndOpacityBinding);

	MyBorder->SetBorderImage(ImageBinding);
	
	MyBorder->SetDesiredSizeScale(DesiredSizeScale);
	MyBorder->SetShowEffectWhenDisabled(bShowEffectWhenDisabled != 0);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MyBorder->SetOnMouseButtonDown(BIND_UOBJECT_DELEGATE(FPointerEventHandler, HandleMouseButtonDown));
	MyBorder->SetOnMouseButtonUp(BIND_UOBJECT_DELEGATE(FPointerEventHandler, HandleMouseButtonUp));
	MyBorder->SetOnMouseMove(BIND_UOBJECT_DELEGATE(FPointerEventHandler, HandleMouseMove));
	MyBorder->SetOnMouseDoubleClick(BIND_UOBJECT_DELEGATE(FPointerEventHandler, HandleMouseDoubleClick));
}

UClass* UBorder::GetSlotClass() const
{
	return UBorderSlot::StaticClass();
}

void UBorder::OnSlotAdded(UPanelSlot* InSlot)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Copy the content properties into the new slot so that it matches what has been setup
	// so far by the user.
	UBorderSlot* BorderSlot = CastChecked<UBorderSlot>(InSlot);
	BorderSlot->Padding = Padding;
	BorderSlot->HorizontalAlignment = HorizontalAlignment;
	BorderSlot->VerticalAlignment = VerticalAlignment;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Add the child to the live slot if it already exists
	if ( MyBorder.IsValid() )
	{
		// Construct the underlying slot.
		BorderSlot->BuildSlot(MyBorder.ToSharedRef());
	}
}

void UBorder::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyBorder.IsValid() )
	{
		MyBorder->SetContent(SNullWidget::NullWidget);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FLinearColor UBorder::GetContentColorAndOpacity() const
{
	if (MyBorder.IsValid())
	{
		return MyBorder->GetColorAndOpacity();
	}
	return ContentColorAndOpacity;
}

void UBorder::SetContentColorAndOpacity(FLinearColor Color)
{
	ContentColorAndOpacity = Color;
	if ( MyBorder.IsValid() )
	{
		MyBorder->SetColorAndOpacity(Color);
	}
}

FMargin UBorder::GetPadding() const
{
	return Padding;
}

void UBorder::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( MyBorder.IsValid() )
	{
		MyBorder->SetPadding(InPadding);
	}
}

EHorizontalAlignment UBorder::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UBorder::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if (MyBorder.IsValid())
	{
		MyBorder->SetHAlign(InHorizontalAlignment);
	}
}

EVerticalAlignment UBorder::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UBorder::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( MyBorder.IsValid() )
	{
		MyBorder->SetVAlign(InVerticalAlignment);
	}
}

FLinearColor UBorder::GetBrushColor() const
{
	return BrushColor;
}

void UBorder::SetBrushColor(FLinearColor Color)
{
	BrushColor = Color;
	if ( MyBorder.IsValid() )
	{
		MyBorder->SetBorderBackgroundColor(Color);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FReply UBorder::HandleMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if ( OnMouseButtonDownEvent.IsBound() )
	{
		return OnMouseButtonDownEvent.Execute(Geometry, MouseEvent).NativeReply;
	}

	return FReply::Unhandled();
}

FReply UBorder::HandleMouseButtonUp(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if ( OnMouseButtonUpEvent.IsBound() )
	{
		return OnMouseButtonUpEvent.Execute(Geometry, MouseEvent).NativeReply;
	}

	return FReply::Unhandled();
}

FReply UBorder::HandleMouseMove(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if ( OnMouseMoveEvent.IsBound() )
	{
		return OnMouseMoveEvent.Execute(Geometry, MouseEvent).NativeReply;
	}

	return FReply::Unhandled();
}

FReply UBorder::HandleMouseDoubleClick(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if ( OnMouseDoubleClickEvent.IsBound() )
	{
		return OnMouseDoubleClickEvent.Execute(Geometry, MouseEvent).NativeReply;
	}

	return FReply::Unhandled();
}

void UBorder::SetBrush(const FSlateBrush& Brush)
{
	Background = Brush;

	if ( MyBorder.IsValid() )
	{
		MyBorder->SetBorderImage(&Background);
	}
}

void UBorder::SetBrushFromAsset(USlateBrushAsset* Asset)
{
	Background = Asset ? Asset->Brush : FSlateBrush();

	if ( MyBorder.IsValid() )
	{
		MyBorder->SetBorderImage(&Background);
	}
}

void UBorder::SetBrushFromTexture(UTexture2D* Texture)
{
	Background.SetResourceObject(Texture);

	if ( MyBorder.IsValid() )
	{
		MyBorder->SetBorderImage(&Background);
	}
}

void UBorder::SetBrushFromMaterial(UMaterialInterface* Material)
{
	if (!Material)
	{
		UE_LOGF(LogSlate, Log, "UBorder::SetBrushFromMaterial. Incoming material is null. %ls", *GetPathName());
	}

	Background.SetResourceObject(Material);

	//TODO UMG Check if the material can be used with the UI

	if ( MyBorder.IsValid() )
	{
		MyBorder->SetBorderImage(&Background);
	}
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UBorder::GetShowEffectWhenDisabled() const
{
	return bShowEffectWhenDisabled;
}

void UBorder::SetShowEffectWhenDisabled(bool bInShowEffectWhenDisabled)
{
	bShowEffectWhenDisabled = bInShowEffectWhenDisabled;
	if(MyBorder)
	{
		MyBorder->SetShowEffectWhenDisabled(bShowEffectWhenDisabled != 0);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UMaterialInstanceDynamic* UBorder::GetDynamicMaterial()
{
	UMaterialInterface* Material = nullptr;

	UObject* Resource = Background.GetResourceObject();
	Material = Cast<UMaterialInterface>(Resource);

	if ( Material )
	{
		UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);

		if ( !DynamicMaterial )
		{
			DynamicMaterial = UMaterialInstanceDynamic::Create(Material, this);
			Background.SetResourceObject(DynamicMaterial);

			if ( MyBorder.IsValid() )
			{
				MyBorder->SetBorderImage(&Background);
			}
		}

		return DynamicMaterial;
	}

	//TODO UMG can we do something for textures?  General purpose dynamic material for them?
	return nullptr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVector2D UBorder::GetDesiredSizeScale() const
{
	return DesiredSizeScale;
}

void UBorder::SetDesiredSizeScale(FVector2D InScale)
{
	DesiredSizeScale = InScale;
	if (MyBorder.IsValid())
	{
		MyBorder->SetDesiredSizeScale(InScale);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const FSlateBrush* UBorder::ConvertImage(TAttribute<FSlateBrush> InImageAsset) const
{
	UBorder* MutableThis = const_cast<UBorder*>( this );
	MutableThis->Background = InImageAsset.Get();

	return &Background;
}

void UBorder::PostLoad()
{
	Super::PostLoad();

	if ( GetChildrenCount() > 0 )
	{
		//TODO UMG Pre-Release Upgrade, now have slots of their own.  Convert existing slot to new slot.
		if ( UPanelSlot* PanelSlot = GetContentSlot() )
		{
			UBorderSlot* BorderSlot = Cast<UBorderSlot>(PanelSlot);
			if ( BorderSlot == NULL )
			{
				BorderSlot = NewObject<UBorderSlot>(this);
				BorderSlot->Content = GetContentSlot()->Content;
				BorderSlot->Content->Slot = BorderSlot;
				Slots[0] = BorderSlot;
			}
		}
	}
}

#if WITH_EDITOR

void UBorder::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static bool IsReentrant = false;

	if ( !IsReentrant )
	{
		IsReentrant = true;

		if ( PropertyChangedEvent.Property )
		{
			static const FName PaddingName("Padding");
			static const FName HorizontalAlignmentName("HorizontalAlignment");
			static const FName VerticalAlignmentName("VerticalAlignment");

			FName PropertyName = PropertyChangedEvent.Property->GetFName();

			if ( UBorderSlot* BorderSlot = Cast<UBorderSlot>(GetContentSlot()) )
			{
				if (PropertyName == PaddingName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, PaddingName, BorderSlot, PaddingName);
				}
				else if (PropertyName == HorizontalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, HorizontalAlignmentName, BorderSlot, HorizontalAlignmentName);
				}
				else if (PropertyName == VerticalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, VerticalAlignmentName, BorderSlot, VerticalAlignmentName);
				}
			}
		}

		IsReentrant = false;
	}
}

const FText UBorder::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

