// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/BorderSlot.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Components/Border.h"
#include "ObjectEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BorderSlot)

/////////////////////////////////////////////////////
// UBorderSlot

UBorderSlot::UBorderSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Padding = FMargin(4.f, 2.f);

	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UBorderSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Border.Reset();
}

void UBorderSlot::BuildSlot(TSharedRef<SBorder> InBorder)
{
	Border = InBorder;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InBorder->SetPadding(Padding);
	InBorder->SetHAlign(HorizontalAlignment);
	InBorder->SetVAlign(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	InBorder->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UBorderSlot::GetPadding() const
{
	return Padding;
}

void UBorderSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	CastChecked<UBorder>(Parent)->SetPadding(InPadding);
}

EHorizontalAlignment UBorderSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UBorderSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	CastChecked<UBorder>(Parent)->SetHorizontalAlignment(InHorizontalAlignment);
}

EVerticalAlignment UBorderSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UBorderSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	CastChecked<UBorder>(Parent)->SetVerticalAlignment(InVerticalAlignment);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UBorderSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ( Border.IsValid() )
	{
		SetPadding(Padding);
		SetHorizontalAlignment(HorizontalAlignment);
		SetVerticalAlignment(VerticalAlignment);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

void UBorderSlot::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static bool IsReentrant = false;

	if ( !IsReentrant )
	{
		IsReentrant = true;

		if ( PropertyChangedEvent.Property )
		{
			static const FName PaddingName("Padding");
			static const FName HorizontalAlignmentName("HorizontalAlignment");
			static const FName VerticalAlignmentName("VerticalAlignment");

			FName PropertyName = PropertyChangedEvent.Property->GetFName();

			if ( UBorder* ParentBorder = CastChecked<UBorder>(Parent) )
			{
				if ( PropertyName == PaddingName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, PaddingName, ParentBorder, PaddingName);
				}
				else if ( PropertyName == HorizontalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, HorizontalAlignmentName, ParentBorder, HorizontalAlignmentName);
				}
				else if ( PropertyName == VerticalAlignmentName)
				{
					FObjectEditorUtils::MigratePropertyValue(this, VerticalAlignmentName, ParentBorder, VerticalAlignmentName);
				}
			}
		}

		IsReentrant = false;
	}
}

#endif

