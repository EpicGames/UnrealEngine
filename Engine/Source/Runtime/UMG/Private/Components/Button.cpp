// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Button.h"

#include "Binding/States/WidgetStateBitfield.h"
#include "Binding/States/WidgetStateRegistration.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Components/ButtonSlot.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"
#include "Blueprint/WidgetTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Button)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UButton

UButton::UButton(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetButtonStyle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR 
	if (IsEditorWidget())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetButtonStyle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ColorAndOpacity = FLinearColor::White;
	BackgroundColor = FLinearColor::White;

	ClickMethod = EButtonClickMethod::DownAndUp;
	TouchMethod = EButtonTouchMethod::DownAndUp;

	// Buttons default to not being draggable
	bAllowDragDrop = false;

	IsFocusable = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

void UButton::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyButton.Reset();
}

TSharedRef<SWidget> UButton::RebuildWidget()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MyButton = SNew(SButton)
		.OnClicked(BIND_UOBJECT_DELEGATE(FOnClicked, SlateHandleClicked))
		.OnPressed(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandlePressed))
		.OnReleased(BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleReleased))
		.OnHovered_UObject(this, &ThisClass::SlateHandleHovered)
		.OnUnhovered_UObject(this, &ThisClass::SlateHandleUnhovered)
		.OnReceivedFocus_UObject(this, &ThisClass::SlateHandleOnReceivedFocus)
		.OnLostFocus_UObject(this, &ThisClass::SlateHandleOnLostFocus)
		.OnSlateButtonDragDetected(BIND_UOBJECT_DELEGATE(FOnDragDetected, SlateHandleDragDetected))
		.OnSlateButtonDragEnter(BIND_UOBJECT_DELEGATE(FOnDragEnter, SlateHandleDragEnter))
		.OnSlateButtonDragLeave(BIND_UOBJECT_DELEGATE(FOnDragLeave, SlateHandleDragLeave))
		.OnSlateButtonDragOver(BIND_UOBJECT_DELEGATE(FOnDragOver, SlateHandleDragOver))
		.OnSlateButtonDrop(BIND_UOBJECT_DELEGATE(FOnDrop, SlateHandleDrop))
		.ButtonStyle(&WidgetStyle)
		.ClickMethod(ClickMethod)
		.TouchMethod(TouchMethod)
		.PressMethod(PressMethod)
		.IsFocusable(IsFocusable)
		.AllowDragDrop(bAllowDragDrop);
		;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if ( GetChildrenCount() > 0 )
	{
		Cast<UButtonSlot>(GetContentSlot())->BuildSlot(MyButton.ToSharedRef());
	}
	
	return MyButton.ToSharedRef();
}

void UButton::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyButton.IsValid())
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyButton->SetButtonStyle(&WidgetStyle);
	MyButton->SetColorAndOpacity( ColorAndOpacity );
	MyButton->SetBorderBackgroundColor( BackgroundColor );
	MyButton->SetClickMethod(ClickMethod);
	MyButton->SetTouchMethod(TouchMethod);
	MyButton->SetPressMethod(PressMethod);
	MyButton->SetAllowDragDrop(bAllowDragDrop);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UClass* UButton::GetSlotClass() const
{
	return UButtonSlot::StaticClass();
}

void UButton::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if ( MyButton.IsValid() )
	{
		CastChecked<UButtonSlot>(InSlot)->BuildSlot(MyButton.ToSharedRef());
	}
}

void UButton::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyButton.IsValid() )
	{
		MyButton->SetContent(SNullWidget::NullWidget);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UButton::SetStyle(const FButtonStyle& InStyle)
{
	WidgetStyle = InStyle;
	if ( MyButton.IsValid() )
	{
		MyButton->SetButtonStyle(&WidgetStyle);
	}
}

const FButtonStyle& UButton::GetStyle() const
{
	return WidgetStyle;
}

void UButton::SetColorAndOpacity(FLinearColor InColorAndOpacity)
{
	ColorAndOpacity = InColorAndOpacity;
	if ( MyButton.IsValid() )
	{
		MyButton->SetColorAndOpacity(InColorAndOpacity);
	}
}

FLinearColor UButton::GetColorAndOpacity() const
{
	return ColorAndOpacity;
}

void UButton::SetBackgroundColor(FLinearColor InBackgroundColor)
{
	BackgroundColor = InBackgroundColor;
	if ( MyButton.IsValid() )
	{
		MyButton->SetBorderBackgroundColor(InBackgroundColor);
	}
}

FLinearColor UButton::GetBackgroundColor() const
{
	return BackgroundColor;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UButton::IsPressed() const
{
	if ( MyButton.IsValid() )
	{
		return MyButton->IsPressed();
	}

	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UButton::SetClickMethod(EButtonClickMethod::Type InClickMethod)
{
	ClickMethod = InClickMethod;
	if ( MyButton.IsValid() )
	{
		MyButton->SetClickMethod(ClickMethod);
	}
}

EButtonClickMethod::Type UButton::GetClickMethod() const
{
	return ClickMethod;
}

void UButton::SetTouchMethod(EButtonTouchMethod::Type InTouchMethod)
{
	TouchMethod = InTouchMethod;
	if ( MyButton.IsValid() )
	{
		MyButton->SetTouchMethod(TouchMethod);
	}
}

EButtonTouchMethod::Type UButton::GetTouchMethod() const
{
	return TouchMethod;
}

void UButton::SetPressMethod(EButtonPressMethod::Type InPressMethod)
{
	PressMethod = InPressMethod;
	if ( MyButton.IsValid() )
	{
		MyButton->SetPressMethod(PressMethod);
	}
}

EButtonPressMethod::Type UButton::GetPressMethod() const
{
	return PressMethod;
}

bool UButton::GetIsFocusable() const
{
	return IsFocusable;
}

UMG_API void UButton::SetAllowDragDrop(bool bInAllowDragDrop)
{
	bAllowDragDrop = bInAllowDragDrop;
	if (MyButton.IsValid() && MyButton->IsParentValid())
	{
		MyButton->SetAllowDragDrop(bAllowDragDrop);
	}
}

void UButton::InitIsFocusable(bool InIsFocusable)
{
	IsFocusable = InIsFocusable;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UButton::PostLoad()
{
	Super::PostLoad();

	if ( GetChildrenCount() > 0 )
	{
		//TODO UMG Pre-Release Upgrade, now buttons have slots of their own.  Convert existing slot to new slot.
		if ( UPanelSlot* PanelSlot = GetContentSlot() )
		{
			UButtonSlot* ButtonSlot = Cast<UButtonSlot>(PanelSlot);
			if ( ButtonSlot == NULL )
			{
				ButtonSlot = NewObject<UButtonSlot>(this);
				ButtonSlot->Content = GetContentSlot()->Content;
				ButtonSlot->Content->Slot = ButtonSlot;
				Slots[0] = ButtonSlot;
			}
		}
	}
}

FReply UButton::SlateHandleClicked()
{
	OnClicked.Broadcast();

	return FReply::Handled();
}

void UButton::SlateHandlePressed()
{
	OnPressed.Broadcast();
	BroadcastBinaryPostStateChange(UWidgetPressedStateRegistration::Bit, true);
}

void UButton::SlateHandleReleased()
{
	OnReleased.Broadcast();
	BroadcastBinaryPostStateChange(UWidgetPressedStateRegistration::Bit, false);
}

void UButton::SlateHandleHovered()
{
	OnHovered.Broadcast();
	BroadcastBinaryPostStateChange(UWidgetHoveredStateRegistration::Bit, true);
}

void UButton::SlateHandleUnhovered()
{
	OnUnhovered.Broadcast();
	BroadcastBinaryPostStateChange(UWidgetHoveredStateRegistration::Bit, false);
}

void UButton::SlateHandleOnReceivedFocus()
{
	OnReceivedFocus.ExecuteIfBound();
}

void UButton::SlateHandleOnLostFocus()
{
	OnLostFocus.ExecuteIfBound();
}

FReply UButton::SlateHandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (OnButtonDragDetected.IsBound())
	{
		return OnButtonDragDetected.Execute(MyGeometry, MouseEvent);
	}
	return FReply::Unhandled();
}

void UButton::SlateHandleDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnButtonDragEnter.IsBound())
	{
		OnButtonDragEnter.Execute(MyGeometry, DragDropEvent);
	}
}

void UButton::SlateHandleDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (OnButtonDragLeave.IsBound())
	{
		OnButtonDragLeave.Execute(DragDropEvent);
	}
}

FReply UButton::SlateHandleDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnButtonDragOver.IsBound())
	{
		return OnButtonDragOver.Execute(MyGeometry, DragDropEvent);
	}
	return FReply::Unhandled();
}

FReply UButton::SlateHandleDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnButtonDrop.IsBound())
	{
		return OnButtonDrop.Execute(MyGeometry, DragDropEvent);
	}
	return FReply::Unhandled();
}

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> UButton::GetAccessibleWidget() const
{
	return MyButton;
}
#endif

#if WITH_EDITOR

const FText UButton::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

