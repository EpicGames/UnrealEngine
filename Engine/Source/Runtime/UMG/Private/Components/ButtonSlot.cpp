// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ButtonSlot.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Input/SButton.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ButtonSlot)

/////////////////////////////////////////////////////
// UButtonSlot

UButtonSlot::UButtonSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Padding = FMargin(4.f, 2.f);

	HorizontalAlignment = HAlign_Center;
	VerticalAlignment = VAlign_Center;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UButtonSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Button.Reset();
}

void UButtonSlot::BuildSlot(TSharedRef<SButton> InButton)
{
	Button = InButton;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InButton->SetContentPadding(Padding);
	InButton->SetHAlign(HorizontalAlignment);
	InButton->SetVAlign(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	InButton->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UButtonSlot::GetPadding() const
{
	return Padding;
}

void UButtonSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Button.IsValid() )
	{
		Button.Pin()->SetContentPadding(InPadding);
	}
}

EHorizontalAlignment UButtonSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UButtonSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Button.IsValid() )
	{
		Button.Pin()->SetHAlign(InHorizontalAlignment);
	}
}

EVerticalAlignment UButtonSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UButtonSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Button.IsValid() )
	{
		Button.Pin()->SetVAlign(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UButtonSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetPadding(Padding);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

