// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CanvasPanel.h"
#include "Layout/ArrangedChildren.h"
#include "Components/CanvasPanelSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CanvasPanel)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UCanvasPanel

UCanvasPanel::UCanvasPanel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UCanvasPanel::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyCanvas.Reset();
}

UClass* UCanvasPanel::GetSlotClass() const
{
	return UCanvasPanelSlot::StaticClass();
}

void UCanvasPanel::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyCanvas.IsValid() )
	{
		CastChecked<UCanvasPanelSlot>(InSlot)->BuildSlot(MyCanvas.ToSharedRef());
	}
}

void UCanvasPanel::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyCanvas.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWrappedWidget();
		if ( Widget.IsValid() )
		{
			MyCanvas->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

TSharedRef<SWidget> UCanvasPanel::RebuildWidget()
{
	MyCanvas = SNew(SConstraintCanvas);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UCanvasPanelSlot* TypedSlot = Cast<UCanvasPanelSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyCanvas.ToSharedRef());
		}
	}

	return MyCanvas.ToSharedRef();
}

UCanvasPanelSlot* UCanvasPanel::AddChildToCanvas(UWidget* Content)
{
	return Cast<UCanvasPanelSlot>( Super::AddChild(Content) );
}

TSharedPtr<SConstraintCanvas> UCanvasPanel::GetCanvasWidget() const
{
	return MyCanvas;
}

bool UCanvasPanel::GetGeometryForSlot(int32 SlotIndex, FGeometry& ArrangedGeometry) const
{
	UCanvasPanelSlot* PanelSlot = CastChecked<UCanvasPanelSlot>(Slots[SlotIndex]);
	return GetGeometryForSlot(PanelSlot, ArrangedGeometry);
}

bool UCanvasPanel::GetGeometryForSlot(UCanvasPanelSlot* InSlot, FGeometry& ArrangedGeometry) const
{
	if ( InSlot->Content == nullptr )
	{
		return false;
	}

	TSharedPtr<SConstraintCanvas> Canvas = GetCanvasWidget();
	if ( Canvas.IsValid() )
	{
		FArrangedChildren ArrangedChildren(EVisibility::All);
		Canvas->ArrangeChildren(Canvas->GetCachedGeometry(), ArrangedChildren);

		for ( int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ChildIndex++ )
		{
			if ( ArrangedChildren[ChildIndex].Widget == InSlot->Content->GetCachedWrappedWidget() )
			{
				ArrangedGeometry = ArrangedChildren[ChildIndex].Geometry;
				return true;
			}
		}
	}

	return false;
}

#if WITH_EDITOR

const FText UCanvasPanel::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

