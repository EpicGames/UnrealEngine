// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CanvasPanelSlot.h"
#include "Components/CanvasPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CanvasPanelSlot)

/////////////////////////////////////////////////////
// UCanvasPanelSlot

UCanvasPanelSlot::UCanvasPanelSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LayoutData.Offsets = FMargin(0.f, 0.f, 100.f, 30.f);
	LayoutData.Anchors = FAnchors(0.0f, 0.0f);
	LayoutData.Alignment = FVector2D(0.0f, 0.0f);
	bAutoSize = false;
	ZOrder = 0;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UCanvasPanelSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UCanvasPanelSlot::BuildSlot(TSharedRef<SConstraintCanvas> Canvas)
{
	Canvas->AddSlot()
		.Expose(Slot)
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];

	SynchronizeProperties();
}

#if WITH_EDITOR

bool UCanvasPanelSlot::NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize)
{
	const FVector2D OldPosition = GetPosition();
	FVector2D NewPosition = OldPosition + NudgeDirection;

	// Determine the new position aligned to the grid.
	if (GridSnapSize.IsSet())
	{
		if (NudgeDirection.X != 0)
		{
			NewPosition.X = ((int32)NewPosition.X) - (((int32)NewPosition.X) % GridSnapSize.GetValue());
		}
		if (NudgeDirection.Y != 0)
		{
			NewPosition.Y = ((int32)NewPosition.Y) - (((int32)NewPosition.Y) % GridSnapSize.GetValue());
		}
	}

	// Offset the size by the same amount moved if we're anchoring along that axis.
	const FVector2D OldSize = GetSize();
	FVector2D NewSize = OldSize;
	if (GetAnchors().IsStretchedHorizontal())
	{
		NewSize.X -= NewPosition.X - OldPosition.X;
	}
	if (GetAnchors().IsStretchedVertical())
	{
		NewSize.Y -= NewPosition.Y - OldPosition.Y;
	}

	// Return false and early out if there are no effective changes.
	if (OldPosition == NewPosition && OldSize == NewSize)
	{
		return false;
	}

	Modify();

	SetPosition(NewPosition);
	SetSize(NewSize);

	return true;
}

bool UCanvasPanelSlot::DragDropPreviewByDesigner(const FVector2D& LocalCursorPosition, const TOptional<int32>& XGridSnapSize, const TOptional<int32>& YGridSnapSize)
{
	// If the widget is not constructed yet, we need to call ReleaseSlateResources
	bool bReleaseSlateResources = !Content->IsConstructed();

	// HACK UMG - This seems like a bad idea to call TakeWidget
	TSharedPtr<SWidget> SlateWidget = Content->TakeWidget();
	SlateWidget->SlatePrepass();
	const FVector2D& WidgetDesiredSize = SlateWidget->GetDesiredSize();

	static const FVector2D MinimumDefaultSize(100, 40);
	FVector2D LocalSize = FVector2D(FMath::Max(WidgetDesiredSize.X, MinimumDefaultSize.X), FMath::Max(WidgetDesiredSize.Y, MinimumDefaultSize.Y));

	FVector2D NewPosition = LocalCursorPosition;
	if (XGridSnapSize.IsSet())
	{
		NewPosition.X = ((int32)NewPosition.X) - (((int32)NewPosition.X) % XGridSnapSize.GetValue());
	}
	if (YGridSnapSize.IsSet())
	{
		NewPosition.Y = ((int32)NewPosition.Y) - (((int32)NewPosition.Y) % YGridSnapSize.GetValue());
	}

	bool LayoutChanged = true;
	// Return false and early out if there are no effective changes.
	if (GetSize() == LocalSize && GetPosition() == NewPosition)
	{
		LayoutChanged = false;
	}
	else
	{
		SetPosition(NewPosition);
		SetSize(LocalSize);
	}

	if (bReleaseSlateResources)
	{
		// When we are done, we free the Widget that was created by TakeWidget.
		Content->ReleaseSlateResources(true);
	}

	return LayoutChanged;
}

void UCanvasPanelSlot::SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot)
{
	const ThisClass* const TemplateCanvasPanelSlot = CastChecked<ThisClass>(TemplateSlot);
	SetPosition(TemplateCanvasPanelSlot->GetPosition());
	SetSize(TemplateCanvasPanelSlot->GetSize());
}

#endif //WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UCanvasPanelSlot::SetLayout(const FAnchorData& InLayoutData)
{
	LayoutData = InLayoutData;

	if ( Slot )
	{
		Slot->SetOffset(LayoutData.Offsets);
		Slot->SetAnchors(LayoutData.Anchors);
		Slot->SetAlignment(LayoutData.Alignment);
	}
}

FAnchorData UCanvasPanelSlot::GetLayout() const
{
	return LayoutData;
}

void UCanvasPanelSlot::SetPosition(FVector2D InPosition)
{
	FVector2f Position = UE::Slate::CastToVector2f(InPosition);
	LayoutData.Offsets.Left = Position.X;
	LayoutData.Offsets.Top = Position.Y;

	if ( Slot )
	{
		Slot->SetOffset(LayoutData.Offsets);
	}
}

FVector2D UCanvasPanelSlot::GetPosition() const
{
	if ( Slot )
	{
		FMargin Offsets = Slot->GetOffset();
		return FVector2D(Offsets.Left, Offsets.Top);
	}

	return FVector2D(LayoutData.Offsets.Left, LayoutData.Offsets.Top);
}

void UCanvasPanelSlot::SetSize(FVector2D InSize)
{
	FVector2f Size = UE::Slate::CastToVector2f(InSize);
	LayoutData.Offsets.Right = Size.X;
	LayoutData.Offsets.Bottom = Size.Y;

	if ( Slot )
	{
		Slot->SetOffset(LayoutData.Offsets);
	}
}

FVector2D UCanvasPanelSlot::GetSize() const
{
	if ( Slot )
	{
		FMargin Offsets = Slot->GetOffset();
		return FVector2D(Offsets.Right, Offsets.Bottom);
	}

	return FVector2D(LayoutData.Offsets.Right, LayoutData.Offsets.Bottom);
}

void UCanvasPanelSlot::SetOffsets(FMargin InOffset)
{
	LayoutData.Offsets = InOffset;
	if ( Slot )
	{
		Slot->SetOffset(InOffset);
	}
}

FMargin UCanvasPanelSlot::GetOffsets() const
{
	if ( Slot )
	{
		return Slot->GetOffset();
	}

	return LayoutData.Offsets;
}

void UCanvasPanelSlot::SetAnchors(FAnchors InAnchors)
{
	LayoutData.Anchors = InAnchors;
	if ( Slot )
	{
		Slot->SetAnchors(InAnchors);
	}
}

FAnchors UCanvasPanelSlot::GetAnchors() const
{
	if ( Slot )
	{
		return Slot->GetAnchors();
	}

	return LayoutData.Anchors;
}

void UCanvasPanelSlot::SetAlignment(FVector2D InAlignment)
{
	LayoutData.Alignment = InAlignment;
	if ( Slot )
	{
		Slot->SetAlignment(InAlignment);
	}
}

FVector2D UCanvasPanelSlot::GetAlignment() const
{
	if ( Slot )
	{
		return Slot->GetAlignment();
	}

	return LayoutData.Alignment;
}

void UCanvasPanelSlot::SetAutoSize(bool InbAutoSize)
{
	bAutoSize = InbAutoSize;
	if ( Slot )
	{
		Slot->SetAutoSize(InbAutoSize);
	}
}

bool UCanvasPanelSlot::GetAutoSize() const
{
	if ( Slot )
	{
		return Slot->GetAutoSize();
	}

	return bAutoSize;
}

void UCanvasPanelSlot::SetZOrder(int32 InZOrder)
{
	ZOrder = InZOrder;
	if ( Slot )
	{
		Slot->SetZOrder(static_cast<float>(InZOrder));
	}
}

int32 UCanvasPanelSlot::GetZOrder() const
{
	if ( Slot )
	{
		return FMath::TruncToInt32(Slot->GetZOrder());
	}

	return ZOrder;
}

void UCanvasPanelSlot::SetMinimum(FVector2D InMinimumAnchors)
{
	LayoutData.Anchors.Minimum = InMinimumAnchors;
	if ( Slot )
	{
		Slot->SetAnchors(LayoutData.Anchors);
	}
}

void UCanvasPanelSlot::SetMaximum(FVector2D InMaximumAnchors)
{
	LayoutData.Anchors.Maximum = InMaximumAnchors;
	if ( Slot )
	{
		Slot->SetAnchors(LayoutData.Anchors);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UCanvasPanelSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetOffsets(LayoutData.Offsets);
	SetAnchors(LayoutData.Anchors);
	SetAlignment(LayoutData.Alignment);
	SetAutoSize(bAutoSize);
	SetZOrder(ZOrder);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

void UCanvasPanelSlot::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	SaveBaseLayout();
}

void UCanvasPanelSlot::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	SynchronizeProperties();

	static FName AnchorsProperty("Anchors");

	if (FEditPropertyChain::TDoubleLinkedListNode* AnchorNode = PropertyChangedEvent.PropertyChain.GetHead()->GetNextNode())
	{
		if (FEditPropertyChain::TDoubleLinkedListNode* LayoutDataNode = AnchorNode->GetNextNode())
		{
			FProperty* AnchorProperty = LayoutDataNode->GetValue();
			if (AnchorProperty && AnchorProperty->GetFName() == AnchorsProperty)
			{
				RebaseLayout();
			}
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedEvent);
}

void UCanvasPanelSlot::SaveBaseLayout()
{
	// Get the current location
	if ( UCanvasPanel* Canvas = Cast<UCanvasPanel>(Parent) )
	{
		FGeometry Geometry;
		if ( Canvas->GetGeometryForSlot(this, Geometry) )
		{
			PreEditGeometry = Geometry;
			PreEditLayoutData = GetLayout();
		}
	}
}

void UCanvasPanelSlot::SetDesiredPosition(FVector2D InPosition)
{
	DesiredPosition = InPosition;
}

void UCanvasPanelSlot::RebaseLayout(bool PreserveSize)
{
	// Ensure we have a parent canvas
	if ( UCanvasPanel* Canvas = Cast<UCanvasPanel>(Parent) )
	{
		FGeometry Geometry;
		if ( Canvas->GetGeometryForSlot(this, Geometry) )
		{
			// Calculate the default anchor offset, ie where would this control be laid out if no offset were provided.
			FVector2D CanvasSize = Canvas->GetCanvasWidget()->GetCachedGeometry().Size;
			FAnchorData LocalLayoutData = GetLayout();
			FMargin AnchorPositions = FMargin(
				LocalLayoutData.Anchors.Minimum.X * CanvasSize.X,
				LocalLayoutData.Anchors.Minimum.Y * CanvasSize.Y,
				LocalLayoutData.Anchors.Maximum.X * CanvasSize.X,
				LocalLayoutData.Anchors.Maximum.Y * CanvasSize.Y);
			FVector2f DefaultAnchorPosition = FVector2f(AnchorPositions.Left, AnchorPositions.Top);

			// Determine the amount that would be offset from the anchor position if alignment was applied.
			FVector2f AlignmentOffset = UE::Slate::CastToVector2f(LocalLayoutData.Alignment) * PreEditGeometry.Size;

			FVector2f MoveDelta = Geometry.Position - PreEditGeometry.Position;

			// Determine where the widget's new position needs to be to maintain a stable location when the anchors change.
			FVector2f LeftTopDelta = PreEditGeometry.Position - DefaultAnchorPosition;

			const bool bAnchorsMoved = PreEditLayoutData.Anchors.Minimum != LocalLayoutData.Anchors.Minimum || PreEditLayoutData.Anchors.Maximum != LocalLayoutData.Anchors.Maximum;
			const bool bMoved = PreEditLayoutData.Offsets.Left != LocalLayoutData.Offsets.Left || PreEditLayoutData.Offsets.Top != LocalLayoutData.Offsets.Top;

			if ( bAnchorsMoved )
			{
				// Adjust the size to remain constant
				if ( !LocalLayoutData.Anchors.IsStretchedHorizontal() && PreEditLayoutData.Anchors.IsStretchedHorizontal() )
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Left = LeftTopDelta.X + AlignmentOffset.X;
					LocalLayoutData.Offsets.Right = PreEditGeometry.Size.X;
				}
				else if ( !PreserveSize && LocalLayoutData.Anchors.IsStretchedHorizontal() && !PreEditLayoutData.Anchors.IsStretchedHorizontal() )
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Left = 0;
					LocalLayoutData.Offsets.Right = 0;
				}
				else if ( LocalLayoutData.Anchors.IsStretchedHorizontal() )
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Left = LeftTopDelta.X;
					LocalLayoutData.Offsets.Right = AnchorPositions.Right - ( AnchorPositions.Left + LocalLayoutData.Offsets.Left + PreEditGeometry.Size.X );
				}
				else
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Left = LeftTopDelta.X + AlignmentOffset.X;
				}

				if ( !LocalLayoutData.Anchors.IsStretchedVertical() && PreEditLayoutData.Anchors.IsStretchedVertical() )
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Top = LeftTopDelta.Y + AlignmentOffset.Y;
					LocalLayoutData.Offsets.Bottom = PreEditGeometry.Size.Y;
				}
				else if ( !PreserveSize && LocalLayoutData.Anchors.IsStretchedVertical() && !PreEditLayoutData.Anchors.IsStretchedVertical() )
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Top = 0;
					LocalLayoutData.Offsets.Bottom = 0;
				}
				else if ( LocalLayoutData.Anchors.IsStretchedVertical() )
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Top = LeftTopDelta.Y;
					LocalLayoutData.Offsets.Bottom = AnchorPositions.Bottom - ( AnchorPositions.Top + LocalLayoutData.Offsets.Top + PreEditGeometry.Size.Y );
				}
				else
				{
					// Adjust the position to remain constant
					LocalLayoutData.Offsets.Top = LeftTopDelta.Y + AlignmentOffset.Y;
				}
			}
			else if ( DesiredPosition.IsSet() )
			{
				FVector2f NewLocalPosition = UE::Slate::CastToVector2f(DesiredPosition.GetValue());

				LocalLayoutData.Offsets.Left = NewLocalPosition.X - AnchorPositions.Left;
				LocalLayoutData.Offsets.Top = NewLocalPosition.Y - AnchorPositions.Top;

				if ( LocalLayoutData.Anchors.IsStretchedHorizontal() )
				{
					LocalLayoutData.Offsets.Right -= LocalLayoutData.Offsets.Left - PreEditLayoutData.Offsets.Left;
				}
				else
				{
					LocalLayoutData.Offsets.Left += AlignmentOffset.X;
				}

				if ( LocalLayoutData.Anchors.IsStretchedVertical() )
				{
					LocalLayoutData.Offsets.Bottom -= LocalLayoutData.Offsets.Top - PreEditLayoutData.Offsets.Top;
				}
				else
				{
					LocalLayoutData.Offsets.Top += AlignmentOffset.Y;
				}

				DesiredPosition.Reset();
			}
			else if ( bMoved )
			{
				LocalLayoutData.Offsets.Left -= DefaultAnchorPosition.X;
				LocalLayoutData.Offsets.Top -= DefaultAnchorPosition.Y;

				// If the slot is stretched horizontally we need to move the right side as it no longer represents width, but
				// now represents margin from the right stretched side.
				if ( LocalLayoutData.Anchors.IsStretchedHorizontal() )
				{
					//LocalLayoutData.Offsets.Right = PreEditLayoutData.Offsets.Top;
				}
				else
				{
					LocalLayoutData.Offsets.Left += AlignmentOffset.X;
				}

				// If the slot is stretched vertically we need to move the bottom side as it no longer represents width, but
				// now represents margin from the bottom stretched side.
				if ( LocalLayoutData.Anchors.IsStretchedVertical() )
				{
					//LocalLayoutData.Offsets.Bottom -= MoveDelta.Y;
				}
				else
				{
					LocalLayoutData.Offsets.Top += AlignmentOffset.Y;
				}
			}
			SetLayout(LocalLayoutData);
		}

		// Apply the changes to the properties.
		SynchronizeProperties();
	}
}

#endif

