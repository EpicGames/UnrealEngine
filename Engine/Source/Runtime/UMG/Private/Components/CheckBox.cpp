// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CheckBox.h"
#include "Binding/States/WidgetStateBitfield.h"
#include "Binding/States/WidgetStateRegistration.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Slate/SlateBrushAsset.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CheckBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UCheckBox

UCheckBox::UCheckBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetCheckboxStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetCheckboxStyle();
	}
#endif // WITH_EDITOR

	CheckedState = ECheckBoxState::Unchecked;

	HorizontalAlignment = HAlign_Fill;

	ClickMethod = EButtonClickMethod::DownAndUp;
	TouchMethod = EButtonTouchMethod::DownAndUp;
	PressMethod = EButtonPressMethod::DownAndUp;

	IsFocusable = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

void UCheckBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyCheckbox.Reset();
}

TSharedRef<SWidget> UCheckBox::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyCheckbox = SNew(SCheckBox)
		.OnCheckStateChanged( BIND_UOBJECT_DELEGATE(FOnCheckStateChanged, SlateOnCheckStateChangedCallback) )
		.Style(&WidgetStyle)
		.HAlign( HorizontalAlignment )
		.ClickMethod(ClickMethod)
		.TouchMethod(TouchMethod)
		.PressMethod(PressMethod)
		.IsFocusable(IsFocusable)
		;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if ( GetChildrenCount() > 0 )
	{
		MyCheckbox->SetContent(GetContentSlot()->Content ? GetContentSlot()->Content->TakeWidget() : SNullWidget::NullWidget);
	}
	
	return MyCheckbox.ToSharedRef();
}

void UCheckBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyCheckbox.IsValid())
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyCheckbox->SetStyle(&WidgetStyle);
	MyCheckbox->SetIsChecked( PROPERTY_BINDING(ECheckBoxState, CheckedState) );
	MyCheckbox->SetClickMethod(ClickMethod);
	MyCheckbox->SetTouchMethod(TouchMethod);
	MyCheckbox->SetPressMethod(PressMethod);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UCheckBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if ( MyCheckbox.IsValid() )
	{
		MyCheckbox->SetContent(InSlot->Content ? InSlot->Content->TakeWidget() : SNullWidget::NullWidget);
	}
}

void UCheckBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyCheckbox.IsValid() )
	{
		MyCheckbox->SetContent(SNullWidget::NullWidget);
	}
}

bool UCheckBox::IsPressed() const
{
	if ( MyCheckbox.IsValid() )
	{
		return MyCheckbox->IsPressed();
	}

	return false;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
EButtonClickMethod::Type UCheckBox::GetClickMethod() const
{
	return ClickMethod;
}

void UCheckBox::SetClickMethod(EButtonClickMethod::Type InClickMethod)
{
	ClickMethod = InClickMethod;
	if (MyCheckbox.IsValid())
	{
		MyCheckbox->SetClickMethod(ClickMethod);
	}
}

EButtonTouchMethod::Type UCheckBox::GetTouchMethod() const
{
	return TouchMethod;
}

void UCheckBox::SetTouchMethod(EButtonTouchMethod::Type InTouchMethod)
{
	TouchMethod = InTouchMethod;
	if (MyCheckbox.IsValid())
	{
		MyCheckbox->SetTouchMethod(TouchMethod);
	}
}

void UCheckBox::SetPressMethod(EButtonPressMethod::Type InPressMethod)
{
	PressMethod = InPressMethod;
	if (MyCheckbox.IsValid())
	{
		MyCheckbox->SetPressMethod(PressMethod);
	}
}

EButtonPressMethod::Type UCheckBox::GetPressMethod() const
{
	return PressMethod;
}

bool UCheckBox::IsChecked() const
{
	if ( MyCheckbox.IsValid() )
	{
		return MyCheckbox->IsChecked();
	}

	return ( CheckedState == ECheckBoxState::Checked );
}

ECheckBoxState UCheckBox::GetCheckedState() const
{
	if ( MyCheckbox.IsValid() )
	{
		return MyCheckbox->GetCheckedState();
	}

	return CheckedState;
}

void UCheckBox::SetIsChecked(bool InIsChecked)
{
	bool bValueChanged = false;

	ECheckBoxState NewState = InIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	if (NewState != CheckedState)
	{
		bValueChanged = true;
		CheckedState = NewState;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::CheckedState);
	}

	if ( MyCheckbox.IsValid() )
	{
		MyCheckbox->SetIsChecked(PROPERTY_BINDING(ECheckBoxState, CheckedState));
	}

	if (bValueChanged)
	{
		BroadcastEnumPostStateChange(InIsChecked ? UWidgetCheckedStateRegistration::Checked : UWidgetCheckedStateRegistration::Unchecked);
	}
}

void UCheckBox::SetCheckedState(ECheckBoxState InCheckedState)
{
	bool bValueChanged = false;

	if (CheckedState != InCheckedState)
	{
		bValueChanged = true;
		CheckedState = InCheckedState;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::CheckedState);
	}

	if ( MyCheckbox.IsValid() )
	{
		MyCheckbox->SetIsChecked(PROPERTY_BINDING(ECheckBoxState, CheckedState));
	}

	if (bValueChanged)
	{
		BroadcastEnumPostStateChange(UWidgetCheckedStateRegistration::GetBitfieldFromValue((uint8)CheckedState));
	}
}

const FCheckBoxStyle& UCheckBox::GetWidgetStyle() const
{
	return WidgetStyle;
}

void UCheckBox::SetWidgetStyle(const FCheckBoxStyle& InStyle)
{
	WidgetStyle = InStyle;

	if (MyCheckbox)
	{
		MyCheckbox->SetStyle(&WidgetStyle);
	}
}

bool UCheckBox::GetIsFocusable() const
{
	return IsFocusable;
}

void UCheckBox::InitIsFocusable(bool InIsFocusable)
{
	ensureMsgf(!MyCheckbox.IsValid(), TEXT("The widget is already created."));
	IsFocusable = InIsFocusable;
}

void UCheckBox::InitCheckedStateDelegate(FGetCheckBoxState InCheckedStateDelegate)
{
	ensureMsgf(!MyCheckbox.IsValid(), TEXT("The widget is already created."));
	CheckedStateDelegate = InCheckedStateDelegate;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UCheckBox::SlateOnCheckStateChangedCallback(ECheckBoxState NewState)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (CheckedState != NewState)
	{
		CheckedState = NewState;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::CheckedState);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const bool bWantsToBeChecked = NewState != ECheckBoxState::Unchecked;
	OnCheckStateChanged.Broadcast(bWantsToBeChecked);
}

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> UCheckBox::GetAccessibleWidget() const
{
	return MyCheckbox;
}
#endif

#if WITH_EDITOR

const FText UCheckBox::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

#endif

FWidgetStateBitfield UWidgetCheckedStateRegistration::Unchecked;
FWidgetStateBitfield UWidgetCheckedStateRegistration::Checked;
FWidgetStateBitfield UWidgetCheckedStateRegistration::Undetermined;

FName UWidgetCheckedStateRegistration::GetStateName() const
{
	return StateName;
};

uint8 UWidgetCheckedStateRegistration::GetRegisteredWidgetState(const UWidget* InWidget) const
{
	if (const UCheckBox* CheckBox = Cast<UCheckBox>(InWidget))
	{
		return (uint8)CheckBox->GetCheckedState();
	}

	return 0;
}

const FWidgetStateBitfield& UWidgetCheckedStateRegistration::GetBitfieldFromValue(uint8 InValue)
{
	switch ((ECheckBoxState)InValue)
	{
	case ECheckBoxState::Unchecked:
		return UWidgetCheckedStateRegistration::Unchecked;
	case ECheckBoxState::Checked:
		return UWidgetCheckedStateRegistration::Checked;
	case ECheckBoxState::Undetermined:
		return UWidgetCheckedStateRegistration::Undetermined;
	default:
		return UWidgetCheckedStateRegistration::Undetermined;
	}
}

void UWidgetCheckedStateRegistration::InitializeStaticBitfields() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Unchecked = FWidgetStateBitfield(GetStateName(), (uint8)ECheckBoxState::Unchecked);
	Checked = FWidgetStateBitfield(GetStateName(), (uint8)ECheckBoxState::Checked);
	Undetermined = FWidgetStateBitfield(GetStateName(), (uint8)ECheckBoxState::Undetermined);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
