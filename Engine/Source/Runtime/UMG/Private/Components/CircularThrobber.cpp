// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/CircularThrobber.h"
#include "Slate/SlateBrushAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Components/CanvasPanelSlot.h"
#include "Widgets/Images/SThrobber.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CircularThrobber)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UCircularThrobber

UCircularThrobber::UCircularThrobber(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableRadius(true)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Image = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetCircularThrobberBrushStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		Image = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetCircularThrobberBrushStyle();
	}
#endif // WITH_EDITOR

	NumberOfPieces = 6;
	Period = 0.75f;
	Radius = 16.f;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UCircularThrobber::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyCircularThrobber.Reset();
}

TSharedRef<SWidget> UCircularThrobber::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyCircularThrobber = SNew(SCircularThrobber)
		.PieceImage(&Image)
		.NumPieces(FMath::Clamp(NumberOfPieces, 1, 25))
		.Period(FMath::Max(Period, SCircularThrobber::MinimumPeriodValue))
		.Radius(Radius);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return MyCircularThrobber.ToSharedRef();
}

void UCircularThrobber::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyCircularThrobber.IsValid())
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyCircularThrobber->SetNumPieces(FMath::Clamp(NumberOfPieces, 1, 25));
	MyCircularThrobber->SetPeriod(FMath::Max(Period, SCircularThrobber::MinimumPeriodValue));
	MyCircularThrobber->SetRadius(Radius);
	MyCircularThrobber->SetPieceImage(&Image);
	MyCircularThrobber->InvalidatePieceImage();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// If widget is child of Canvas Panel and 'Size to Content' is enabled, we allow user to modify radius.
	bEnableRadius = true;
	if (UCanvasPanelSlot* Panel = Cast<UCanvasPanelSlot>(Slot))
	{
		if (!Panel->GetAutoSize())
		{
			bEnableRadius = false;
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UCircularThrobber::SetNumberOfPieces(int32 InNumberOfPieces)
{
	NumberOfPieces = InNumberOfPieces;
	if (MyCircularThrobber.IsValid())
	{
		MyCircularThrobber->SetNumPieces(FMath::Clamp(NumberOfPieces, 1, 25));
	}
}

int32 UCircularThrobber::GetNumberOfPieces() const
{
	return NumberOfPieces;
}

void UCircularThrobber::SetPeriod(float InPeriod)
{
	Period = InPeriod;
	if (MyCircularThrobber.IsValid())
	{
		MyCircularThrobber->SetPeriod(FMath::Max(InPeriod, SCircularThrobber::MinimumPeriodValue));
	}
}

float UCircularThrobber::GetPeriod() const
{
	return Period;
}

void UCircularThrobber::SetRadius(float InRadius)
{
	Radius = InRadius;
	if (MyCircularThrobber.IsValid())
	{
		MyCircularThrobber->SetRadius(InRadius);
	}
}

float UCircularThrobber::GetRadius() const
{
	return Radius;
}

void UCircularThrobber::SetImage(const FSlateBrush& InImage)
{
	Image = InImage;
	if (MyCircularThrobber.IsValid())
	{
		MyCircularThrobber->InvalidatePieceImage();
	}
}

const FSlateBrush& UCircularThrobber::GetImage() const
{
	return Image;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UCircularThrobber::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

