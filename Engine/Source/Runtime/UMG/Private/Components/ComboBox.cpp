// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ComboBox.h"
#include "Styling/DefaultStyleCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComboBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UComboBox

UComboBox::UComboBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsFocusable = true;
	ScrollBarStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBarStyle();
}

void UComboBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyComboBox.Reset();
}

TSharedRef<SWidget> UComboBox::RebuildWidget()
{
	TSet<UObject*> UniqueItems(Items);
	Items = UniqueItems.Array();
	
	MyComboBox =
		SNew(SComboBox<UObject*>)
		.OptionsSource(&ToRawPtrTArrayUnsafe(Items))
		.OnGenerateWidget(BIND_UOBJECT_DELEGATE(SComboBox<UObject*>::FOnGenerateWidget, HandleGenerateWidget))
		.IsFocusable(bIsFocusable)
		.ScrollBarStyle(&ScrollBarStyle)
	;

	return MyComboBox.ToSharedRef();
}

TSharedRef<SWidget> UComboBox::HandleGenerateWidget(UObject* Item) const
{
	// Call the user's delegate to see if they want to generate a custom widget bound to the data source.
	if ( OnGenerateWidgetEvent.IsBound() )
	{
		UWidget* Widget = OnGenerateWidgetEvent.Execute(Item);
		if ( Widget != nullptr )
		{
			return Widget->TakeWidget();
		}
	}

	// If a row wasn't generated just create the default one, a simple text block of the item's name.
	return SNew(STextBlock).Text(Item ? FText::FromString(Item->GetName()) : LOCTEXT("null", "null"));
}

#if WITH_EDITOR

const FText UComboBox::GetPaletteCategory()
{
	return LOCTEXT("Misc", "Misc");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

