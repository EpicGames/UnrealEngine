// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ComboBoxKey.h"

#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"
#include "UObject/ConstructorHelpers.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComboBoxKey)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UComboBoxKey

UComboBoxKey::UComboBoxKey()
{
	// We don't want to try and load fonts on the server.
	if (!IsRunningDedicatedServer())
	{
#if WITH_EDITOR 
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		WidgetStyle = IsEditorWidget() ? UE::Slate::Private::FDefaultStyleCache::GetEditor().GetComboBoxStyle() : UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetComboBoxStyle();
		ItemStyle = IsEditorWidget() ? UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTableRowStyle() : UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetTableRowStyle();
		ScrollBarStyle = IsEditorWidget() ? UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle() : UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBarStyle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetComboBoxStyle();
		ItemStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetTableRowStyle();
		ScrollBarStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBarStyle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ContentPadding = FMargin(4.0, 2.0);
	ForegroundColor = ItemStyle.TextColor;

	MaxListHeight = 450.0f;
	bHasDownArrow = true;
	bEnableGamepadNavigationMode = true;
	bIsFocusable = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


void UComboBoxKey::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyComboBox.Reset();
	ComboBoxContent.Reset();
}


TSharedRef<SWidget> UComboBoxKey::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyComboBox =
		SNew(SComboBox<FName>)
		.ComboBoxStyle(&WidgetStyle)
		.ItemStyle(&ItemStyle)
		.ForegroundColor(ForegroundColor)
		.OptionsSource(&Options)
		.InitiallySelectedItem(SelectedOption)
		.ContentPadding(ContentPadding)
		.MaxListHeight(MaxListHeight)
		.HasDownArrow(bHasDownArrow)
		.EnableGamepadNavigationMode(bEnableGamepadNavigationMode)
		.OnGenerateWidget_UObject(this, &UComboBoxKey::HandleGenerateItemWidget)
		.OnSelectionChanged_UObject(this, &UComboBoxKey::HandleSelectionChanged)
		.OnComboBoxOpening_UObject(this, &UComboBoxKey::HandleOpening)
		.IsFocusable(bIsFocusable)
		.ScrollBarStyle(&ScrollBarStyle)
		[
			SAssignNew(ComboBoxContent, SBox)
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	GenerateContent();

	return MyComboBox.ToSharedRef();
}


void UComboBoxKey::AddOption(FName Key)
{
	if (Options.AddUnique(Key) >= 0)
	{
		if (MyComboBox)
		{
			MyComboBox->RefreshOptions();
		}
	}
}


bool UComboBoxKey::RemoveOption(FName Key)
{
	bool bResult = Options.RemoveSingle(Key) > 0;
	if (bResult)
	{
		if (Key == SelectedOption)
		{
			ClearSelection();
		}

		if (MyComboBox)
		{
			MyComboBox->RefreshOptions();
		}
	}
	return bResult;
}


void UComboBoxKey::ClearOptions()
{
	Options.Reset();
	ClearSelection();
	if (MyComboBox)
	{
		MyComboBox->RefreshOptions();
	}
}


void UComboBoxKey::ClearSelection()
{
	if (MyComboBox)
	{
		MyComboBox->ClearSelection();
	}
}


void UComboBoxKey::SetSelectedOption(FName Option)
{
	if (SelectedOption != Option)
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::SelectedOption);
		if (MyComboBox)
		{
			MyComboBox->SetSelectedItem(Option);
		}
		else
		{
			SelectedOption = Option;
		}
	}
}

FName UComboBoxKey::GetSelectedOption() const
{
	return SelectedOption;
}


bool UComboBoxKey::IsOpen() const
{
	return MyComboBox && MyComboBox->IsOpen();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void  UComboBoxKey::SetContentPadding(FMargin InPadding)
{
	ContentPadding = InPadding;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetButtonContentPadding(InPadding);
	}
}

FMargin  UComboBoxKey::GetContentPadding() const
{
	return ContentPadding;
}

void UComboBoxKey::SetEnableGamepadNavigationMode(bool InEnableGamepadNavigationMode)
{
	bEnableGamepadNavigationMode = InEnableGamepadNavigationMode;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetEnableGamepadNavigationMode(bEnableGamepadNavigationMode);
	}
}

bool UComboBoxKey::IsEnableGamepadNavigationMode() const
{
	return bEnableGamepadNavigationMode;
}

bool UComboBoxKey::IsHasDownArrow() const
{
	return bHasDownArrow;
}

void UComboBoxKey::SetHasDownArrow(bool InHasDownArrow)
{
	bHasDownArrow = InHasDownArrow;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetHasDownArrow(InHasDownArrow);
	}
}

float UComboBoxKey::GetMaxListHeight() const
{
	return MaxListHeight;
}

void UComboBoxKey::SetMaxListHeight(float InMaxHeight)
{
	MaxListHeight = InMaxHeight;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetMaxHeight(MaxListHeight);
	}
}

const FComboBoxStyle& UComboBoxKey::GetWidgetStyle() const
{
	return WidgetStyle;
}

const FTableRowStyle& UComboBoxKey::GetItemStyle() const
{
	return ItemStyle;
}

const FScrollBarStyle& UComboBoxKey::GetScrollBarStyle() const
{
	return ScrollBarStyle;
}

bool UComboBoxKey::IsFocusable() const
{
	return bIsFocusable;
}

FSlateColor UComboBoxKey::GetForegroundColor() const
{
	return ForegroundColor;
}

void UComboBoxKey::SetWidgetStyle(const FComboBoxStyle& InWidgetStyle)
{
	WidgetStyle = InWidgetStyle;
	if (MyComboBox.IsValid())
	{
		MyComboBox->InvalidateStyle();
	}
}

void UComboBoxKey::SetItemStyle(const FTableRowStyle& InItemStyle)
{
	ItemStyle = InItemStyle;
	if (MyComboBox.IsValid())
	{
		MyComboBox->InvalidateItemStyle();
	}
}

void UComboBoxKey::InitScrollBarStyle(const FScrollBarStyle& InScrollBarStyle)
{
	ensureMsgf(!MyComboBox.IsValid(), TEXT("The widget is already created."));
	ScrollBarStyle = InScrollBarStyle;
}

void UComboBoxKey::InitIsFocusable(bool InIsFocusable)
{
	ensureMsgf(!MyComboBox.IsValid(), TEXT("The widget is already created."));
	bIsFocusable = InIsFocusable;
}

void UComboBoxKey::InitForegroundColor(FSlateColor InForegroundColor)
{
	ensureMsgf(!MyComboBox.IsValid(), TEXT("The widget is already created."));
	ForegroundColor = InForegroundColor;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UComboBoxKey::GenerateContent()
{
	if (ComboBoxContent.IsValid())
	{
		bool bSetToNull = true;
		if (IsDesignTime())
		{
			if (!SelectedOption.IsNone())
			{
				ComboBoxContent->SetContent(SNew(STextBlock)
					.Text(FText::FromName(SelectedOption)));
				bSetToNull = false;
			}
		}
		else
		{
			// Call the user's delegate to see if they want to generate a custom widget bound to the data source.
			if (OnGenerateContentWidget.IsBound())
			{
				if (UWidget* Widget = OnGenerateContentWidget.Execute(SelectedOption))
				{
					ComboBoxContent->SetContent(Widget->TakeWidget());
					bSetToNull = false;
				}
			}
			else
			{
				// Warn the user that they need to generate a widget for the item
				ComboBoxContent->SetContent(
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("WarningMissingOnGenerateContentWidget", "{0} Missing OnGenerateContentWidget"), FText::FromName(SelectedOption)))
				);
				bSetToNull = false;
			}
		}

		if (bSetToNull)
		{
			ComboBoxContent->SetContent(SNullWidget::NullWidget);
		}
	}
}


TSharedRef<SWidget> UComboBoxKey::HandleGenerateItemWidget(FName Item)
{
	if (IsDesignTime())
	{
		return SNew(STextBlock)
			.Text(FText::FromName(Item));
	}
	else
	{
		// Call the user's delegate to see if they want to generate a custom widget bound to the data source.
		if (OnGenerateItemWidget.IsBound())
		{
			if (UWidget* Widget = OnGenerateItemWidget.Execute(Item))
			{
				return Widget->TakeWidget();
			}
			else
			{
				return SNullWidget::NullWidget;
			}
		}
	}

	return SNew(STextBlock)
		.Text(FText::Format(LOCTEXT("WarningMissingOnGenerateItemWidget", "{0} Missing OnGenerateItemWidget"), FText::FromName(Item)));
}


void UComboBoxKey::HandleSelectionChanged(FName Item, ESelectInfo::Type SelectionType)
{
	if (SelectedOption != Item)
	{
		SelectedOption = Item;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::SelectedOption);

		if (!IsDesignTime())
		{
			OnSelectionChanged.Broadcast(Item, SelectionType);
		}
		GenerateContent();
	}
}

void UComboBoxKey::HandleOpening()
{
	if (!IsDesignTime())
	{
		OnOpening.Broadcast();
	}
}

#if WITH_EDITOR

const FText UComboBoxKey::GetPaletteCategory()
{
	return LOCTEXT("Input", "Input");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

