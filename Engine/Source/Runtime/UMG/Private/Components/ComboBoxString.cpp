// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ComboBoxString.h"

#include "Widgets/SNullWidget.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Font.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ComboBoxString)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UComboBoxString

UComboBoxString::UComboBoxString(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetComboBoxStyle();
	ItemStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetTableRowStyle();
	ScrollBarStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBarStyle();

#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetComboBoxStyle();
		ItemStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTableRowStyle();
		ScrollBarStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle();
	}
#endif
	
	ForegroundColor = ItemStyle.TextColor;
	bIsFocusable = true;

	ContentPadding = FMargin(4.0, 2.0);
	MaxListHeight = 450.0f;
	HasDownArrow = true;
	EnableGamepadNavigationMode = true;
	// We don't want to try and load fonts on the server.
	if ( !IsRunningDedicatedServer() )
	{
		static ConstructorHelpers::FObjectFinder<UFont> RobotoFontObj(*UWidget::GetDefaultFontName());
		Font = FSlateFontInfo(RobotoFontObj.Object, 16, FName("Bold"));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UComboBoxString::PostInitProperties()
{
	Super::PostInitProperties();

	// Initialize the set of options from the default set only once.
	for (const FString& DefaultOption : DefaultOptions)
	{
		AddOption(DefaultOption);
	}
}

void UComboBoxString::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyComboBox.Reset();
	ComboBoxContent.Reset();
}

void UComboBoxString::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
}

void UComboBoxString::PostLoad()
{
	Super::PostLoad();

	// Initialize the set of options from the default set only once.
	for (const FString& DefaultOption : DefaultOptions)
	{
		AddOption(DefaultOption);
	}

	if (GetLinkerCustomVersion(FEditorObjectVersion::GUID) < FEditorObjectVersion::ComboBoxControllerSupportUpdate)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EnableGamepadNavigationMode = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

TSharedRef<SWidget> UComboBoxString::RebuildWidget()
{
	const int32 InitialIndex = FindOptionIndex(SelectedOption);
	if ( InitialIndex != -1 )
	{
		CurrentOptionPtr = Options[InitialIndex];
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyComboBox =
		SNew(SComboBox< TSharedPtr<FString> >)
		.ComboBoxStyle(&WidgetStyle)
		.ItemStyle(&ItemStyle)
		.ForegroundColor(ForegroundColor)
		.OptionsSource(&Options)
		.InitiallySelectedItem(CurrentOptionPtr)
		.ContentPadding(ContentPadding)
		.MaxListHeight(MaxListHeight)
		.HasDownArrow(HasDownArrow)
		.EnableGamepadNavigationMode(EnableGamepadNavigationMode)
		.OnGenerateWidget(BIND_UOBJECT_DELEGATE(SComboBox< TSharedPtr<FString> >::FOnGenerateWidget, HandleGenerateWidget))
		.OnSelectionChanged(BIND_UOBJECT_DELEGATE(SComboBox< TSharedPtr<FString> >::FOnSelectionChanged, HandleSelectionChanged))
		.OnComboBoxOpening(BIND_UOBJECT_DELEGATE(FOnComboBoxOpening, HandleOpening))
	    .ScrollBarStyle(&ScrollBarStyle)
		.IsFocusable(bIsFocusable)
		[
			SAssignNew(ComboBoxContent, SBox)
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if ( InitialIndex != -1 )
	{
		// Generate the widget for the initially selected widget if needed
		UpdateOrGenerateWidget(CurrentOptionPtr);
	}

	return MyComboBox.ToSharedRef();
}

void UComboBoxString::AddOption(const FString& Option)
{
	Options.Add(MakeShareable(new FString(Option)));

	RefreshOptions();
}

bool UComboBoxString::RemoveOption(const FString& Option)
{
	const int32 OptionIndex = FindOptionIndex(Option);

	if ( OptionIndex != -1 )
	{
		if ( Options[OptionIndex] == CurrentOptionPtr )
		{
			ClearSelection();
		}

		Options.RemoveAt(OptionIndex);

		RefreshOptions();

		return true;
	}

	return false;
}

int32 UComboBoxString::FindOptionIndex(const FString& Option) const
{
	for ( int32 OptionIndex = 0; OptionIndex < Options.Num(); OptionIndex++ )
	{
		const TSharedPtr<FString>& OptionAtIndex = Options[OptionIndex];

		if ( ( *OptionAtIndex ) == Option )
		{
			return OptionIndex;
		}
	}

	return -1;
}

FString UComboBoxString::GetOptionAtIndex(int32 Index) const
{
	if (Index >= 0 && Index < Options.Num())
	{
		return *(Options[Index]);
	}
	return FString();
}

void UComboBoxString::ClearOptions()
{
	ClearSelection();

	Options.Empty();

	if ( MyComboBox.IsValid() )
	{
		MyComboBox->RefreshOptions();
	}
}

void UComboBoxString::ClearSelection()
{
	CurrentOptionPtr.Reset();

	if ( MyComboBox.IsValid() )
	{
		MyComboBox->ClearSelection();
	}

	if ( ComboBoxContent.IsValid() )
	{
		ComboBoxContent->SetContent(SNullWidget::NullWidget);
	}
}

void UComboBoxString::RefreshOptions()
{
	if ( MyComboBox.IsValid() )
	{
		MyComboBox->RefreshOptions();
	}
}

void UComboBoxString::SetSelectedOption(FString Option)
{
	const int32 InitialIndex = FindOptionIndex(Option);
	SetSelectedIndex(InitialIndex);
}

void UComboBoxString::SetSelectedIndex(const int32 Index)
{
	if (Options.IsValidIndex(Index))
	{
		CurrentOptionPtr = Options[Index];
		// Don't select item if its already selected
		if (SelectedOption != *CurrentOptionPtr)
		{
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::SelectedOption);
			SelectedOption = *CurrentOptionPtr;

			if (ComboBoxContent.IsValid())
			{
				MyComboBox->SetSelectedItem(CurrentOptionPtr);
				UpdateOrGenerateWidget(CurrentOptionPtr);
			}		
			else
			{
				HandleSelectionChanged(CurrentOptionPtr, ESelectInfo::Direct);
			}
		}
	}
}
FString UComboBoxString::GetSelectedOption() const
{
	if (CurrentOptionPtr.IsValid())
	{
		return *CurrentOptionPtr;
	}
	return FString();
}

int32 UComboBoxString::GetSelectedIndex() const
{
	if (CurrentOptionPtr.IsValid())
	{
		for (int32 OptionIndex = 0; OptionIndex < Options.Num(); ++OptionIndex)
		{
			if (Options[OptionIndex] == CurrentOptionPtr)
			{
				return OptionIndex;
			}
		}
	}

	return -1;
}

int32 UComboBoxString::GetOptionCount() const
{
	return Options.Num();
}

bool UComboBoxString::IsOpen() const
{
	return MyComboBox.IsValid() && MyComboBox->IsOpen();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void  UComboBoxString::SetContentPadding(FMargin InPadding)
{
	ContentPadding = InPadding;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetButtonContentPadding(InPadding);
	}
}

FMargin  UComboBoxString::GetContentPadding() const
{
	return ContentPadding;
}

void UComboBoxString::SetEnableGamepadNavigationMode(bool InEnableGamepadNavigationMode)
{
	EnableGamepadNavigationMode = InEnableGamepadNavigationMode;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetEnableGamepadNavigationMode(EnableGamepadNavigationMode);
	}
}

bool UComboBoxString::IsEnableGamepadNavigationMode() const
{
	return EnableGamepadNavigationMode;
}

bool UComboBoxString::IsHasDownArrow() const
{
	return HasDownArrow;
}

void UComboBoxString::SetHasDownArrow(bool InHasDownArrow)
{
	HasDownArrow = InHasDownArrow;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetHasDownArrow(HasDownArrow);
	}
}

float UComboBoxString::GetMaxListHeight() const
{
	return MaxListHeight;
}

void UComboBoxString::SetMaxListHeight(float InMaxHeight)
{
	MaxListHeight = InMaxHeight;
	if (MyComboBox.IsValid())
	{
		MyComboBox->SetMaxHeight(MaxListHeight);
	}
}

const FSlateFontInfo&  UComboBoxString::GetFont() const
{
	return Font;
}

const FComboBoxStyle& UComboBoxString::GetWidgetStyle() const
{
	return WidgetStyle;
}

const FTableRowStyle& UComboBoxString::GetItemStyle() const
{
	return ItemStyle;
}

const FScrollBarStyle& UComboBoxString::GetScrollBarStyle() const
{
	return ScrollBarStyle;
}

bool UComboBoxString::IsFocusable() const
{
	return bIsFocusable;
}

FSlateColor UComboBoxString::GetForegroundColor() const
{
	return ForegroundColor;
}

void UComboBoxString::SetWidgetStyle(const FComboBoxStyle& InWidgetStyle)
{
	WidgetStyle = InWidgetStyle;
	if (MyComboBox.IsValid())
	{
		MyComboBox->InvalidateStyle();
	}
}

void UComboBoxString::SetItemStyle(const FTableRowStyle& InItemStyle)
{
	ItemStyle = InItemStyle;
	if (MyComboBox.IsValid())
	{
		MyComboBox->InvalidateItemStyle();
	}
}

void UComboBoxString::InitScrollBarStyle(const FScrollBarStyle& InScrollBarStyle)
{
	ensureMsgf(!MyComboBox.IsValid(), TEXT("The widget is already created."));
	ScrollBarStyle = InScrollBarStyle;
}

void UComboBoxString::InitFont(FSlateFontInfo InFont)
{
	ensureMsgf(!MyComboBox.IsValid(), TEXT("The widget is already created."));
	Font = InFont;
}

void UComboBoxString::InitIsFocusable(bool InIsFocusable)
{
	ensureMsgf(!MyComboBox.IsValid(), TEXT("The widget is already created."));
	bIsFocusable = InIsFocusable;
}

void UComboBoxString::InitForegroundColor(FSlateColor InForegroundColor)
{
	ensureMsgf(!MyComboBox.IsValid(), TEXT("The widget is already created."));
	ForegroundColor = InForegroundColor;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UComboBoxString::UpdateOrGenerateWidget(TSharedPtr<FString> Item)
{
	// If no custom widget was supplied and the default STextBlock already exists,
	// just update its text instead of rebuilding the widget.
	if (DefaultComboBoxContent.IsValid() && (IsDesignTime() || OnGenerateWidgetEvent.IsBound()))
	{
		const FString StringItem = Item.IsValid() ? *Item : FString();
		DefaultComboBoxContent.Pin()->SetText(FText::FromString(StringItem));
	}
	else if (ComboBoxContent.IsValid())
	{
		DefaultComboBoxContent.Reset();
		ComboBoxContent->SetContent(HandleGenerateWidget(Item));
	}
}

TSharedRef<SWidget> UComboBoxString::HandleGenerateWidget(TSharedPtr<FString> Item) const
{
	const FString StringItem = Item.IsValid() ? *Item : FString();

	// Call the user's delegate to see if they want to generate a custom widget bound to the data source.
	if ( !IsDesignTime() && OnGenerateWidgetEvent.IsBound() )
	{
		UWidget* Widget = OnGenerateWidgetEvent.Execute(StringItem);
		if ( Widget != NULL )
		{
			return Widget->TakeWidget();
		}
	}

	// If a row wasn't generated just create the default one, a simple text block of the item's name.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return SNew(STextBlock)
		.Text(FText::FromString(StringItem))
		.Font(Font);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UComboBoxString::HandleSelectionChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectionType)
{
	CurrentOptionPtr = Item;
	BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::SelectedOption);
	SelectedOption = CurrentOptionPtr.IsValid() ? CurrentOptionPtr.ToSharedRef().Get() : FString();

	// When the selection changes we always generate another widget to represent the content area of the combobox.
	if ( ComboBoxContent.IsValid() )
	{
		UpdateOrGenerateWidget(CurrentOptionPtr);
	}

	if ( !IsDesignTime() )
	{
		OnSelectionChanged.Broadcast(Item.IsValid() ? *Item : FString(), SelectionType);
	}
}

void UComboBoxString::HandleOpening()
{
	OnOpening.Broadcast();
}

#if WITH_EDITOR

const FText UComboBoxString::GetPaletteCategory()
{
	return LOCTEXT("Input", "Input");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
