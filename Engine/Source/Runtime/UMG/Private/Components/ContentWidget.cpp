// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ContentWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentWidget)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UContentWidget

UContentWidget::UContentWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanHaveMultipleChildren = false;
}

UPanelSlot* UContentWidget::GetContentSlot() const
{
	return Slots.Num() > 0 ? Slots[0] : nullptr;
}

UWidget* UContentWidget::GetContent() const
{
	if ( UPanelSlot* ContentSlot = GetContentSlot() )
	{
		return ContentSlot->Content;
	}

	return nullptr;
}

UPanelSlot* UContentWidget::SetContent(UWidget* Content)
{
	ClearChildren();
	return AddChild(Content);
}

UClass* UContentWidget::GetSlotClass() const
{
	return UPanelSlot::StaticClass();
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

