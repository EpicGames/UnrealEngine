// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DynamicEntryBox.h"
#include "UMGPrivate.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Editor/WidgetCompilerLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicEntryBox)

#define LOCTEXT_NAMESPACE "UMG"

//////////////////////////////////////////////////////////////////////////
// UDynamicEntryBox
//////////////////////////////////////////////////////////////////////////

void UDynamicEntryBox::Reset(bool bDeleteWidgets)
{
	ResetInternal(bDeleteWidgets);
}

void UDynamicEntryBox::RemoveEntry(UUserWidget* EntryWidget)
{
	RemoveEntryInternal(EntryWidget);
}

#if WITH_EDITOR
void UDynamicEntryBox::ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const
{
	Super::ValidateCompiledDefaults(CompileLog);
	
	if (!EntryWidgetClass)
	{
		CompileLog.Error(FText::Format(LOCTEXT("Error_DynamicEntryBox_MissingEntryClass", "{0} has no EntryWidgetClass specified - required for any Dynamic Entry Box to function."), FText::FromString(GetName())));
	}
	else if (CompileLog.GetContextClass() && EntryWidgetClass->IsChildOf(CompileLog.GetContextClass()))
	{
		CompileLog.Error(FText::Format(LOCTEXT("Error_DynamicEntryBox_RecursiveEntryClass", "{0} has a recursive EntryWidgetClass specified (reference itself)."), FText::FromString(GetName())));
	}
}
#endif

void UDynamicEntryBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	// At design-time, preview the desired number of entries
#if WITH_EDITORONLY_DATA
	if (IsDesignTime() && MyPanelWidget.IsValid())
	{
		if (!EntryWidgetClass || !IsEntryClassValid(EntryWidgetClass))
		{
			// We have no entry class, so clear everything out
			Reset(true);
		}
		else if (MyPanelWidget->GetChildren()->Num() != NumDesignerPreviewEntries)
		{
			// When the number of entries to preview changes, the easiest thing to do is just soft-rebuild
			Reset();
			int32 StartingNumber = MyPanelWidget->GetChildren()->Num();
			while (StartingNumber < NumDesignerPreviewEntries)
			{
				UUserWidget* PreviewEntry = CreateEntryInternal(EntryWidgetClass);
				if (PreviewEntry && IsDesignTime() && OnPreviewEntryCreatedFunc)
				{
					OnPreviewEntryCreatedFunc(PreviewEntry);
				}
				StartingNumber++;
			}
		}
	}
#endif
}

UUserWidget* UDynamicEntryBox::BP_CreateEntry()
{
	return CreateEntry();
}

UUserWidget* UDynamicEntryBox::BP_CreateEntryOfClass(TSubclassOf<UUserWidget> EntryClass)
{
	if (EntryClass && IsEntryClassValid(EntryClass))
	{
		return CreateEntryInternal(EntryClass);
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
