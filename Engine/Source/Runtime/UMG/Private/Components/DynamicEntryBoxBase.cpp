// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DynamicEntryBoxBase.h"
#include "UMGPrivate.h"
#include "Blueprint/WidgetTree.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/UserWidgetPool.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Editor/WidgetCompilerLog.h"
#include "Widgets/Layout/SRadialBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicEntryBoxBase)

#define LOCTEXT_NAMESPACE "UMG"

UDynamicEntryBoxBase::UDynamicEntryBoxBase(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, EntryWidgetPool(*this)
{
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EntrySizeRule.SizeRule = ESlateSizeRule::Automatic;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDynamicEntryBoxBase::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	EntryWidgetPool.ReleaseAllSlateResources();
	MyPanelWidget.Reset();
}

void UDynamicEntryBoxBase::ResetInternal(bool bDeleteWidgets)
{
	EntryWidgetPool.ReleaseAll(bDeleteWidgets);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (MyPanelWidget.IsValid())
	{
		switch (EntryBoxType)
		{
		case EDynamicBoxType::Horizontal:
		case EDynamicBoxType::Vertical:
			StaticCastSharedPtr<SBoxPanel>(MyPanelWidget)->ClearChildren();
			break;
		case EDynamicBoxType::Wrap:
		case EDynamicBoxType::VerticalWrap:
			StaticCastSharedPtr<SWrapBox>(MyPanelWidget)->ClearChildren();
			break;
		case EDynamicBoxType::Radial:
			StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->ClearChildren();
			break;
		case EDynamicBoxType::Overlay:
			StaticCastSharedPtr<SOverlay>(MyPanelWidget)->ClearChildren();
			break;
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const TArray<UUserWidget*>& UDynamicEntryBoxBase::GetAllEntries() const
{
	return EntryWidgetPool.GetActiveWidgets();
}

int32 UDynamicEntryBoxBase::GetNumEntries() const
{
	return EntryWidgetPool.GetActiveWidgets().Num();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EDynamicBoxType UDynamicEntryBoxBase::GetBoxType() const
{
	return EntryBoxType;
}

const FVector2D& UDynamicEntryBoxBase::GetEntrySpacing() const
{
	return EntrySpacing;
}

const FSlateChildSize& UDynamicEntryBoxBase::GetEntrySizeRule() const
{
	return EntrySizeRule;
}

const FRadialBoxSettings& UDynamicEntryBoxBase::GetRadialBoxSettings() const
{
	return RadialBoxSettings;
}

void UDynamicEntryBoxBase::RemoveEntryInternal(UUserWidget* EntryWidget, bool bReleaseSlate)
{
	if (EntryWidget)
	{
		if (MyPanelWidget.IsValid())
		{
			TSharedPtr<SWidget> CachedEntryWidget = EntryWidget->GetCachedWidget();
			if (CachedEntryWidget.IsValid())
			{
				switch (EntryBoxType)
				{
				case EDynamicBoxType::Horizontal:
				case EDynamicBoxType::Vertical:
					StaticCastSharedPtr<SBoxPanel>(MyPanelWidget)->RemoveSlot(CachedEntryWidget.ToSharedRef());
					break;
				case EDynamicBoxType::Wrap:
				case EDynamicBoxType::VerticalWrap:
					StaticCastSharedPtr<SWrapBox>(MyPanelWidget)->RemoveSlot(CachedEntryWidget.ToSharedRef());
					break;
				case EDynamicBoxType::Radial:
					StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->RemoveSlot(CachedEntryWidget.ToSharedRef());
					break;
				case EDynamicBoxType::Overlay:
					StaticCastSharedPtr<SOverlay>(MyPanelWidget)->RemoveSlot(CachedEntryWidget.ToSharedRef());
					break;
				}
			}
		}
		EntryWidgetPool.Release(EntryWidget, bReleaseSlate);
	}
}

void UDynamicEntryBoxBase::SetEntrySpacing(const FVector2D& InEntrySpacing)
{
	EntrySpacing = InEntrySpacing;
	FVector2f Spacing = UE::Slate::CastToVector2f(EntrySpacing);

	if (MyPanelWidget.IsValid())
	{
		if (EntryBoxType == EDynamicBoxType::Wrap || EntryBoxType == EDynamicBoxType::VerticalWrap)
		{
			// Wrap boxes can change their widget spacing on the fly
			StaticCastSharedPtr<SWrapBox>(MyPanelWidget)->SetInnerSlotPadding(EntrySpacing);
		}
		else if (EntryBoxType == EDynamicBoxType::Overlay)
		{
			TPanelChildren<SOverlay::FOverlaySlot>* OverlayChildren = static_cast<TPanelChildren<SOverlay::FOverlaySlot>*>(MyPanelWidget->GetChildren());
			for (int32 ChildIdx = 0; ChildIdx < OverlayChildren->Num(); ++ChildIdx)
			{
				FMargin Padding;
				if (SpacingPattern.Num() > 0)
				{
					// Initialize to no spacing
					Spacing = FVector2f(0.f, 0.f);

					// First establish the starting location
					for (int32 CountIdx = 0; CountIdx < ChildIdx; ++CountIdx)
					{
						int32 PatternIdx = CountIdx % SpacingPattern.Num();
						Spacing += UE::Slate::CastToVector2f(SpacingPattern[PatternIdx]);
					}
					// Negative padding is no good, so negative spacing is expressed as positive spacing on the opposite side
					if (Spacing.X >= 0.f)
					{
						Padding.Left = Spacing.X;
					}
					else
					{
						Padding.Right = -Spacing.X;
					}

					// Negative padding is no good, so negative spacing is expressed as positive spacing on the opposite side
					if (Spacing.Y >= 0.f)
					{
						Padding.Top = Spacing.Y;
					}
					else
					{
						Padding.Bottom = -Spacing.Y;
					}
				}
				else
				{
					// Negative padding is no good, so negative spacing is expressed as positive spacing on the opposite side
					if (Spacing.X >= 0.f)
					{
						Padding.Left = ChildIdx * Spacing.X;
					}
					else
					{
						Padding.Right = ChildIdx * -Spacing.X;
					}

					// Negative padding is no good, so negative spacing is expressed as positive spacing on the opposite side
					if (Spacing.Y >= 0.f)
					{
						Padding.Top = ChildIdx * Spacing.Y;
					}
					else
					{
						Padding.Bottom = ChildIdx * -Spacing.Y;
					}
				}
				SOverlay::FOverlaySlot& OverlaySlot = (*OverlayChildren)[ChildIdx];
				OverlaySlot.SetPadding(Padding);
			}
		}
		else if (EntryBoxType == EDynamicBoxType::Horizontal || EntryBoxType == EDynamicBoxType::Vertical)
		{
			// Vertical & Horizontal have to manually update the padding on each slot
			const bool bIsHBox = EntryBoxType == EDynamicBoxType::Horizontal;
			TPanelChildren<SBoxPanel::FSlot>* BoxChildren = static_cast<TPanelChildren<SBoxPanel::FSlot>*>(MyPanelWidget->GetChildren());
			for (int32 ChildIdx = 0; ChildIdx < BoxChildren->Num(); ++ChildIdx)
			{
				const bool bIsFirstChild = ChildIdx == 0;

				FMargin Padding;
				Padding.Top = bIsHBox || bIsFirstChild ? 0.f : Spacing.Y;
				Padding.Left = bIsHBox && !bIsFirstChild ? Spacing.X : 0.f;
				SBoxPanel::FSlot& BoxSlot = (*BoxChildren)[ChildIdx];
				BoxSlot.SetPadding(Padding);
			}
		}
	}
}

void UDynamicEntryBoxBase::SetRadialSettings(const FRadialBoxSettings& InSettings)
{
	RadialBoxSettings = InSettings;

	if (MyPanelWidget.IsValid() && EntryBoxType == EDynamicBoxType::Radial)
	{
		StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->SetStartingAngle(InSettings.StartingAngle);
		StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->SetDistributeItemsEvenly(InSettings.bDistributeItemsEvenly);
		StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->SetClockwiseOrder(InSettings.bClockwiseOrder);
		StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->SetAngleBetweenItems(InSettings.AngleBetweenItems);
		StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->SetSectorCentralAngle(InSettings.SectorCentralAngle);
		StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->SetMarginSize(InSettings.MarginSize);
	}
}

EVerticalAlignment UDynamicEntryBoxBase::GetEntryVerticalAlignment() const
{
	return EntryVerticalAlignment.GetValue();
}

EHorizontalAlignment UDynamicEntryBoxBase::GetEntryHorizontalAlignment() const
{
	return EntryHorizontalAlignment.GetValue();
}

int32 UDynamicEntryBoxBase::GetMaxElementSize() const
{
	return MaxElementSize;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UDynamicEntryBoxBase::GetPaletteCategory()
{
	return LOCTEXT("Advanced", "Advanced");
}
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedRef<SWidget> UDynamicEntryBoxBase::RebuildWidget()
{
	TSharedPtr<SWidget> EntryBoxWidget;
	switch (EntryBoxType)
	{
	case EDynamicBoxType::Horizontal:
		EntryBoxWidget = SAssignNew(MyPanelWidget, SHorizontalBox);
		break;
	case EDynamicBoxType::Vertical:
		EntryBoxWidget = SAssignNew(MyPanelWidget, SVerticalBox);
		break;
	case EDynamicBoxType::Wrap:
		EntryBoxWidget = SAssignNew(MyPanelWidget, SWrapBox)
			.UseAllottedSize(true)
			.Orientation(EOrientation::Orient_Horizontal)
			.InnerSlotPadding(EntrySpacing);
		break;
	case EDynamicBoxType::VerticalWrap:
		EntryBoxWidget = SAssignNew(MyPanelWidget, SWrapBox)
			.UseAllottedSize(true)
			.Orientation(EOrientation::Orient_Vertical)
			.InnerSlotPadding(EntrySpacing);
		break;
	case EDynamicBoxType::Radial:
		EntryBoxWidget = SAssignNew(MyPanelWidget, SRadialBox)
			.UseAllottedWidth(true)
			.StartingAngle(RadialBoxSettings.StartingAngle)
			.bDistributeItemsEvenly(RadialBoxSettings.bDistributeItemsEvenly)
			.bClockwiseOrder(RadialBoxSettings.bClockwiseOrder)
			.MarginSize(RadialBoxSettings.MarginSize)
			.AngleBetweenItems(RadialBoxSettings.AngleBetweenItems)
			.SectorCentralAngle(RadialBoxSettings.SectorCentralAngle);
		break;
	case EDynamicBoxType::Overlay:
		EntryBoxWidget = SAssignNew(MyPanelWidget, SOverlay)
			.Clipping(EWidgetClipping::ClipToBounds);
		break;
	}

	if (!IsDesignTime())
	{
		// Populate with all the entries that have been created so far
		// Avoided during design time because we manage the pool a little differently for designer previews (see SynchronizeProperties)
		EntryWidgetPool.RebuildWidgets();
		for (UUserWidget* ActiveWidget : EntryWidgetPool.GetActiveWidgets())
		{
			AddEntryChild(*ActiveWidget);
		}
	}

	return EntryBoxWidget.ToSharedRef();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
void UDynamicEntryBoxBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (MyPanelWidget.IsValid() && PropertyChangedEvent.GetPropertyName() == TEXT("EntryBoxType"))
	{
		MyPanelWidget.Reset();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDynamicEntryBoxBase::SynchronizeProperties()
{
	Super::SynchronizeProperties();

#if WITH_EDITORONLY_DATA
	if (IsDesignTime())
	{
		SetEntrySpacing(EntrySpacing);
		SetRadialSettings(RadialBoxSettings);
	}
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UDynamicEntryBoxBase::IsEntryClassValid(TSubclassOf<UUserWidget> InEntryClass) const
{
	if (InEntryClass)
	{
		// Would InEntryClass create an instance of the same DynamicEntryBox
		if (UWidgetTree* WidgetTree = Cast<UWidgetTree>(GetOuter()))
		{
			if (UUserWidget* UserWidget = Cast<UUserWidget>(WidgetTree->GetOuter()))
			{
				if (InEntryClass->IsChildOf(UserWidget->GetClass()))
				{
					return false;
				}
			}
		}
	}

	return true;
}

namespace DynamicEntryBoxBaseCreateEntryInternal
{
	TArray<TSubclassOf<UUserWidget>, TInlineAllocator<4>> RecursiveDetection;
}

UUserWidget* UDynamicEntryBoxBase::CreateEntryInternal(TSubclassOf<UUserWidget> InEntryClass)
{
	const bool bHasResursiveUserWidget = DynamicEntryBoxBaseCreateEntryInternal::RecursiveDetection.ContainsByPredicate([InEntryClass](TSubclassOf<UUserWidget> RecursiveItem)
		{
			return InEntryClass->IsChildOf(RecursiveItem);
		});
	if (bHasResursiveUserWidget)
	{
		UE_LOGF(LogSlate, Error, "'%ls' cannot be added to DynamicEntry '%ls' because it is already a child and it would create a recurssion."
			, *InEntryClass->GetName()
			, *DynamicEntryBoxBaseCreateEntryInternal::RecursiveDetection.Last()->GetName());
#if 0
		for (TSubclassOf<UUserWidget> RecursiveItem : DynamicEntryBoxBaseCreateEntryInternal::RecursiveDetection)
		{
			UE_LOGF(LogSlate, Log, "%ls", *RecursiveItem->GetName());
		}
#endif
		return nullptr;
	}
	DynamicEntryBoxBaseCreateEntryInternal::RecursiveDetection.Push(InEntryClass);

	UUserWidget* NewEntryWidget = EntryWidgetPool.GetOrCreateInstance(InEntryClass);
	if (MyPanelWidget.IsValid() && NewEntryWidget != nullptr)
	{
		// If we've already been constructed, immediately add the child to our panel widget
		AddEntryChild(*NewEntryWidget);
	}

	DynamicEntryBoxBaseCreateEntryInternal::RecursiveDetection.Pop();
	return NewEntryWidget;
}

FMargin UDynamicEntryBoxBase::BuildEntryPadding(const FVector2D& InDesiredSpacing)
{
	FMargin EntryPadding;
	const FVector2f DesiredSpacing = UE::Slate::CastToVector2f(InDesiredSpacing);
	if (DesiredSpacing.X >= 0.f)
	{
		EntryPadding.Left = DesiredSpacing.X;
	}
	else
	{
		EntryPadding.Right = -DesiredSpacing.X;
	}

	if (DesiredSpacing.Y >= 0.f)
	{
		EntryPadding.Top = DesiredSpacing.Y;
	}
	else
	{
		EntryPadding.Bottom = -DesiredSpacing.Y;
	}

	return EntryPadding;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDynamicEntryBoxBase::AddEntryChild(UUserWidget& ChildWidget)
{
	if (EntryBoxType == EDynamicBoxType::Wrap || EntryBoxType == EDynamicBoxType::VerticalWrap)
	{
		StaticCastSharedPtr<SWrapBox>(MyPanelWidget)->AddSlot()
			.FillEmptySpace(false)
			.HAlign(EntryHorizontalAlignment)
			.VAlign(EntryVerticalAlignment)
			[
				ChildWidget.TakeWidget()
			];
	}
	else if (EntryBoxType == EDynamicBoxType::Radial)
	{
		StaticCastSharedPtr<SRadialBox>(MyPanelWidget)->AddSlot()
			[
				ChildWidget.TakeWidget()
			];
	}
	else if (EntryBoxType == EDynamicBoxType::Overlay)
	{
		if (MyPanelWidget.IsValid())
		{
			const int32 ChildIdx = MyPanelWidget->GetChildren()->Num();

			EHorizontalAlignment HAlign = EntryHorizontalAlignment;
			EVerticalAlignment VAlign = EntryVerticalAlignment;

			FVector2D TargetSpacing = FVector2D::ZeroVector;
			if (SpacingPattern.Num() > 0)
			{
				for (int32 CountIdx = 0; CountIdx < ChildIdx; ++CountIdx)
				{
					const int32 PatternIdx = CountIdx % SpacingPattern.Num();
					TargetSpacing += SpacingPattern[PatternIdx];
				}
			}
			else
			{
				TargetSpacing = EntrySpacing * ChildIdx;
				HAlign = EntrySpacing.X >= 0.f ? EHorizontalAlignment::HAlign_Left : EHorizontalAlignment::HAlign_Right;
				VAlign = EntrySpacing.Y >= 0.f ? EVerticalAlignment::VAlign_Top : EVerticalAlignment::VAlign_Bottom;
			}
		
			StaticCastSharedPtr<SOverlay>(MyPanelWidget)->AddSlot()
				.HAlign(HAlign)
				.VAlign(VAlign)
				.Padding(BuildEntryPadding(TargetSpacing))
				[
					ChildWidget.TakeWidget()
				];
		}
	}
	else
	{
		if (MyPanelWidget.IsValid())
		{
			const bool bIsHBox = EntryBoxType == EDynamicBoxType::Horizontal;
			const bool bIsFirstChild = MyPanelWidget->GetChildren()->Num() == 0;

			const FVector2f Spacing = UE::Slate::CastToVector2f(EntrySpacing);
			FMargin Padding;
			Padding.Top = bIsHBox || bIsFirstChild ? 0.f : Spacing.Y;
			Padding.Left = bIsHBox && !bIsFirstChild ? Spacing.X : 0.f;

			if (bIsHBox)
			{
				StaticCastSharedPtr<SHorizontalBox>(MyPanelWidget)->AddSlot()
					.MaxWidth(MaxElementSize)
					.HAlign(EntryHorizontalAlignment)
					.VAlign(EntryVerticalAlignment)
					.SizeParam(UWidget::ConvertSerializedSizeParamToRuntime(EntrySizeRule))
					.Padding(Padding)
					[
						ChildWidget.TakeWidget()
					];
			}
			else
			{
				StaticCastSharedPtr<SVerticalBox>(MyPanelWidget)->AddSlot()
					.MaxHeight(MaxElementSize)
					.HAlign(EntryHorizontalAlignment)
					.VAlign(EntryVerticalAlignment)
					.SizeParam(UWidget::ConvertSerializedSizeParamToRuntime(EntrySizeRule))
					.Padding(Padding)
					[
						ChildWidget.TakeWidget()
					];
			}
		}
	}
}

void UDynamicEntryBoxBase::InitEntryBoxType(EDynamicBoxType InEntryBoxType)
{
	ensureMsgf(!MyPanelWidget.IsValid(), TEXT("The widget is already created."));
	EntryBoxType = InEntryBoxType;
}

void UDynamicEntryBoxBase::InitEntrySizeRule(FSlateChildSize InEntrySizeRule)
{
	ensureMsgf(!MyPanelWidget.IsValid(), TEXT("The widget is already created."));
	EntrySizeRule = InEntrySizeRule;
}

void UDynamicEntryBoxBase::InitEntryHorizontalAlignment(EHorizontalAlignment InEntryHorizontalAlignment)
{
	ensureMsgf(!MyPanelWidget.IsValid(), TEXT("The widget is already created."));
	EntryHorizontalAlignment = InEntryHorizontalAlignment;
}

void UDynamicEntryBoxBase::InitEntryVerticalAlignment(EVerticalAlignment InEntryVerticalAlignment)
{
	ensureMsgf(!MyPanelWidget.IsValid(), TEXT("The widget is already created."));
	EntryVerticalAlignment = InEntryVerticalAlignment;
}

void UDynamicEntryBoxBase::InitMaxElementSize(int32 InMaxElementSize)
{
	ensureMsgf(!MyPanelWidget.IsValid(), TEXT("The widget is already created."));
	MaxElementSize = InMaxElementSize;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
