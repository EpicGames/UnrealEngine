// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/EditableText.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Font.h"
#include "Engine/LocalPlayer.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableText.h"
#include "Slate/SlateBrushAsset.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditableText)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UEditableText

UEditableText::UEditableText(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetEditableTextStyle();

	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UFont> DefaultFontObj(*UWidget::GetDefaultFontName());
		FSlateFontInfo Font(DefaultFontObj.Object, 24, FName("Regular"));
		//The FSlateFontInfo just created doesn't contain a composite font (while the default from the WidgetStyle does),
		//so in the case the Font object is replaced by a null one, we have to keep the composite one as a fallback.
		Font.CompositeFont = WidgetStyle.Font.CompositeFont;
		WidgetStyle.SetFont(Font);
	}
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetEditableTextStyle();
	}
#endif // WITH_EDITOR

	IsReadOnly = false;
	IsPassword = false;
	MinimumDesiredWidth = 0.0f;
	IsCaretMovedWhenGainFocus = true;
	SelectAllTextWhenFocused = false;
	RevertTextOnEscape = false;
	ClearKeyboardFocusOnCommit = true;
	SelectAllTextOnCommit = false;
	AllowContextMenu = true;
	EnableIntegratedKeyboard = false;
	VirtualKeyboardTrigger = EVirtualKeyboardTrigger::OnFocusByPointer;
	VirtualKeyboardDismissAction = EVirtualKeyboardDismissAction::TextChangeOnDismiss;
	SetClipping(EWidgetClipping::ClipToBounds);
	OverflowPolicy = ETextOverflowPolicy::Clip;
	FontFacesLoadingPaintPolicy = EFontFacesLoadingPaintPolicy::DoNotPaint;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::Auto;
	bCanChildrenBeAccessible = false;
#endif
}

void UEditableText::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyEditableText.Reset();
}

TSharedRef<SWidget> UEditableText::RebuildWidget()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyEditableText = SNew(SEditableText)
		.Style(&WidgetStyle)
		.IsReadOnly(IsReadOnly)
		.IsPassword(IsPassword)
		.MinDesiredWidth(MinimumDesiredWidth)
		.IsCaretMovedWhenGainFocus(IsCaretMovedWhenGainFocus)
		.SelectAllTextWhenFocused(SelectAllTextWhenFocused)
		.RevertTextOnEscape(RevertTextOnEscape)
		.ClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit)
		.SelectAllTextOnCommit(SelectAllTextOnCommit)
		.OnTextChanged(BIND_UOBJECT_DELEGATE(FOnTextChanged, HandleOnTextChanged))
		.OnTextCommitted(BIND_UOBJECT_DELEGATE(FOnTextCommitted, HandleOnTextCommitted))
		.VirtualKeyboardType(EVirtualKeyboardType::AsKeyboardType(KeyboardType.GetValue()))
		.VirtualKeyboardOptions(VirtualKeyboardOptions)
		.VirtualKeyboardTrigger(VirtualKeyboardTrigger)
		.VirtualKeyboardDismissAction(VirtualKeyboardDismissAction)
		.Justification(Justification)
		.OverflowPolicy(OverflowPolicy)
		.EnableIntegratedKeyboard(EnableIntegratedKeyboard)
		.OnAllFontFacesFinishLoading(OnAllFontFacesFinishLoading.IsBound() ? BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleAllFontFacesFinishLoading) : FSimpleDelegate());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	return MyEditableText.ToSharedRef();
}

void UEditableText::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyEditableText.IsValid())
	{
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	TAttribute<FText> TextBinding = PROPERTY_BINDING(FText, Text);
	TAttribute<FText> HintTextBinding = PROPERTY_BINDING(FText, HintText);

	MyEditableText->SetText(TextBinding);
	MyEditableText->SetTextStyle(WidgetStyle);
	MyEditableText->SetHintText(HintTextBinding);
	MyEditableText->SetIsReadOnly(IsReadOnly);
	MyEditableText->SetIsPassword(IsPassword);
	MyEditableText->SetAllowContextMenu(AllowContextMenu);
	MyEditableText->SetVirtualKeyboardDismissAction(VirtualKeyboardDismissAction);
	MyEditableText->SetJustification(Justification);
	MyEditableText->SetOverflowPolicy(OverflowPolicy);
	MyEditableText->SetEnableIntegratedKeyboard(EnableIntegratedKeyboard);
	MyEditableText->SetFontFacesLoadingPaintPolicy(FontFacesLoadingPaintPolicy);

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// TODO UMG Complete making all properties settable on SEditableText

	ShapedTextOptions.SynchronizeShapedTextProperties(*MyEditableText);
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS

FText UEditableText::GetText() const
{
	if ( MyEditableText.IsValid() )
	{
		return MyEditableText->GetText();
	}

	return Text;
}

void UEditableText::SetText(FText InText)
{
	if (SetTextInternal(InText))
	{
		if (MyEditableText.IsValid())
		{
			MyEditableText->SetText(Text);
		}
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
}

bool UEditableText::SetTextInternal(const FText& InText)
{
	if (!Text.IdenticalTo(InText, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		Text = InText;
		return true;
	}

	return false;
}

void UEditableText::SetIsPassword(bool InbIsPassword)
{
	IsPassword = InbIsPassword;
	if ( MyEditableText.IsValid() )
	{
		MyEditableText->SetIsPassword(IsPassword);
	}
}

FText UEditableText::GetHintText() const
{
	if (MyEditableText.IsValid())
	{
		return MyEditableText->GetHintText();
	}

	return HintText;
}

void UEditableText::SetHintText(FText InHintText)
{
	HintText = InHintText;
	if ( MyEditableText.IsValid() )
	{
		MyEditableText->SetHintText(HintText);
	}
}

float UEditableText::GetMinimumDesiredWidth() const
{
	return MinimumDesiredWidth;
}

void UEditableText::SetMinimumDesiredWidth(float InMinDesiredWidth)
{
	MinimumDesiredWidth = InMinDesiredWidth;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetMinDesiredWidth(MinimumDesiredWidth);
	}
}

void UEditableText::SetIsCaretMovedWhenGainFocus(bool bIsCaretMovedWhenGainFocus)
{
	IsCaretMovedWhenGainFocus = bIsCaretMovedWhenGainFocus;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetIsCaretMovedWhenGainFocus(bIsCaretMovedWhenGainFocus);
	}
}

bool UEditableText::GetIsCaretMovedWhenGainFocus() const
{
	return IsCaretMovedWhenGainFocus;
}

void UEditableText::SetSelectAllTextWhenFocused(bool bSelectAllTextWhenFocused)
{
	SelectAllTextWhenFocused = bSelectAllTextWhenFocused;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetSelectAllTextWhenFocused(bSelectAllTextWhenFocused);
	}
}

bool UEditableText::GetSelectAllTextWhenFocused() const
{
	return SelectAllTextWhenFocused;
}

void UEditableText::SetRevertTextOnEscape(bool bRevertTextOnEscape)
{
	RevertTextOnEscape = bRevertTextOnEscape;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetRevertTextOnEscape(bRevertTextOnEscape);
	}
}

bool UEditableText::GetRevertTextOnEscape() const
{
	return RevertTextOnEscape;
}

bool UEditableText::GetClearKeyboardFocusOnCommit() const
{
	return ClearKeyboardFocusOnCommit;
}

void UEditableText::SetSelectAllTextOnCommit(bool bSelectAllTextOnCommit)
{
	SelectAllTextOnCommit = bSelectAllTextOnCommit;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetSelectAllTextOnCommit(bSelectAllTextOnCommit);
	}
}

bool UEditableText::GetSelectAllTextOnCommit() const
{
	return SelectAllTextOnCommit;
}

void UEditableText::SetWidgetStyle(const FEditableTextStyle& InEditableTextStyle)
{
	WidgetStyle = InEditableTextStyle;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetTextStyle(WidgetStyle);
	}
}

bool UEditableText::GetIsReadOnly() const
{
	return IsReadOnly;
}

void UEditableText::SetIsReadOnly(bool InbIsReadyOnly)
{
	IsReadOnly = InbIsReadyOnly;
	if ( MyEditableText.IsValid() )
	{
		MyEditableText->SetIsReadOnly(IsReadOnly);
	}
}


bool UEditableText::GetIsPassword() const
{
	return IsPassword;
}


ETextJustify::Type UEditableText::GetJustification() const
{
	return Justification;
}

void UEditableText::SetJustification(ETextJustify::Type InJustification)
{
	Justification = InJustification;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetJustification(InJustification);
	}
}

ETextOverflowPolicy UEditableText::GetTextOverflowPolicy() const
{
	return OverflowPolicy;
}

void UEditableText::SetTextOverflowPolicy(ETextOverflowPolicy InOverflowPolicy)
{
	OverflowPolicy = InOverflowPolicy;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetOverflowPolicy(InOverflowPolicy);
	}
}

void UEditableText::SetClearKeyboardFocusOnCommit(bool bClearKeyboardFocusOnCommit)
{
	ClearKeyboardFocusOnCommit = bClearKeyboardFocusOnCommit;
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UEditableText::SetKeyboardType(EVirtualKeyboardType::Type Type)
{
	KeyboardType = Type;
}

const FSlateFontInfo& UEditableText::GetFont() const
{
	return WidgetStyle.Font;
}

void UEditableText::SetFont(FSlateFontInfo InFontInfo)
{
	WidgetStyle.SetFont(InFontInfo);

	if (MyEditableText.IsValid())
	{
		MyEditableText->SetTextStyle(WidgetStyle);
	}
}

void UEditableText::SetFontMaterial(UMaterialInterface* InMaterial)
{
	WidgetStyle.SetFontMaterial(InMaterial);

	if (MyEditableText.IsValid())
	{
		MyEditableText->SetTextStyle(WidgetStyle);
	}
}

void UEditableText::SetFontOutlineMaterial(UMaterialInterface* InMaterial)
{
	WidgetStyle.SetFontOutlineMaterial(InMaterial);

	if (MyEditableText.IsValid())
	{
		MyEditableText->SetTextStyle(WidgetStyle);
	}
}

bool UEditableText::GetEnableIntegratedKeyboard() const
{
	return EnableIntegratedKeyboard;
}

void UEditableText::SetEnableIntegratedKeyboard(bool InEnableIntegratedKeyboard)
{
	EnableIntegratedKeyboard = InEnableIntegratedKeyboard;
	
	if (MyEditableText.IsValid())
	{
		MyEditableText->SetEnableIntegratedKeyboard(EnableIntegratedKeyboard);
	}
}

void UEditableText::HandleOnTextChanged(const FText& InText)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
		OnTextChanged.Broadcast(InText);
	}
}

void UEditableText::HandleOnTextCommitted(const FText& InText, ETextCommit::Type CommitMethod)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
	OnTextCommitted.Broadcast(InText, CommitMethod);
}

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> UEditableText::GetAccessibleWidget() const
{
	return MyEditableText;
}
#endif

#if WITH_EDITOR

const FText UEditableText::GetPaletteCategory()
{
	return LOCTEXT("Input", "Input");
}

#endif

void UEditableText::ToggleVirtualKeyboard(bool bShow)
{
	if (MyEditableText.IsValid())
	{
		if (ULocalPlayer* LocalPlayer = GetOwningLocalPlayer())
		{
			const int32 UserIndex = FSlateApplication::Get().GetUserIndexForController(LocalPlayer->GetControllerId());
			MyEditableText->ToggleVirtualKeyboard(bShow, UserIndex);
		}
	}
}

void UEditableText::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	FontFacesLoadingPaintPolicy = InFontFacesLoadingPaintPolicy;

	if (MyEditableText)
	{
		MyEditableText->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
	}
}


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

