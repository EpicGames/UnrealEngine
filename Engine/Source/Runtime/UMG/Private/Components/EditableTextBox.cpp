// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/EditableTextBox.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Engine/Font.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditableTextBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UEditableTextBox

UEditableTextBox::UEditableTextBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITOR
	bIsFontDeprecationDone = false;
#endif
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	IsReadOnly = false;
	IsPassword = false;
	MinimumDesiredWidth = 0.0f;
	IsCaretMovedWhenGainFocus = true;
	SelectAllTextWhenFocused = false;
	RevertTextOnEscape = false;
	ClearKeyboardFocusOnCommit = true;
	SelectAllTextOnCommit = false;
	AllowContextMenu = true;
	VirtualKeyboardDismissAction = EVirtualKeyboardDismissAction::TextChangeOnDismiss;
	OverflowPolicy = ETextOverflowPolicy::Clip;
	FontFacesLoadingPaintPolicy = EFontFacesLoadingPaintPolicy::DoNotPaint;

	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetEditableTextBoxStyle();

	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UFont> DefaultFontObj(*UWidget::GetDefaultFontName());
		FSlateFontInfo Font(DefaultFontObj.Object, 24, FName("Regular"));
		//The FSlateFontInfo just created doesn't contain a composite font (while the default from the WidgetStyle does),
		//so in the case the Font object is replaced by a null one, we have to keep the composite one as a fallback.
		Font.CompositeFont = WidgetStyle.TextStyle.Font.CompositeFont;

		WidgetStyle.SetFont(Font);
	}
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetEditableTextBoxStyle();
	}
#endif // WITH_EDITOR
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::Auto;
	bCanChildrenBeAccessible = false;
#endif
}

void UEditableTextBox::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading() && !bIsFontDeprecationDone && GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RemoveDuplicatedStyleInfo)
	{
		FTextBlockStyle& TextStyle = WidgetStyle.TextStyle;
		TextStyle.SetFont(WidgetStyle.Font_DEPRECATED);
		bIsFontDeprecationDone = true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

void UEditableTextBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyEditableTextBlock.Reset();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedRef<SWidget> UEditableTextBox::RebuildWidget()
{
	MyEditableTextBlock = SNew(SEditableTextBox)
		.Style(&WidgetStyle)
		.IsReadOnly(IsReadOnly)
		.IsPassword(IsPassword)
		.MinDesiredWidth(MinimumDesiredWidth)
		.IsCaretMovedWhenGainFocus(IsCaretMovedWhenGainFocus)
		.SelectAllTextWhenFocused(SelectAllTextWhenFocused)
		.RevertTextOnEscape(RevertTextOnEscape)
		.ClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit)
		.SelectAllTextOnCommit(SelectAllTextOnCommit)
		.AllowContextMenu(AllowContextMenu)
		.OnTextChanged(BIND_UOBJECT_DELEGATE(FOnTextChanged, HandleOnTextChanged))
		.OnTextCommitted(BIND_UOBJECT_DELEGATE(FOnTextCommitted, HandleOnTextCommitted))
		.OnCursorMovedWithSelection(BIND_UOBJECT_DELEGATE(FOnCursorMovedWithSelection, HandleOnCursorMovedWithSelection))
		.VirtualKeyboardType(EVirtualKeyboardType::AsKeyboardType(KeyboardType.GetValue()))
		.VirtualKeyboardOptions(VirtualKeyboardOptions)
		.VirtualKeyboardTrigger(VirtualKeyboardTrigger)
		.VirtualKeyboardDismissAction(VirtualKeyboardDismissAction)
		.Justification(Justification)
		.OverflowPolicy(OverflowPolicy)
		.OnAllFontFacesFinishLoading(OnAllFontFacesFinishLoading.IsBound() ? BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleAllFontFacesFinishLoading) : FSimpleDelegate());

	return MyEditableTextBlock.ToSharedRef();
}

void UEditableTextBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyEditableTextBlock.IsValid())
	{
		return;
	}

	TAttribute<FText> TextBinding = PROPERTY_BINDING(FText, Text);
	TAttribute<FText> HintTextBinding = PROPERTY_BINDING(FText, HintText);

	MyEditableTextBlock->SetStyle(&WidgetStyle);
	MyEditableTextBlock->SetText(TextBinding);
	MyEditableTextBlock->SetHintText(HintTextBinding);
	MyEditableTextBlock->SetIsReadOnly(IsReadOnly);
	MyEditableTextBlock->SetIsPassword(IsPassword);
	MyEditableTextBlock->SetMinimumDesiredWidth(MinimumDesiredWidth);
	MyEditableTextBlock->SetIsCaretMovedWhenGainFocus(IsCaretMovedWhenGainFocus);
	MyEditableTextBlock->SetSelectAllTextWhenFocused(SelectAllTextWhenFocused);
	MyEditableTextBlock->SetRevertTextOnEscape(RevertTextOnEscape);
	MyEditableTextBlock->SetClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit);
	MyEditableTextBlock->SetSelectAllTextOnCommit(SelectAllTextOnCommit);
	MyEditableTextBlock->SetAllowContextMenu(AllowContextMenu);
	MyEditableTextBlock->SetVirtualKeyboardDismissAction(VirtualKeyboardDismissAction);
	MyEditableTextBlock->SetJustification(Justification);
	MyEditableTextBlock->SetOverflowPolicy(OverflowPolicy);
	MyEditableTextBlock->SetFontFacesLoadingPaintPolicy(FontFacesLoadingPaintPolicy);

	ShapedTextOptions.SynchronizeShapedTextProperties(*MyEditableTextBlock);
}

FText UEditableTextBox::GetText() const
{
	if ( MyEditableTextBlock.IsValid() )
	{
		return MyEditableTextBlock->GetText();
	}

	return Text;
}

void UEditableTextBox::SetText(FText InText)
{
	if (SetTextInternal(InText))
	{
		if (MyEditableTextBlock.IsValid())
		{
			MyEditableTextBlock->SetText(Text);
		}
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
}

bool UEditableTextBox::SetTextInternal(const FText& InText)
{
	if (!Text.IdenticalTo(InText, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		Text = InText;
		return true;
	}

	return false;
}

FText UEditableTextBox::GetHintText() const
{
	if (MyEditableTextBlock.IsValid())
	{
		return MyEditableTextBlock->GetHintText();
	}

	return HintText;
}

void UEditableTextBox::SetHintText(FText InText)
{
	HintText = InText;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetHintText(HintText);
	}
}

float UEditableTextBox::GetMinimumDesiredWidth() const
{
	return MinimumDesiredWidth;
}

void UEditableTextBox::SetMinDesiredWidth(float InMinDesiredWidth)
{
	MinimumDesiredWidth = InMinDesiredWidth;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetMinimumDesiredWidth(MinimumDesiredWidth);
	}
}

void UEditableTextBox::SetIsCaretMovedWhenGainFocus(bool bIsCaretMovedWhenGainFocus)
{
	IsCaretMovedWhenGainFocus = bIsCaretMovedWhenGainFocus;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetIsCaretMovedWhenGainFocus(bIsCaretMovedWhenGainFocus);
	}
}

bool UEditableTextBox::GetIsCaretMovedWhenGainFocus() const
{
	return IsCaretMovedWhenGainFocus;
}

void UEditableTextBox::SetSelectAllTextWhenFocused(bool bSelectAllTextWhenFocused)
{
	SelectAllTextWhenFocused = bSelectAllTextWhenFocused;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetSelectAllTextWhenFocused(bSelectAllTextWhenFocused);
	}
}

bool UEditableTextBox::GetSelectAllTextWhenFocused() const
{
	return SelectAllTextWhenFocused;
}

void UEditableTextBox::SetRevertTextOnEscape(bool bRevertTextOnEscape)
{
	RevertTextOnEscape = bRevertTextOnEscape;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetRevertTextOnEscape(bRevertTextOnEscape);
	}
}

bool UEditableTextBox::GetRevertTextOnEscape() const
{
	return RevertTextOnEscape;
}

void UEditableTextBox::SetClearKeyboardFocusOnCommit(bool bClearKeyboardFocusOnCommit)
{
	ClearKeyboardFocusOnCommit = bClearKeyboardFocusOnCommit;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetClearKeyboardFocusOnCommit(bClearKeyboardFocusOnCommit);
	}
}

bool UEditableTextBox::GetClearKeyboardFocusOnCommit() const
{
	return ClearKeyboardFocusOnCommit;
}

void UEditableTextBox::SetSelectAllTextOnCommit(bool bSelectAllTextOnCommit)
{
	SelectAllTextOnCommit = bSelectAllTextOnCommit;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetSelectAllTextOnCommit(bSelectAllTextOnCommit);
	}

}

bool UEditableTextBox::GetSelectAllTextOnCommit() const
{
	return SelectAllTextOnCommit;
}

bool UEditableTextBox::IsAnyTextSelected() const
{
	if ( MyEditableTextBlock.IsValid() )
	{
		return MyEditableTextBlock->AnyTextSelected();
	}
	
	return false;
}

void UEditableTextBox::SetForegroundColor(FLinearColor color)
{
	WidgetStyle.ForegroundColor = color;
	if ( MyEditableTextBlock.IsValid() )
	{
		MyEditableTextBlock->SetForegroundColor(color);
	}
}

void UEditableTextBox::SetError(FText InError)
{
	if ( MyEditableTextBlock.IsValid() )
	{
		MyEditableTextBlock->SetError(InError);
	}
}

bool UEditableTextBox::GetIsReadOnly() const
{
	return IsReadOnly;
}

void UEditableTextBox::SetIsReadOnly(bool bIsReadOnly)
{
	IsReadOnly = bIsReadOnly;
	if ( MyEditableTextBlock.IsValid() )
	{
		MyEditableTextBlock->SetIsReadOnly(IsReadOnly);
	}
}

bool UEditableTextBox::GetIsPassword() const
{
	return IsPassword;
}

void UEditableTextBox::SetIsPassword(bool bIsPassword)
{
	IsPassword = bIsPassword;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetIsPassword(IsPassword);
	}
}

void UEditableTextBox::ClearError()
{
	if ( MyEditableTextBlock.IsValid() )
	{
		MyEditableTextBlock->SetError(FText::GetEmpty());
	}
}

bool UEditableTextBox::HasError() const
{
	if ( MyEditableTextBlock.IsValid() )
	{
		return MyEditableTextBlock->HasError();
	}

	return false;
}


ETextJustify::Type UEditableTextBox::GetJustification() const
{
	return Justification;
}

void UEditableTextBox::SetJustification(ETextJustify::Type InJustification)
{
	Justification = InJustification;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetJustification(InJustification);
	}
}

ETextOverflowPolicy UEditableTextBox::GetTextOverflowPolicy() const
{
	return OverflowPolicy;
}

void UEditableTextBox::SetTextOverflowPolicy(ETextOverflowPolicy InOverflowPolicy)
{
	OverflowPolicy = InOverflowPolicy;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetOverflowPolicy(InOverflowPolicy);
	}
}

void UEditableTextBox::SetWidgetStyle(const FEditableTextBoxStyle& InStyle)
{
	WidgetStyle = InStyle;
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetStyle(&InStyle);
	}
}

const FEditableTextBoxStyle& UEditableTextBox::GetWidgetStyle() const
{
	return WidgetStyle;
}

void UEditableTextBox::HandleOnTextChanged(const FText& InText)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
		OnTextChanged.Broadcast(InText);
	}
}

void UEditableTextBox::HandleOnCursorMovedWithSelection(const FTextLocation& InLocation, const FTextSelection& Selection)
{
	OnCursorMovedWithSelectionEvent.Broadcast(InLocation, Selection);
}

void UEditableTextBox::HandleOnTextCommitted(const FText& InText, ETextCommit::Type CommitMethod)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
	OnTextCommitted.Broadcast(InText, CommitMethod);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> UEditableTextBox::GetAccessibleWidget() const
{
	return MyEditableTextBlock;
}
#endif

#if WITH_EDITOR

const FText UEditableTextBox::GetPaletteCategory()
{
	return LOCTEXT("Input", "Input");
}
#endif

void UEditableTextBox::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	FontFacesLoadingPaintPolicy = InFontFacesLoadingPaintPolicy;

	if (MyEditableTextBlock)
	{
		MyEditableTextBlock->SetFontFacesLoadingPaintPolicy(InFontFacesLoadingPaintPolicy);
	}
}
/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

