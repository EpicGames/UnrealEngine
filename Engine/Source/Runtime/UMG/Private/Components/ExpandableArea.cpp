// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ExpandableArea.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include "Widgets/Layout/SExpandableArea.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExpandableArea)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UExpandableArea

static const FName HeaderName("Header");
static const FName BodyName("Body");

UExpandableArea::UExpandableArea(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsExpanded(false)
{
	bIsVariable = true;
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Style = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetExpandableAreaStyle();
	BorderBrush = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetExpandableAreaBorderBrush();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		Style = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetExpandableAreaStyle();
		BorderBrush = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetExpandableAreaBorderBrush();
	}
#endif // WITH_EDITOR

	BorderColor = FLinearColor::White;
	AreaPadding = FMargin(1);
	HeaderPadding = FMargin(4.0f, 2.0f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UExpandableArea::GetIsExpanded() const
{
	return bIsExpanded;
}

void UExpandableArea::SetIsExpanded(bool IsExpanded)
{
	if (bIsExpanded != IsExpanded)
	{
		bIsExpanded = IsExpanded;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::bIsExpanded);
	}
	if ( MyExpandableArea.IsValid() )
	{
		MyExpandableArea->SetExpanded(IsExpanded);
	}
}

void UExpandableArea::SetIsExpanded_Animated(bool IsExpanded)
{
	if (bIsExpanded != IsExpanded)
	{
		bIsExpanded = IsExpanded;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::bIsExpanded);
	}
	if (MyExpandableArea.IsValid())
	{
		MyExpandableArea->SetExpanded_Animated(IsExpanded);
	}
}

void UExpandableArea::ReleaseSlateResources( bool bReleaseChildren )
{
	Super::ReleaseSlateResources(bReleaseChildren);
	ReleaseNamedSlotSlateResources(bReleaseChildren);

	MyExpandableArea.Reset();
}

void UExpandableArea::GetSlotNames(TArray<FName>& SlotNames) const
{
	SlotNames.Add(HeaderName);
	SlotNames.Add(BodyName);
}

UWidget* UExpandableArea::GetContentForSlot(FName SlotName) const
{
	if ( SlotName == HeaderName )
	{
		return HeaderContent;
	}
	else if ( SlotName == BodyName )
	{
		return BodyContent;
	}

	return nullptr;
}

void UExpandableArea::SetContentForSlot(FName SlotName, UWidget* Content)
{
	if ( SlotName == HeaderName )
	{
		if ( HeaderContent )
		{
			const bool bReleaseChildren = true;
			HeaderContent->ReleaseSlateResources(bReleaseChildren);
		}

		HeaderContent = Content;
	}
	else if ( SlotName == BodyName )
	{
		if ( BodyContent )
		{
			const bool bReleaseChildren = true;
			BodyContent->ReleaseSlateResources(bReleaseChildren);
		}

		BodyContent = Content;
	}
}

const FExpandableAreaStyle& UExpandableArea::GetStyle() const
{
	return Style;
}

void UExpandableArea::SetStyle(const FExpandableAreaStyle& InStyle)
{
	Style = InStyle;
	if (MyExpandableArea.IsValid())
	{
		MyExpandableArea->SetStyle(&Style);
	}
}

const FSlateBrush& UExpandableArea::GetBorderBrush() const
{
	return BorderBrush;
}

void UExpandableArea::SetBorderBrush(const FSlateBrush& InBorderBrush)
{
	if (BorderBrush != InBorderBrush)
	{
		BorderBrush = InBorderBrush;
		if (MyExpandableArea.IsValid())
		{
			MyExpandableArea->SetBorderBrush(&InBorderBrush);
			MyExpandableArea->InvalidateBorderBrush();
		}
	}
}

const FSlateColor& UExpandableArea::GetBorderColor() const
{
	return BorderColor;
}

void UExpandableArea::SetBorderColor(const FSlateColor& InBorderColor)
{
	BorderColor = InBorderColor;
	if (MyExpandableArea.IsValid())
	{
		MyExpandableArea->SetBorderBackgroundColor(InBorderColor);
	}
}

float UExpandableArea::GetMaxHeight() const
{
	return MaxHeight;
}

void UExpandableArea::SetMaxHeight(float InMaxHeight)
{
	MaxHeight = InMaxHeight;
	if (MyExpandableArea.IsValid())
	{
		MyExpandableArea->SetMaxHeight(MaxHeight);
	}
}

FMargin UExpandableArea::GetHeaderPadding() const
{
	return HeaderPadding;
}

void UExpandableArea::SetHeaderPadding(FMargin InHeaderPadding)
{
	HeaderPadding = InHeaderPadding;
	if (MyExpandableArea.IsValid())
	{
		MyExpandableArea->SetHeaderPadding(HeaderPadding);
	}
}

FMargin UExpandableArea::GetAreaPadding() const
{
	return AreaPadding;
}

void UExpandableArea::SetAreaPadding(FMargin InAreaPadding)
{
	AreaPadding = InAreaPadding;
	if (MyExpandableArea.IsValid())
	{
		MyExpandableArea->SetAreaPadding(AreaPadding);
	}
}

TSharedRef<SWidget> UExpandableArea::RebuildWidget()
{
	TSharedRef<SWidget> HeaderWidget = HeaderContent ? HeaderContent->TakeWidget() : SNullWidget::NullWidget;
	TSharedRef<SWidget> BodyWidget = BodyContent ? BodyContent->TakeWidget() : SNullWidget::NullWidget;

	MyExpandableArea = SNew(SExpandableArea)
		.Style(&Style)
		.BorderImage(&BorderBrush)
		.BorderBackgroundColor(BorderColor)
		.MaxHeight(MaxHeight)
		.Padding(AreaPadding)
		.HeaderPadding(HeaderPadding)
		.OnAreaExpansionChanged(BIND_UOBJECT_DELEGATE(FOnBooleanValueChanged, SlateExpansionChanged))
		.HeaderContent()
		[
			HeaderWidget
		]
		.BodyContent()
		[
			BodyWidget
		];

	return MyExpandableArea.ToSharedRef();
}

void UExpandableArea::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyExpandableArea.IsValid())
	{
		return;
	}
	
	MyExpandableArea->SetStyle(&Style);
	MyExpandableArea->SetExpanded(bIsExpanded);
	MyExpandableArea->SetAreaPadding(AreaPadding);
	MyExpandableArea->SetHeaderPadding(HeaderPadding);
	MyExpandableArea->SetBorderBrush(&BorderBrush);
	MyExpandableArea->SetBorderBackgroundColor(BorderColor);
	MyExpandableArea->SetMaxHeight(MaxHeight);

}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UExpandableArea::SlateExpansionChanged(bool NewState)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bIsExpanded != NewState)
	{
		bIsExpanded = NewState;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::bIsExpanded);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if ( OnExpansionChanged.IsBound() )
	{
		OnExpansionChanged.Broadcast(this, NewState);
	}
}

#if WITH_EDITOR

const FText UExpandableArea::GetPaletteCategory()
{
	return LOCTEXT("Misc", "Misc");
}

void UExpandableArea::OnDescendantSelectedByDesigner(UWidget* DescendantWidget)
{
	// Temporarily sets the active child to the selected child to make
	// dragging and dropping easier in the editor.
	UWidget* SelectedChild = UWidget::FindChildContainingDescendant(BodyContent, DescendantWidget);
	if ( SelectedChild )
	{
		MyExpandableArea->SetExpanded(true);
	}
}

void UExpandableArea::OnDescendantDeselectedByDesigner(UWidget* DescendantWidget)
{
	if ( MyExpandableArea.IsValid() )
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MyExpandableArea->SetExpanded(bIsExpanded);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

