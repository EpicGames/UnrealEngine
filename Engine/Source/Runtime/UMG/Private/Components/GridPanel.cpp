// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/GridPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Components/GridSlot.h"
#include "Editor/WidgetCompilerLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GridPanel)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UGridPanel

UGridPanel::UGridPanel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UGridPanel::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyGridPanel.Reset();
}

UClass* UGridPanel::GetSlotClass() const
{
	return UGridSlot::StaticClass();
}

void UGridPanel::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyGridPanel.IsValid() )
	{
		CastChecked<UGridSlot>(InSlot)->BuildSlot(MyGridPanel.ToSharedRef());
	}
}

void UGridPanel::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyGridPanel.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyGridPanel->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

TSharedRef<SWidget> UGridPanel::RebuildWidget()
{
	MyGridPanel = SNew(SGridPanel);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UGridSlot* TypedSlot = Cast<UGridSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot( MyGridPanel.ToSharedRef() );
		}
	}

	return MyGridPanel.ToSharedRef();
}

UGridSlot* UGridPanel::AddChildToGrid(UWidget* Content, int32 InRow, int32 InColumn)
{
	UGridSlot* GridSlot = Cast<UGridSlot>(Super::AddChild(Content));

	if (GridSlot != nullptr)
	{
		GridSlot->SetRow(InRow);
		GridSlot->SetColumn(InColumn);
	}

	return GridSlot;
}

void UGridPanel::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyGridPanel.IsValid())
	{
		return;
	}

	MyGridPanel->ClearFill();

	for ( int ColumnIndex = 0; ColumnIndex < ColumnFill.Num(); ColumnIndex++ )
	{
		MyGridPanel->SetColumnFill(ColumnIndex, ColumnFill[ColumnIndex]);
	}

	for ( int RowIndex = 0; RowIndex < RowFill.Num(); RowIndex++ )
	{
		MyGridPanel->SetRowFill(RowIndex, RowFill[RowIndex]);
	}
}

void UGridPanel::SetColumnFill(int32 ColumnIndex, float Coefficient)
{
	while (ColumnFill.Num() <= ColumnIndex)
	{
		ColumnFill.Emplace(0.0f);
	}
	ColumnFill[ColumnIndex] = Coefficient;

	if (MyGridPanel.IsValid())
	{
		MyGridPanel->SetColumnFill(ColumnIndex, Coefficient);
	}
}

void UGridPanel::SetRowFill(int32 RowIndex, float Coefficient)
{
	while (RowFill.Num() <= RowIndex)
	{
		RowFill.Emplace(0.0f);
	}
	RowFill[RowIndex] = Coefficient;

	if (MyGridPanel.IsValid())
	{
		MyGridPanel->SetRowFill(RowIndex, Coefficient);
	}
}

void UGridPanel::ClearFill()
{
	ColumnFill.Reset();
	RowFill.Reset();

	if (MyGridPanel.IsValid())
	{
		MyGridPanel->ClearFill();
	}
}

#if WITH_EDITOR

const FText UGridPanel::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

