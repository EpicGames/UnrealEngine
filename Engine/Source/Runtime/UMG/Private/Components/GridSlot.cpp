// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/GridSlot.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GridSlot)

/////////////////////////////////////////////////////
// UGridSlot

UGridSlot::UGridSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;

	Layer = 0;
	Nudge = FVector2D(0, 0);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UGridSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UGridSlot::BuildSlot(TSharedRef<SGridPanel> GridPanel)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GridPanel->AddSlot(Column, Row, SGridPanel::Layer(Layer))
		.Expose(Slot)
		.Padding(Padding)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		.RowSpan(RowSpan)
		.ColumnSpan(ColumnSpan)
		.Nudge(Nudge)
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UGridSlot::GetPadding() const
{
	return Slot ? Slot->GetPadding() : Padding;
}

void UGridSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Slot )
	{
		Slot->SetPadding(InPadding);
	}
}

int32 UGridSlot::GetRow() const
{
	return Slot ? Slot->GetRow() : Row;
}

void UGridSlot::SetRow(int32 InRow)
{
	Row = InRow;
	if ( Slot )
	{
		Slot->SetRow(InRow);
	}
}

int32 UGridSlot::GetRowSpan() const
{
	return Slot ? Slot->GetRowSpan() : RowSpan;
}

void UGridSlot::SetRowSpan(int32 InRowSpan)
{
	RowSpan = InRowSpan;
	if ( Slot )
	{
		Slot->SetRowSpan(InRowSpan);
	}
}

int32 UGridSlot::GetColumn() const
{
	return Slot ? Slot->GetColumn() : Column;
}


void UGridSlot::SetColumn(int32 InColumn)
{
	Column = InColumn;
	if ( Slot )
	{
		Slot->SetColumn(InColumn);
	}
}

int32 UGridSlot::GetColumnSpan() const
{
	return Slot ? Slot->GetColumnSpan() : ColumnSpan;
}

void UGridSlot::SetColumnSpan(int32 InColumnSpan)
{
	ColumnSpan = InColumnSpan;
	if ( Slot )
	{
		Slot->SetColumnSpan(InColumnSpan);
	}
}

int32 UGridSlot::GetLayer() const
{
	return Slot ? Slot->GetLayer() : Layer;
}

void UGridSlot::SetLayer(int32 InLayer)
{
	Layer = InLayer;
	if (Slot)
	{
		Slot->SetLayer(InLayer);
	}
}

FVector2D UGridSlot::GetNudge() const
{
	return Slot ? Slot->GetNudge() : Nudge;
}

void UGridSlot::SetNudge(FVector2D InNudge)
{
	Nudge = InNudge;
	if ( Slot )
	{
		Slot->SetNudge(InNudge);
	}
}

EHorizontalAlignment UGridSlot::GetHorizontalAlignment() const
{
	return Slot ? Slot->GetHorizontalAlignment() : HorizontalAlignment.GetValue();
}

void UGridSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UGridSlot::GetVerticalAlignment() const
{
	return Slot ? Slot->GetVerticalAlignment() : VerticalAlignment.GetValue();
}

void UGridSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Slot )
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UGridSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
	SetPadding(Padding);

	SetRow(Row);
	SetRowSpan(RowSpan);
	SetColumn(Column);
	SetColumnSpan(ColumnSpan);
	SetNudge(Nudge);

	SetLayer(Layer);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

bool UGridSlot::NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize)
{
	const FVector2D ClampedDirection = NudgeDirection.ClampAxes(-1.0f, 1.0f);
	const int32 NewColumn = GetColumn() + FMath::TruncToInt32(ClampedDirection.X);
	const int32 NewRow = GetRow() + FMath::TruncToInt32(ClampedDirection.Y);

	if (NewColumn < 0 || NewRow < 0 || (NewColumn == GetColumn() && NewRow == GetRow()))
	{
		return false;
	}
	
	Modify();

	SetRow(NewRow);
	SetColumn(NewColumn);

	return true;
}

void UGridSlot::SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot)
{
	const ThisClass* const TemplateGridSlot = CastChecked<ThisClass>(TemplateSlot);
	SetRow(TemplateGridSlot->GetRow());
	SetColumn(TemplateGridSlot->GetColumn());
}

#endif

