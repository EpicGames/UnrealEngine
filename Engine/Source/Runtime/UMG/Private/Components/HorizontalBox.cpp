// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/HorizontalBox.h"
#include "Components/HorizontalBoxSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HorizontalBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UHorizontalBox

UHorizontalBox::UHorizontalBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UHorizontalBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyHorizontalBox.Reset();
}

UClass* UHorizontalBox::GetSlotClass() const
{
	return UHorizontalBoxSlot::StaticClass();
}

void UHorizontalBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyHorizontalBox.IsValid() )
	{
		CastChecked<UHorizontalBoxSlot>(InSlot)->BuildSlot(MyHorizontalBox.ToSharedRef());
	}
}

void UHorizontalBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyHorizontalBox.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyHorizontalBox->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

UHorizontalBoxSlot* UHorizontalBox::AddChildToHorizontalBox(UWidget* Content)
{
	return Cast<UHorizontalBoxSlot>( Super::AddChild(Content) );
}

TSharedRef<SWidget> UHorizontalBox::RebuildWidget()
{
	MyHorizontalBox = SNew(SHorizontalBox);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UHorizontalBoxSlot* TypedSlot = Cast<UHorizontalBoxSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyHorizontalBox.ToSharedRef());
		}
	}

	return MyHorizontalBox.ToSharedRef();
}

#if WITH_EDITOR

const FText UHorizontalBox::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

