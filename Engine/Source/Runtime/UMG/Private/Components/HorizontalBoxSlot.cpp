// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/HorizontalBoxSlot.h"
#include "Components/Widget.h"
#include "Components/HorizontalBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HorizontalBoxSlot)

/////////////////////////////////////////////////////
// UHorizontalBoxSlot

UHorizontalBoxSlot::UHorizontalBoxSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
	Size = FSlateChildSize(ESlateSizeRule::Automatic);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UHorizontalBoxSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UHorizontalBoxSlot::BuildSlot(TSharedRef<SHorizontalBox> HorizontalBox)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalBox->AddSlot()
		.Expose(Slot)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		.Padding(Padding)
		.SizeParam(UWidget::ConvertSerializedSizeParamToRuntime(Size))
		[
			Content == NULL ? SNullWidget::NullWidget : Content->TakeWidget()
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UHorizontalBoxSlot::GetPadding() const
{
	return Padding;
}

void UHorizontalBoxSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Slot )
	{
		Slot->SetPadding(InPadding);
	}
}

FSlateChildSize UHorizontalBoxSlot::GetSize() const
{
	return Size;
}

void UHorizontalBoxSlot::SetSize(FSlateChildSize InSize)
{
	Size = InSize;
	if ( Slot )
	{
		Slot->SetSizeParam(UWidget::ConvertSerializedSizeParamToRuntime(InSize));
	}
}

EHorizontalAlignment UHorizontalBoxSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UHorizontalBoxSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UHorizontalBoxSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UHorizontalBoxSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Slot )
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UHorizontalBoxSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetPadding(Padding);
	SetSize(Size);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

bool UHorizontalBoxSlot::NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize)
{
	if (NudgeDirection.X == 0.0f)
	{
		return false;
	}
	
	const FVector2D ClampedDirection = NudgeDirection.ClampAxes(-1, 1);
	UHorizontalBox* ParentHorizontalBox = CastChecked<UHorizontalBox>(Parent);

	const int32 CurrentIndex = ParentHorizontalBox->GetChildIndex(Content);

	if ((CurrentIndex == 0 && ClampedDirection.X < 0.0f) ||
		(CurrentIndex + 1 >= ParentHorizontalBox->GetChildrenCount() && ClampedDirection.X > 0.0f))
	{
		return false;
	}

	ParentHorizontalBox->Modify();
	ParentHorizontalBox->ShiftChild(CurrentIndex + FMath::TruncToInt32(ClampedDirection.X), Content);

	return true;
}

void UHorizontalBoxSlot::SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot)
{
	const ThisClass* const TemplateHorizontalBoxSlot = CastChecked<ThisClass>(TemplateSlot);
	const int32 CurrentIndex = TemplateHorizontalBoxSlot->Parent->GetChildIndex(TemplateHorizontalBoxSlot->Content);

	UHorizontalBox* ParentHorizontalBox = CastChecked<UHorizontalBox>(Parent);
	ParentHorizontalBox->ShiftChild(CurrentIndex, Content);
}

#endif

