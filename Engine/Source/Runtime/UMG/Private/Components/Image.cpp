// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Image.h"
#include "Slate/SlateBrushAsset.h"
#include "Materials/MaterialInterface.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DDynamic.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Engine/AssetManager.h"
#include "Engine/StreamableManager.h"

#if WITH_EDITOR
#include "TextureCompiler.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(Image)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UImage

UImage::UImage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ColorAndOpacity(FLinearColor::White)
{
}

void UImage::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyImage.Reset();
}

TSharedRef<SWidget> UImage::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyImage = SNew(SImage)
			.FlipForRightToLeftFlowDirection(bFlipForRightToLeftFlowDirection);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return MyImage.ToSharedRef();
}

void UImage::SynchronizeProperties()
{
	Super::SynchronizeProperties();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TAttribute<FSlateColor> ColorAndOpacityBinding = PROPERTY_BINDING(FSlateColor, ColorAndOpacity);
	TAttribute<const FSlateBrush*> ImageBinding = OPTIONAL_BINDING_CONVERT(FSlateBrush, Brush, const FSlateBrush*, ConvertImage);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (MyImage.IsValid())
	{
		MyImage->SetImage(ImageBinding);
		MyImage->InvalidateImage();
		MyImage->SetColorAndOpacity(ColorAndOpacityBinding);
		MyImage->SetOnMouseButtonDown(BIND_UOBJECT_DELEGATE(FPointerEventHandler, HandleMouseButtonDown));
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UImage::SetColorAndOpacity(FLinearColor InColorAndOpacity)
{
	ColorAndOpacity = InColorAndOpacity;
	if ( MyImage.IsValid() )
	{
		MyImage->SetColorAndOpacity(ColorAndOpacity);
	}
}


const FLinearColor& UImage::GetColorAndOpacity() const
{
	return ColorAndOpacity;
}

void UImage::SetOpacity(float InOpacity)
{
	ColorAndOpacity.A = InOpacity;
	if ( MyImage.IsValid() )
	{
		MyImage->SetColorAndOpacity(ColorAndOpacity);
	}
}

const FSlateBrush* UImage::ConvertImage(TAttribute<FSlateBrush> InImageAsset) const
{
	UImage* MutableThis = const_cast<UImage*>( this );
	MutableThis->UImage::SetBrush(InImageAsset.Get());

	return &Brush;
}

void UImage::SetBrush(const FSlateBrush& InBrush)
{
	if(Brush != InBrush)
	{
		Brush = InBrush;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Brush);
		if (MyImage.IsValid())
		{
			MyImage->InvalidateImage();
		}
	}
}

const FSlateBrush& UImage::GetBrush() const
{
	return Brush;

}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UImage::SetBrushSize(FVector2D DesiredSize)
{
	SetDesiredSizeOverride(DesiredSize);
}

void UImage::SetDesiredSizeOverride(FVector2D DesiredSize)
{
	if (MyImage.IsValid())
	{
		MyImage->SetDesiredSizeOverride(DesiredSize);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UImage::SetBrushTintColor(FSlateColor TintColor)
{
	if(Brush.TintColor != TintColor)
	{
		Brush.TintColor = TintColor;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Brush);

		if (MyImage.IsValid())
		{
			MyImage->InvalidateImage();
		}
	}
}

void UImage::SetBrushResourceObject(UObject* ResourceObject)
{
	if (Brush.GetResourceObject() != ResourceObject)
	{
		Brush.SetResourceObject(ResourceObject);
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Brush);

		if (MyImage.IsValid())
		{
			MyImage->InvalidateImage();
		}
	}
}

void UImage::SetBrushFromAsset(USlateBrushAsset* Asset)
{
	if(!Asset || Brush != Asset->Brush)
	{
		CancelImageStreaming();
		if (Asset)
		{
			UImage::SetBrush(Asset->Brush);
		}
		else
		{
			UImage::SetBrush(FSlateBrush());
		}
	}
}

void UImage::SetBrushFromTexture(UTexture2D* Texture, bool bMatchSize)
{
	CancelImageStreaming();

	if(Brush.GetResourceObject() != Texture)
	{
		Brush.SetResourceObject(Texture);
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Brush);

		if (Texture) // Since this texture is used as UI, don't allow it affected by budget.
		{
			Texture->bForceMiplevelsToBeResident = true;
			Texture->bIgnoreStreamingMipBias = true;
		}

		if (bMatchSize)
		{
			if (Texture)
			{
#if WITH_EDITOR
				FTextureCompilingManager::Get().FinishCompilation({ Texture });
#endif
				Brush.ImageSize.X = static_cast<float>(Texture->GetSizeX());
				Brush.ImageSize.Y = static_cast<float>(Texture->GetSizeY());
			}
			else
			{
				Brush.ImageSize = FVector2D(0, 0);
			}
		}

		if (MyImage.IsValid())
		{
			MyImage->InvalidateImage();
		}
	}
}

void UImage::SetBrushFromAtlasInterface(TScriptInterface<ISlateTextureAtlasInterface> AtlasRegion, bool bMatchSize)
{
	if(Brush.GetResourceObject() != AtlasRegion.GetObject())
	{
		CancelImageStreaming();
		Brush.SetResourceObject(AtlasRegion.GetObject());
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Brush);

		if (bMatchSize)
		{
			if (AtlasRegion)
			{
				FSlateAtlasData AtlasData = AtlasRegion->GetSlateAtlasData();
				Brush.ImageSize = AtlasData.GetSourceDimensions();
			}
			else
			{
				Brush.ImageSize = FVector2D(0, 0);
			}
		}

		if (MyImage.IsValid())
		{
			MyImage->InvalidateImage();
		}
	}
}

void UImage::SetBrushFromTextureDynamic(UTexture2DDynamic* Texture, bool bMatchSize)
{
	if(Brush.GetResourceObject() != Texture)
	{
		CancelImageStreaming();
		Brush.SetResourceObject(Texture);
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Brush);

		if (bMatchSize && Texture)
		{
			Brush.ImageSize.X = static_cast<float>(Texture->SizeX);
			Brush.ImageSize.Y = static_cast<float>(Texture->SizeY);
		}

		if (MyImage.IsValid())
		{
			MyImage->InvalidateImage();
		}
	}
}

void UImage::SetBrushFromMaterial(UMaterialInterface* Material)
{
	if(Brush.GetResourceObject() != Material)
	{
		CancelImageStreaming();
		Brush.SetResourceObject(Material);
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Brush);

		//TODO UMG Check if the material can be used with the UI

		if (MyImage.IsValid())
		{
			MyImage->InvalidateImage();
		}
	}
}

void UImage::SetFlipForRightToLeftFlowDirection(bool InbFlipForRightToLeftFlowDirection)
{
	if (bFlipForRightToLeftFlowDirection != InbFlipForRightToLeftFlowDirection)
	{
		bFlipForRightToLeftFlowDirection = InbFlipForRightToLeftFlowDirection;

		if (MyImage.IsValid())
		{
			MyImage->FlipForRightToLeftFlowDirection(InbFlipForRightToLeftFlowDirection);
		}
	}
}

bool UImage::ShouldFlipForRightToLeftFlowDirection() const
{
	return bFlipForRightToLeftFlowDirection;

}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UImage::CancelImageStreaming()
{
	if (StreamingHandle.IsValid())
	{
		StreamingHandle->CancelHandle();
		StreamingHandle.Reset();
	}

	StreamingObjectPath.Reset();
}

void UImage::RequestAsyncLoad(TSoftObjectPtr<UObject> SoftObject, TFunction<void()>&& Callback)
{
	RequestAsyncLoad(SoftObject, FStreamableDelegate::CreateLambda(MoveTemp(Callback)));
}

void UImage::RequestAsyncLoad(TSoftObjectPtr<UObject> SoftObject, FStreamableDelegate DelegateToCall)
{
	CancelImageStreaming();

	if (UObject* StrongObject = SoftObject.Get())
	{
		DelegateToCall.ExecuteIfBound();
		return;  // No streaming was needed, complete immediately.
	}

	OnImageStreamingStarted(SoftObject);

	TWeakObjectPtr<UImage> WeakThis(this);
	StreamingObjectPath = SoftObject.ToSoftObjectPath();
	StreamingHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(
		StreamingObjectPath,
		[WeakThis, DelegateToCall, SoftObject]() {
			if (UImage* StrongThis = WeakThis.Get())
			{
				// If the object paths don't match, then this delegate was interrupted, but had already been queued for a callback
				// so ignore everything and abort.
				if (StrongThis->StreamingObjectPath != SoftObject.ToSoftObjectPath())
				{
					return; // Abort!
				}

				// Call the delegate to do whatever is needed, probably set the new image.
				DelegateToCall.ExecuteIfBound();

				// Note that the streaming has completed.
				StrongThis->OnImageStreamingComplete(SoftObject);
			}
		},
		FStreamableManager::AsyncLoadHighPriority);
}

void UImage::OnImageStreamingStarted(TSoftObjectPtr<UObject> SoftObject)
{
	// No-Op
}

void UImage::OnImageStreamingComplete(TSoftObjectPtr<UObject> LoadedSoftObject)
{
	// No-Op
}

void UImage::SetBrushFromSoftTexture(TSoftObjectPtr<UTexture2D> SoftTexture, bool bMatchSize)
{
	TWeakObjectPtr<UImage> WeakThis(this); // using weak ptr in case 'this' has gone out of scope by the time this lambda is called

	RequestAsyncLoad(SoftTexture,
		[WeakThis, SoftTexture, bMatchSize]() {
			if (UImage* StrongThis = WeakThis.Get())
			{
				ensureMsgf(SoftTexture.Get(), TEXT("Failed to load %s"), *SoftTexture.ToSoftObjectPath().ToString());
				StrongThis->SetBrushFromTexture(SoftTexture.Get(), bMatchSize);
			}
		}
	);
}

void UImage::SetBrushFromSoftMaterial(TSoftObjectPtr<UMaterialInterface> SoftMaterial)
{
	TWeakObjectPtr<UImage> WeakThis(this); // using weak ptr in case 'this' has gone out of scope by the time this lambda is called

	RequestAsyncLoad(SoftMaterial,
		[WeakThis, SoftMaterial]() {
			if (UImage* StrongThis = WeakThis.Get())
			{
				ensureMsgf(SoftMaterial.Get(), TEXT("Failed to load %s"), *SoftMaterial.ToSoftObjectPath().ToString());
				StrongThis->SetBrushFromMaterial(SoftMaterial.Get());
			}
		}
	);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UMaterialInstanceDynamic* UImage::GetDynamicMaterial()
{
	UMaterialInterface* Material = NULL;

	UObject* Resource = Brush.GetResourceObject();
	Material = Cast<UMaterialInterface>(Resource);

	if ( Material )
	{
		UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);

		if ( !DynamicMaterial )
		{
			DynamicMaterial = UMaterialInstanceDynamic::Create(Material, this);
			SetBrushResourceObject(DynamicMaterial);

			if ( MyImage.IsValid() )
			{
				MyImage->InvalidateImage();
			}
		}
		return DynamicMaterial;
	}

	//TODO UMG can we do something for textures?  General purpose dynamic material for them?

	return NULL;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FReply UImage::HandleMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if ( OnMouseButtonDownEvent.IsBound() )
	{
		return OnMouseButtonDownEvent.Execute(Geometry, MouseEvent).NativeReply;
	}

	return FReply::Unhandled();
}

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> UImage::GetAccessibleWidget() const
{
	return MyImage;
}
#endif

#if WITH_EDITOR

const FText UImage::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

#endif


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

