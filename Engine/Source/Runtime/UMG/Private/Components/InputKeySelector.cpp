// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/InputKeySelector.h"
#include "Engine/Font.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SInputKeySelector.h"
#include "Internationalization/Internationalization.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputKeySelector)

#define LOCTEXT_NAMESPACE "UMG"

UInputKeySelector::UInputKeySelector( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetButtonStyle();
	TextStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetTextBlockStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetButtonStyle();
		TextStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTextBlockStyle();
	}
#endif // WITH_EDITOR

	KeySelectionText = NSLOCTEXT("InputKeySelector", "DefaultKeySelectionText", "...");
	NoKeySpecifiedText = NSLOCTEXT("InputKeySelector", "DefaultEmptyText", "Empty");
	SelectedKey = FInputChord(EKeys::Invalid);
	bAllowModifierKeys = true;
	bAllowGamepadKeys = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	EscapeKeys.AddUnique(EKeys::Gamepad_Special_Right); // In most (if not all) cases this is going to be the menu button

	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UFont> RobotoFontObj(*UWidget::GetDefaultFontName());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TextStyle.Font = FSlateFontInfo(RobotoFontObj.Object, 24, FName("Bold"));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UInputKeySelector::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UInputKeySelector::SetSelectedKey( const FInputChord& InSelectedKey )
{
	if (SelectedKey != InSelectedKey)
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::SelectedKey);
		if (MyInputKeySelector.IsValid())
		{
			MyInputKeySelector->SetSelectedKey(InSelectedKey);
		}
		SelectedKey = InSelectedKey;
	}
}

FInputChord UInputKeySelector::GetSelectedKey() const
{
	return SelectedKey;
}

void UInputKeySelector::SetKeySelectionText( FText InKeySelectionText )
{
	if ( MyInputKeySelector.IsValid() )
	{
		MyInputKeySelector->SetKeySelectionText( InKeySelectionText );
	}
	KeySelectionText = MoveTemp(InKeySelectionText);
}

const FText& UInputKeySelector::GetNoKeySpecifiedText() const
{
	return NoKeySpecifiedText;
}

const FText& UInputKeySelector::GetKeySelectionText() const
{
	return KeySelectionText;
}

void UInputKeySelector::SetNoKeySpecifiedText(FText InNoKeySpecifiedText)
{
	if (MyInputKeySelector.IsValid())
	{
		MyInputKeySelector->SetNoKeySpecifiedText(InNoKeySpecifiedText);
	}
	NoKeySpecifiedText = MoveTemp(InNoKeySpecifiedText);
}

void UInputKeySelector::SetAllowModifierKeys( const bool bInAllowModifierKeys )
{
	if ( MyInputKeySelector.IsValid() )
	{
		MyInputKeySelector->SetAllowModifierKeys( bInAllowModifierKeys );
	}
	bAllowModifierKeys = bInAllowModifierKeys;
}

bool UInputKeySelector::AllowModifierKeys() const
{
	return bAllowModifierKeys;
}

void UInputKeySelector::SetAllowGamepadKeys(const bool bInAllowGamepadKeys)
{
	if (MyInputKeySelector.IsValid())
	{
		MyInputKeySelector->SetAllowGamepadKeys(bInAllowGamepadKeys);
	}
	bAllowGamepadKeys = bInAllowGamepadKeys;
}

bool UInputKeySelector::AllowGamepadKeys() const
{
	return bAllowGamepadKeys;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UInputKeySelector::GetIsSelectingKey() const
{
	return MyInputKeySelector.IsValid() ? MyInputKeySelector->GetIsSelectingKey() : false;
}

void UInputKeySelector::SetButtonStyle( const FButtonStyle* InButtonStyle )
{
	SetButtonStyle(*InButtonStyle);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UInputKeySelector::SetButtonStyle(const FButtonStyle& InButtonStyle)
{
	if (MyInputKeySelector.IsValid())
	{
		MyInputKeySelector->SetButtonStyle(&InButtonStyle);
	}
	WidgetStyle = InButtonStyle;
}

const FButtonStyle& UInputKeySelector::GetButtonStyle() const
{
	return WidgetStyle;
}

void UInputKeySelector::SetTextStyle(const FTextBlockStyle& InTextStyle)
{
	if (MyInputKeySelector.IsValid())
	{
		MyInputKeySelector->SetTextStyle(&InTextStyle);
	}
	TextStyle = InTextStyle;
}

const FTextBlockStyle& UInputKeySelector::GetTextStyle() const
{
	return TextStyle;
}

void UInputKeySelector::SetEscapeKeys(const TArray<FKey>& InKeys)
{
	if (MyInputKeySelector.IsValid())
	{
		MyInputKeySelector->SetEscapeKeys(InKeys);
	}
	EscapeKeys = InKeys;
}

void UInputKeySelector::SetMargin(const FMargin& InMargin)
{
	if (MyInputKeySelector.IsValid())
	{
		MyInputKeySelector->SetMargin(InMargin);
	}
	Margin = InMargin;
}

const FMargin& UInputKeySelector::GetMargin() const
{
	return Margin;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
const FText UInputKeySelector::GetPaletteCategory()
{
	return LOCTEXT("Advanced", "Advanced");
}
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UInputKeySelector::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyInputKeySelector.IsValid())
	{
		return;
	}

	MyInputKeySelector->SetSelectedKey( SelectedKey );
	MyInputKeySelector->SetMargin( Margin );
	MyInputKeySelector->SetButtonStyle( &WidgetStyle );
	MyInputKeySelector->SetTextStyle( &TextStyle );
	MyInputKeySelector->SetKeySelectionText( KeySelectionText );
	MyInputKeySelector->SetNoKeySpecifiedText(NoKeySpecifiedText);
	MyInputKeySelector->SetAllowModifierKeys( bAllowModifierKeys );
	MyInputKeySelector->SetAllowGamepadKeys(bAllowGamepadKeys);
	MyInputKeySelector->SetEscapeKeys(EscapeKeys);
}

void UInputKeySelector::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyInputKeySelector.Reset();
}

TSharedRef<SWidget> UInputKeySelector::RebuildWidget()
{
	MyInputKeySelector = SNew(SInputKeySelector)
		.SelectedKey(SelectedKey)
		.Margin(Margin)
		.ButtonStyle(&WidgetStyle)
		.TextStyle(&TextStyle)
		.KeySelectionText(KeySelectionText)
		.NoKeySpecifiedText(NoKeySpecifiedText)
		.AllowModifierKeys(bAllowModifierKeys)
		.AllowGamepadKeys(bAllowGamepadKeys)
		.EscapeKeys(EscapeKeys)
		.OnKeySelected( BIND_UOBJECT_DELEGATE( SInputKeySelector::FOnKeySelected, HandleKeySelected ) )
		.OnIsSelectingKeyChanged( BIND_UOBJECT_DELEGATE( SInputKeySelector::FOnIsSelectingKeyChanged, HandleIsSelectingKeyChanged ) );
	return MyInputKeySelector.ToSharedRef();
}

void UInputKeySelector::HandleKeySelected(const FInputChord& InSelectedKey)
{
	SelectedKey = InSelectedKey;
	BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::SelectedKey);
	OnKeySelected.Broadcast(SelectedKey);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UInputKeySelector::HandleIsSelectingKeyChanged()
{
	OnIsSelectingKeyChanged.Broadcast();
}

void UInputKeySelector::SetTextBlockVisibility(const ESlateVisibility InVisibility)
{
	if (MyInputKeySelector.IsValid())
	{
		EVisibility SlateVisibility = UWidget::ConvertSerializedVisibilityToRuntime(InVisibility);
		MyInputKeySelector->SetTextBlockVisibility(SlateVisibility);
	}
}

#undef LOCTEXT_NAMESPACE
