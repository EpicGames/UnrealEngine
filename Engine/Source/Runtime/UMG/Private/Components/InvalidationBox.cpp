// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/InvalidationBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SInvalidationPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InvalidationBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UInvalidationBox

UInvalidationBox::UInvalidationBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bCanCache = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UInvalidationBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyInvalidationPanel.Reset();
}

TSharedRef<SWidget> UInvalidationBox::RebuildWidget()
{
	MyInvalidationPanel =
		SNew(SInvalidationPanel)
#if !UE_BUILD_SHIPPING
		.DebugName(GetPathName())
#endif
		;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyInvalidationPanel->SetCanCache(IsDesignTime() ? false : bCanCache);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if ( GetChildrenCount() > 0 )
	{
		MyInvalidationPanel->SetContent(GetContentSlot()->Content ? GetContentSlot()->Content->TakeWidget() : SNullWidget::NullWidget);
	}
	
	return MyInvalidationPanel.ToSharedRef();
}

void UInvalidationBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if ( MyInvalidationPanel.IsValid() )
	{
		MyInvalidationPanel->SetContent(InSlot->Content ? InSlot->Content->TakeWidget() : SNullWidget::NullWidget);
	}
}

void UInvalidationBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyInvalidationPanel.IsValid() )
	{
		MyInvalidationPanel->SetContent(SNullWidget::NullWidget);
	}
}

void UInvalidationBox::InvalidateCache()
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UInvalidationBox::GetCanCache() const
{
	if ( MyInvalidationPanel.IsValid() )
	{
		return MyInvalidationPanel->GetCanCache();
	}

	return bCanCache;
}

void UInvalidationBox::SetCanCache(bool CanCache)
{
	bCanCache = CanCache;
	if ( MyInvalidationPanel.IsValid() )
	{
		return MyInvalidationPanel->SetCanCache(bCanCache);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UInvalidationBox::GetPaletteCategory()
{
	return LOCTEXT("Optimization", "Optimization");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

