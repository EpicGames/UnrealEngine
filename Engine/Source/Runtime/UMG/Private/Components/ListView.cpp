// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ListView.h"
#include "Widgets/Views/SListView.h"
#include "Slate/SObjectTableRow.h"
#include "Engine/World.h"
#include "TimerManager.h"
#include "Blueprint/ListViewDesignerPreviewItem.h"
#include "Slate/UMGDragDropOp.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"
#include "UMGPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ListView)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UListView

UListView::UListView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Orientation(EOrientation::Orient_Vertical)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetListViewStyle();
	ScrollBarStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBarStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetListViewStyle();
		ScrollBarStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle();
	}
#endif // WITH_EDITOR
}

void UListView::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyListView.Reset();
}

#if WITH_EDITOR
void UListView::OnRefreshDesignerItems()
{
	RefreshDesignerItems<TObjectPtr<UObject>>(ListItems, [this] () {return NewObject<UListViewDesignerPreviewItem>(this); });
}
#endif

void UListView::AddItem(UObject* Item)
{
	if (Item == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot add null item into ListView."), ELogVerbosity::Warning, "NullListViewItem");
		return;
	}

	if (ListItems.Contains(Item))
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot add duplicate item into ListView."), ELogVerbosity::Warning, "DuplicateListViewItem");
		return;
	}

	ListItems.Add(Item);

	OnItemsChanged({ Item }, {});

	RequestRefresh();
}

void UListView::AddItemAt(UObject* Item, int32 Index)
{
	if (Item == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot add null item into ListView."), ELogVerbosity::Warning, "NullListViewItem");
		return;
	}

	if (ListItems.Contains(Item))
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot add duplicate item into ListView."), ELogVerbosity::Warning, "DuplicateListViewItem");
		return;
	}

	// Safety clamp the provided index to a valid range
	Index = FMath::Clamp(Index, 0, ListItems.Num());
	ListItems.Insert(Item, Index);

	OnItemsChanged({ Item }, {});

	RequestRefresh();
}

void UListView::AddItems(const TArray<UObject*>& Items)
{
	ListItems.Reserve(ListItems.Num() + Items.Num());

	for (UObject* Item : Items)
	{
		if (Item != nullptr)
		{
			ListItems.Add(Item);
		}
	}

	OnItemsChanged(Items, {});

	RequestRefresh();
}

void UListView::AddItemsAt(const TArray<UObject*>& Items, int32 Index)
{
	ListItems.Reserve(ListItems.Num() + Items.Num());

	// Safety clamp the provided index to a valid range
	Index = FMath::Clamp(Index, 0, ListItems.Num());

	int32 Offset = 0;
	for (UObject* Item : Items)
	{
		if (Item != nullptr)
		{
			ListItems.Insert(Item, Index + Offset);
			++Offset;
		}
	}

	OnItemsChanged(Items, {});

	RequestRefresh();
}

void UListView::RemoveItem(UObject* Item)
{
	if (Item == nullptr)
	{
		return;
	}

	ListItems.Remove(Item);

	OnItemsChanged({}, { Item });

	RequestRefresh();
}

void UListView::RemoveItems(const TArray<UObject*>& Items)
{
	for (UObject* Item : Items)
	{
		if (Item)
		{
			ListItems.Remove(Item);
		}
	}

	OnItemsChanged({}, Items);

	RequestRefresh();
}

UObject* UListView::GetItemAt(int32 Index) const
{
	return ListItems.IsValidIndex(Index) ? ListItems[Index] : nullptr;
}

int32 UListView::GetNumItems() const
{
	return ListItems.Num();
}

int32 UListView::GetIndexForItem(const UObject* Item) const
{
	return ListItems.IndexOfByKey(Item);
}

void UListView::ClearListItems()
{
	const TArray<UObject*> Added;
	const TArray<UObject*> Removed = MoveTemp(ListItems);

	ListItems.Reset();

	OnItemsChanged(Added, Removed);

	RequestRefresh();
}

void UListView::SetSelectionMode(TEnumAsByte<ESelectionMode::Type> InSelectionMode)
{
	SelectionMode = InSelectionMode;
	if (MyListView)
	{
		MyListView->SetSelectionMode(InSelectionMode);
	}
}

void UListView::SetScrollIntoViewAlignment (EScrollIntoViewAlignment NewScrollIntoViewAlignment)
{
	ScrollIntoViewAlignment = NewScrollIntoViewAlignment;
	if (MyListView)
	{
		MyListView->SetScrollIntoViewAlignment(NewScrollIntoViewAlignment);
	}
}

void UListView::SetEnableProximateEntryNavigation(bool bNewValue)
{
	bEnableProximateEntryNavigation = bNewValue;
	if (MyListView.IsValid())
	{
		MyListView->SetEnableProximateEntryNavigation(bNewValue);
	}
}

void UListView::SetScrollBarPadding(const FMargin& InScrollBarPadding)
{
	ScrollBarPadding = InScrollBarPadding;
	if (MyListView.IsValid())
	{
		MyListView->SetScrollbarPadding(ScrollBarPadding);
	}
}

int32 UListView::BP_GetNumItemsSelected() const
{
	return GetNumItemsSelected();
}

void UListView::BP_SetListItems(const TArray<UObject*>& InListItems)
{
	SetListItems(InListItems);
}

UObject* UListView::BP_GetSelectedItem() const
{
	return GetSelectedItem();
}

void UListView::HandleOnEntryInitializedInternal(UObject* Item, const TSharedRef<ITableRow>& TableRow)
{
	BP_OnEntryInitialized.Broadcast(Item, GetEntryWidgetFromItem(Item));
}

bool UListView::BP_GetSelectedItems(TArray<UObject*>& Items) const
{
	return GetSelectedItems(Items) > 0;
}

bool UListView::BP_IsItemVisible(UObject* Item) const
{
	return IsItemVisible(Item);
}

void UListView::BP_NavigateToItem(UObject* Item)
{
	if (Item)
	{
		RequestNavigateToItem(Item);
	}
}

void UListView::NavigateToIndex(int32 Index)
{
	RequestNavigateToItem(GetItemAt(Index));
}

void UListView::BP_ScrollItemIntoView(UObject* Item)
{
	if (Item)
	{
		RequestScrollItemIntoView(Item);
	}
}

void UListView::ScrollIndexIntoView(int32 Index)
{
	BP_ScrollItemIntoView(GetItemAt(Index));
}

void UListView::BP_CancelScrollIntoView()
{
	if (MyListView.IsValid())
	{
		MyListView->CancelScrollIntoView();
	}
}

bool UListView::IsRefreshPending() const
{
	if (MyListView.IsValid())
	{
		return MyListView->IsPendingRefresh();
	}
	return false;
}

void UListView::BP_SetSelectedItem(UObject* Item)
{
	if (MyListView.IsValid())
	{
		MyListView->SetSelection(Item, ESelectInfo::Direct);
	}
}

void UListView::SetSelectedItem(const UObject* Item)
{
	ITypedUMGListView<UObject*>::SetSelectedItem(const_cast<UObject*>(Item));
}

void UListView::SetSelectedIndex(int32 Index)
{
	SetSelectedItem(GetItemAt(Index));
}

void UListView::BP_SetItemSelection(UObject* Item, bool bSelected)
{
	SetItemSelection(Item, bSelected);
}

void UListView::BP_ClearSelection()
{
	ClearSelection();
}

void UListView::OnItemsChanged(const TArray<UObject*>& AddedItems, const TArray<UObject*>& RemovedItems)
{
	// Allow subclasses to do special things when objects are added or removed from the list.

	// Keep track of references to Actors and make sure to release them when Actors are about to be removed
	for (UObject* AddedItem : AddedItems)
	{
		if (AActor* AddedActor = Cast<AActor>(AddedItem))
		{
			AddedActor->OnEndPlay.AddDynamic(this, &UListView::OnListItemEndPlayed);
		}
		else if (AActor* AddedItemOuterActor = AddedItem ? AddedItem->GetTypedOuter<AActor>() : nullptr)
		{
			// Unique so that we don't spam events for shared actor outers but this also means we can't
			// unsubscribe when processing RemovedItems
			AddedItemOuterActor->OnEndPlay.AddUniqueDynamic(this, &UListView::OnListItemOuterEndPlayed);
		}
	}
	for (UObject* RemovedItem : RemovedItems)
	{
		if (AActor* RemovedActor = Cast<AActor>(RemovedItem))
		{
			RemovedActor->OnEndPlay.RemoveDynamic(this, &UListView::OnListItemEndPlayed);
		}
	}
}

void UListView::OnListItemEndPlayed(AActor* Item, EEndPlayReason::Type EndPlayReason)
{
	RemoveItem(Item);
}

void UListView::OnListItemOuterEndPlayed(AActor* ItemOuter, EEndPlayReason::Type EndPlayReason)
{
	for (int32 ItemIndex = ListItems.Num() - 1; ItemIndex >= 0; --ItemIndex)
	{
		UObject* Item = ListItems[ItemIndex];
		if (Item->IsIn(ItemOuter))
		{
			RemoveItem(Item);
		}
	}
}

TSharedRef<STableViewBase> UListView::RebuildListWidget()
{
	return ConstructListView<SListView>();
}

void UListView::HandleListEntryHovered(UUserWidget& EntryWidget)
{
	if (const TObjectPtrWrapTypeOf<UObject*>* ListItem = ItemFromEntryWidget(EntryWidget))
	{
		OnItemIsHoveredChanged().Broadcast(*ListItem, true);
		BP_OnItemIsHoveredChanged.Broadcast(*ListItem, true);
	}
}

void UListView::HandleListEntryUnhovered(UUserWidget& EntryWidget)
{
	if (const TObjectPtrWrapTypeOf<UObject*>* ListItem = ItemFromEntryWidget(EntryWidget))
	{
		OnItemIsHoveredChanged().Broadcast(*ListItem, false);
 		BP_OnItemIsHoveredChanged.Broadcast(*ListItem, false);
	}
}

UDragDropOperation* UListView::HandleListEntryDragDetected(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent, UUserWidget& EntryWidget)
{
	if (bAllowDragging)
	{
		if (const TObjectPtrWrapTypeOf<UObject*>* ListItem = ItemFromEntryWidget(EntryWidget))
		{
			OnItemDragDetected().Broadcast(*ListItem);
			BP_OnItemDragDetected.Broadcast(*ListItem, MyGeometry);

			bIsDragging = true;
			BP_OnListViewDraggingStateChanged.Broadcast(bIsDragging);
			return CreateDragDropOperation(*ListItem);
		}
	}

	return nullptr;
}

TOptional<EItemDropZone> UListView::HandleListEntryCanAcceptDrop(const FDragDropEvent& DropEvent, EItemDropZone DropZone, UUserWidget& EntryWidget)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DropEvent.GetOperationAs<FUMGDragDropOp>();
	if (NativeOp.IsValid())
	{
		if (OnItemCanAcceptDrop().IsBound())
		{
			if (const TObjectPtrWrapTypeOf<UObject*>* ListItem = ItemFromEntryWidget(EntryWidget))
			{
				OnItemCanAcceptDrop().Broadcast(*ListItem, true);
			}
		}
		return DropZone;
	}
	return NullOpt;
}

FReply UListView::HandleListEntryAcceptDrop(const FDragDropEvent& DropEvent, EItemDropZone DropZone, UUserWidget& EntryWidget)
{
	bIsDragging = false;
	BP_OnListViewDraggingStateChanged.Broadcast(bIsDragging);

	TSharedPtr<FUMGDragDropOp> NativeOp = DropEvent.GetOperationAs<FUMGDragDropOp>();
	const TObjectPtrWrapTypeOf<UObject*>* ListItem = ItemFromEntryWidget(EntryWidget);
	if (NativeOp.IsValid())
	{
		if (ListItem)
		{
			BP_OnItemAcceptDrop.Broadcast(*ListItem, UDragDropOperation::ConvertSlateDropZoneToUMG(DropZone), NativeOp.Get()->GetOperation());
			OnItemAcceptDrop().Broadcast(*ListItem);
		}
		return FReply::Handled().EndDragDrop();
	}
	return FReply::Unhandled();
}

void UListView::HandleListEntryDragEnter(const FDragDropEvent& DragDropEvent, UUserWidget& EntryWidget)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if (NativeOp.IsValid())
	{
		if (const TObjectPtrWrapTypeOf<UObject*>* ListItem = ItemFromEntryWidget(EntryWidget))
		{
			BP_OnItemDragEnter.Broadcast(*ListItem, NativeOp->GetOperation());
			OnItemDragEnter().Broadcast(*ListItem);
		}
	}
}

void UListView::HandleListEntryDragLeave(const FDragDropEvent& DragDropEvent, UUserWidget& EntryWidget)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if (NativeOp.IsValid())
	{
		if (const TObjectPtrWrapTypeOf<UObject*>* ListItem = ItemFromEntryWidget(EntryWidget))
		{
			BP_OnItemDragLeave.Broadcast(*ListItem, NativeOp->GetOperation());
			OnItemDragLeave().Broadcast(*ListItem);
		}
	}
}

void UListView::HandleListEntryDragCancelled(const FDragDropEvent& DragDropEvent)
{
	bIsDragging = false;
	BP_OnListViewDraggingStateChanged.Broadcast(bIsDragging);

	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if (NativeOp.IsValid())
	{
		OnItemDragCancelled().Broadcast(DragDropEvent);
		BP_OnItemDragCancelled.Broadcast(NativeOp->GetOperation());
	}
}

FMargin UListView::GetDesiredEntryPadding(UObject* Item) const
{
	if (ListItems.Num() > 0 && ListItems[0] != Item)
	{
		if (Orientation == EOrientation::Orient_Horizontal)
		{
			// For all entries after the first one, add the spacing as left padding
			return FMargin(HorizontalEntrySpacing, 0.f, 0.0f, 0.f);
		}
		else
		{
			// For all entries after the first one, add the spacing as top padding
			return FMargin(0.f, VerticalEntrySpacing, 0.f, 0.f);
		}
	}

	return FMargin(0.f);
}

UUserWidget& UListView::OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable)
{
	return GenerateTypedEntry(DesiredEntryClass, OwnerTable);
}

UObject* UListView::GetListObjectFromEntry(UUserWidget& EntryWidget)
{
	const TObjectPtrWrapTypeOf<UObject*>* Item = ITypedUMGListView::ItemFromEntryWidget(EntryWidget);
	return Item ? *Item : nullptr;
}

void UListView::OnItemClickedInternal(UObject* ListItem)
{
	ITypedUMGListView::OnItemClickedInternal(ListItem);
	BP_OnItemClicked.Broadcast(ListItem);
}

void UListView::OnItemDoubleClickedInternal(UObject* ListItem)
{
	ITypedUMGListView::OnItemDoubleClickedInternal(ListItem);
	BP_OnItemDoubleClicked.Broadcast(ListItem);
}

void UListView::OnSelectionChangedInternal(UObject* FirstSelectedItem)
{
	ITypedUMGListView::OnSelectionChangedInternal(FirstSelectedItem);
	BP_OnItemSelectionChanged.Broadcast(FirstSelectedItem, FirstSelectedItem != nullptr);
}

void UListView::OnItemScrolledIntoViewInternal(UObject* ListItem, UUserWidget& EntryWidget)
{
	ITypedUMGListView::OnItemScrolledIntoViewInternal(ListItem, EntryWidget);
	BP_OnItemScrolledIntoView.Broadcast(ListItem, &EntryWidget);
}

void UListView::OnListViewScrolledInternal(float ItemOffset, float DistanceRemaining)
{
	ITypedUMGListView::OnListViewScrolledInternal(ItemOffset, DistanceRemaining);
	BP_OnListViewScrolled.Broadcast(ItemOffset, DistanceRemaining);
}

void UListView::OnListViewFinishedScrollingInternal()
{
	ITypedUMGListView::OnListViewFinishedScrollingInternal();
	BP_OnListViewFinishedScrolling.Broadcast();
}

void UListView::OnListViewTouchStartInternal()
{
	ITypedUMGListView::OnListViewTouchStartInternal();
	BP_OnListViewTouchStart.Broadcast();
}

void UListView::OnListViewTouchMoveInternal()
{
	ITypedUMGListView::OnListViewTouchMoveInternal();
	BP_OnListViewTouchMove.Broadcast();
}

void UListView::OnListViewTouchEndInternal()
{
	ITypedUMGListView::OnListViewTouchEndInternal();
	BP_OnListViewTouchEnd.Broadcast();
}

bool UListView::OnIsSelectableOrNavigableInternal(UObject* FirstSelectedItem)
{
	return BP_OnIsItemSelectableOrNavigable.IsBound() ? BP_OnIsItemSelectableOrNavigable.Execute(FirstSelectedItem) : ITypedUMGListView::OnIsSelectableOrNavigableInternal(FirstSelectedItem);
}

void UListView::SetReturnFocusToSelection(bool bEnabled)
{
	bReturnFocusToSelection = bEnabled;
	if (MyListView.IsValid())
	{
		MyListView->SetReturnFocusToSelection(bEnabled);
	}
}

void UListView::SetVerticalEntrySpacing(float InVerticalEntrySpacing)
{
	if (VerticalEntrySpacing != InVerticalEntrySpacing)
	{
		VerticalEntrySpacing = InVerticalEntrySpacing;
		RequestRefresh();
	}
}

void UListView::SetHorizontalEntrySpacing(float InHorizontalEntrySpacing)
{
	if (HorizontalEntrySpacing != InHorizontalEntrySpacing)
	{
		HorizontalEntrySpacing = InHorizontalEntrySpacing;
		RequestRefresh();
	}
}

void UListView::SetShadowBrushStyle(const FScrollBoxStyle& NewScrollBoxStyle)
{
	ShadowBrushStyle = NewScrollBoxStyle;
}

const FScrollBoxStyle& UListView::GetShadowBrushStyle() const
{
	return ShadowBrushStyle;
}

void UListView::InitHorizontalEntrySpacing(float InHorizontalEntrySpacing)
{
	SetHorizontalEntrySpacing(InHorizontalEntrySpacing);
}

void UListView::InitVerticalEntrySpacing(float InVerticalEntrySpacing)
{
	SetVerticalEntrySpacing(InVerticalEntrySpacing);
}

void UListView::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

		if (EntrySpacing != 0.f)
		{
			HorizontalEntrySpacing = EntrySpacing;
			VerticalEntrySpacing = EntrySpacing;
			EntrySpacing = 0.f;
		}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITOR
}
/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
