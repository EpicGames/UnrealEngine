// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ListViewBase.h"
#include "UMGPrivate.h"
#include "Widgets/Text/STextBlock.h"
#include "Engine/Blueprint.h"
#include "Editor/WidgetCompilerLog.h"
#include "Blueprint/WidgetBlueprintLibrary.h"
#include "Blueprint/DragDropOperation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ListViewBase)

#define LOCTEXT_NAMESPACE "UMG"

UListViewBase::UListViewBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EntryWidgetPool(*this)
{
	bIsVariable = true;
	SetClipping(EWidgetClipping::ClipToBounds);
}

#if WITH_EDITOR
const FText UListViewBase::GetPaletteCategory()
{
	return LOCTEXT("Lists", "Lists");
}

void UListViewBase::ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const
{
	if (!EntryWidgetClass)
	{
		CompileLog.Error(FText::Format(LOCTEXT("Error_ListViewBase_MissingEntryClass", "{0} has no EntryWidgetClass specified - required for any UListViewBase to function."), FText::FromString(GetName())));
	}
	else if (!EntryWidgetClass->ImplementsInterface(UUserListEntry::StaticClass()))
	{
		CompileLog.Error(FText::Format(LOCTEXT("Error_ListViewBase_EntryClassNotImplementingInterface", "'{0}' has EntryWidgetClass property set to'{1}' and that Class doesn't implement User List Entry Interface - required for any UListViewBase to function."), FText::FromString(GetName()), FText::FromString(EntryWidgetClass->GetName())));
	}
}
#endif

void UListViewBase::RegenerateAllEntries()
{
	EntryWidgetPool.ReleaseAll();
	GeneratedEntriesToAnnounce.Reset();

	if (MyTableViewBase.IsValid())
	{
		MyTableViewBase->RebuildList();
	}

	CancelListViewDragDrop();
}

void UListViewBase::ScrollToTop()
{
	if (MyTableViewBase.IsValid())
	{
		MyTableViewBase->ScrollToTop();
	}
}

void UListViewBase::ScrollToBottom()
{
	if (MyTableViewBase.IsValid())
	{
		MyTableViewBase->ScrollToBottom();
	}
}

void UListViewBase::SetScrollOffset(const float InScrollOffset)
{
	if (MyTableViewBase.IsValid())
	{
		MyTableViewBase->SetScrollOffset(InScrollOffset);
	}
}

void UListViewBase::EndInertialScrolling() 
{
	if (MyTableViewBase.IsValid())
	{
		MyTableViewBase->EndInertialScrolling();
	}
}

void UListViewBase::SetWheelScrollMultiplier(float NewWheelScrollMultiplier)
{
	WheelScrollMultiplier = NewWheelScrollMultiplier;
	if (MyTableViewBase)
	{
		MyTableViewBase->SetWheelScrollMultiplier(GetGlobalScrollAmount() * NewWheelScrollMultiplier);
	}
}

void UListViewBase::SetScrollbarVisibility(ESlateVisibility InVisibility)
{
	if (MyTableViewBase)
	{
		MyTableViewBase->SetScrollbarVisibility(UWidget::ConvertSerializedVisibilityToRuntime(InVisibility));
	}
}

ESlateVisibility UListViewBase::GetScrollbarVisibility() const
{
	if (MyTableViewBase)
	{
		return UWidget::ConvertRuntimeToSerializedVisibility(MyTableViewBase->GetScrollbarVisibility());
	}
	// Returning collapsed if TableView is not available, as the scrollbar does not occupy space in the ui layout  
	return ESlateVisibility::Collapsed; 
}

void  UListViewBase::SetAllowOverScroll(bool NewAllowOverscroll)
{
	if (MyTableViewBase)
	{
		AllowOverscroll = NewAllowOverscroll;
		MyTableViewBase->SetAllowOverscroll(AllowOverscroll ? EAllowOverscroll::Yes : EAllowOverscroll::No);
	}
}

UMG_API void UListViewBase::SetIsPointerScrollingEnabled(bool bInIsPointerScrollingEnabled)
{
	bIsPointerScrollingEnabled = bInIsPointerScrollingEnabled;
	if (MyTableViewBase)
	{
		MyTableViewBase->SetIsPointerScrollingEnabled(bInIsPointerScrollingEnabled);
	}
}

UMG_API void UListViewBase::SetIsTouchScrollingEnabled(bool bInIsTouchScrollingEnabled)
{
	bEnableTouchScrolling = bInIsTouchScrollingEnabled;
	if (MyTableViewBase)
	{
		MyTableViewBase->SetIsTouchScrollingEnabled(bInIsTouchScrollingEnabled);
	}
}

UMG_API void UListViewBase::SetIsGamepadScrollingEnabled(bool bInIsGamepadScrollingEnabled)
{
	bIsGamepadScrollingEnabled = bInIsGamepadScrollingEnabled;
	if (MyTableViewBase)
	{
		MyTableViewBase->SetIsGamepadScrollingEnabled(bInIsGamepadScrollingEnabled);
	}
}

UMG_API void UListViewBase::CancelListViewDragDrop()
{
	if (bIsDragging)
	{
		bIsDragging = false;
		if (FSlateApplication::Get().IsDragDropping())
		{
			FSlateApplication::Get().CancelDragDrop();
		}
	}
}

UDragDropOperation* UListViewBase::CreateDragDropOperation(UObject* Item)
{
	if (DragDropOperationClass)
	{
		UDragDropOperation* Operation = UWidgetBlueprintLibrary::CreateDragDropOperation(DragDropOperationClass);
		Operation->Pivot = DragDropVisualPivot;
		Operation->Offset = DragDropVisualOffset;
		Operation->Payload = Item;

		if (DragVisualWidget)
		{
			Operation->DefaultDragVisual = DragVisualWidget;
		}
		else
		{
			TSubclassOf<UUserWidget> DragVisualClass = (DragDropVisualEntryClass) ? DragDropVisualEntryClass : EntryWidgetClass;
			Operation->DefaultDragVisual = CreateWidget<UUserWidget>(GetWorld(), DragVisualClass);
		}
		return Operation;
	}

	return nullptr;
}

const TArray<UUserWidget*>& UListViewBase::GetDisplayedEntryWidgets() const
{ 
	return EntryWidgetPool.GetActiveWidgets(); 
}

float UListViewBase::GetScrollOffset() const
{
	if (MyTableViewBase.IsValid())
	{
		return MyTableViewBase->GetScrollOffset();
	}

	return 0.0f;
}

TSharedRef<SWidget> UListViewBase::RebuildWidget()
{
	FText ErrorText;
	if (!EntryWidgetClass)
	{
		ErrorText = LOCTEXT("Error_MissingEntryWidgetClass", "No EntryWidgetClass specified on this list.\nEven if doing custom stuff, this is always required as a fallback.");
	}
#if WITH_EDITOR
	// if the BP was cooked already, then the ClassGeneratedBy will be null, so nothing to check
	else if (!EntryWidgetClass->bCooked)
	{
		UBlueprint* EntryWidgetBP = Cast<UBlueprint>(EntryWidgetClass->ClassGeneratedBy);
		if (!EntryWidgetBP)
		{
			ErrorText = FText::Format(LOCTEXT("Error_NonBPEntryWidget", "EntryWidgetClass [{0}] is not a Blueprint class"), FText::FromString(EntryWidgetClass->GetName()));
		}
		else if (EntryWidgetBP->Status == BS_Error)
		{
			ErrorText = FText::Format(LOCTEXT("Error_CompilationError", "EntryWidget BP [{0}] has not compiled successfully"), FText::FromString(EntryWidgetBP->GetName()));
		}
	}
#endif

	if (!ErrorText.IsEmpty())
	{
		return SNew(STextBlock)
			.Text(ErrorText);
	}

	MyTableViewBase = RebuildListWidget();
	MyTableViewBase->SetIsScrollAnimationEnabled(bEnableScrollAnimation);
	MyTableViewBase->SetScrollingAnimationInterpolationSpeed(ScrollingAnimationInterpolationSpeed);
	MyTableViewBase->SetEnableTouchAnimatedScrolling(bInEnableTouchAnimatedScrolling);
	MyTableViewBase->SetIsRightClickScrollingEnabled(bEnableRightClickScrolling);
	MyTableViewBase->SetIsTouchScrollingEnabled(bEnableTouchScrolling);
	MyTableViewBase->SetIsPointerScrollingEnabled(bIsPointerScrollingEnabled);
	MyTableViewBase->SetFixedLineScrollOffset(bEnableFixedLineOffset ? TOptional<double>(FixedLineScrollOffset) : TOptional<double>());
	MyTableViewBase->SetWheelScrollMultiplier(GetGlobalScrollAmount() * WheelScrollMultiplier);

	return MyTableViewBase.ToSharedRef();
}

void UListViewBase::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyTableViewBase.Reset();
	EntryWidgetPool.ResetPool();
	GeneratedEntriesToAnnounce.Reset();
}

void UListViewBase::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyTableViewBase)
	{
		MyTableViewBase->SetIsScrollAnimationEnabled(bEnableScrollAnimation);
		MyTableViewBase->SetScrollingAnimationInterpolationSpeed(ScrollingAnimationInterpolationSpeed);
		MyTableViewBase->SetEnableTouchAnimatedScrolling(bInEnableTouchAnimatedScrolling);
		MyTableViewBase->SetIsRightClickScrollingEnabled(bEnableRightClickScrolling);
		MyTableViewBase->SetIsTouchScrollingEnabled(bEnableTouchScrolling);
		MyTableViewBase->SetIsPointerScrollingEnabled(bIsPointerScrollingEnabled);
		MyTableViewBase->SetAllowOverscroll(AllowOverscroll ? EAllowOverscroll::Yes : EAllowOverscroll::No);
		MyTableViewBase->SetFixedLineScrollOffset(bEnableFixedLineOffset ? TOptional<double>(FixedLineScrollOffset) : TOptional<double>());
		MyTableViewBase->SetWheelScrollMultiplier(GetGlobalScrollAmount() * WheelScrollMultiplier);
		MyTableViewBase->SetSelectItemOnNavigation(bSelectItemOnNavigation);
	}

#if WITH_EDITORONLY_DATA
	if (IsDesignTime() && MyTableViewBase.IsValid())
	{
		bNeedsToCallRefreshDesignerItems = true;
		OnRefreshDesignerItems();

		if (!ensureMsgf(!bNeedsToCallRefreshDesignerItems, TEXT("[%s] does not call RefreshDesignerItems<T> from within OnRefreshDesignerItems. Please do so to support design-time previewing of list entries."), *GetClass()->GetName()))
		{
			bNeedsToCallRefreshDesignerItems = false;
		}
	}
#endif
}

TSharedRef<STableViewBase> UListViewBase::RebuildListWidget()
{
	ensureMsgf(false, TEXT("All children of UListViewBase must implement RebuildListWidget using one of the static ITypedUMGListView<T>::ConstructX functions"));
	return SNew(SListView<TSharedPtr<FString>>);
}

void UListViewBase::RequestRefresh()
{
	if (MyTableViewBase.IsValid())
	{
		MyTableViewBase->RequestListRefresh();
	}

	CancelListViewDragDrop();
}

void UListViewBase::HandleRowReleased(const TSharedRef<ITableRow>& Row)
{
	UUserWidget* EntryWidget = StaticCastSharedRef<IObjectTableRow>(Row)->GetUserWidget();
	if (ensure(EntryWidget))
	{
		EntryWidgetPool.Release(EntryWidget);

		if (!IsDesignTime())
		{
			GeneratedEntriesToAnnounce.Remove(EntryWidget);
			OnEntryWidgetReleased().Broadcast(*EntryWidget);
			NativeOnEntryReleased(EntryWidget);
			BP_OnEntryReleased.Broadcast(EntryWidget);
		}
	}
}

void UListViewBase::FinishGeneratingEntry(UUserWidget& GeneratedEntry)
{
	if (!IsDesignTime())
	{
		// Announcing the row generation now is just a bit too soon, as we haven't finished generating the row as far as the underlying list is concerned. 
		// So we cache the row/item pair here and announce their generation on the next tick
		GeneratedEntriesToAnnounce.AddUnique(&GeneratedEntry);
		if (!EntryGenAnnouncementDelegateHandle.IsValid())
		{
			if (UWorld* World = GetWorld())
			{
				EntryGenAnnouncementDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [this](float InDelta)
				{
					HandleAnnounceGeneratedEntries();
					return false;
				}));
			}
		}
	}
}

void UListViewBase::HandleAnnounceGeneratedEntries()
{
	EntryGenAnnouncementDelegateHandle.Reset();
	for (TWeakObjectPtr<UUserWidget> EntryWidget : GeneratedEntriesToAnnounce)
	{
		if (EntryWidget.IsValid())
		{
			OnEntryWidgetGenerated().Broadcast(*EntryWidget);
			NativeOnEntryGenerated(EntryWidget.Get());
			BP_OnEntryGenerated.Broadcast(EntryWidget.Get());
		}
	}
	GeneratedEntriesToAnnounce.Empty();

	NativeOnEntriesGenerated();
	BP_OnEntriesGenerated.Broadcast(GetDisplayedEntryWidgets().Num());
}

#undef LOCTEXT_NAMESPACE

