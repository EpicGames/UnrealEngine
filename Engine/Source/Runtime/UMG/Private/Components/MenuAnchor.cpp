// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MenuAnchor.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Blueprint/UserWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MenuAnchor)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UMenuAnchor

UMenuAnchor::UMenuAnchor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ShouldDeferPaintingAfterWindowContent(true)
	, UseApplicationMenuStack(true)
	, ShowMenuBackground(true)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Placement = MenuPlacement_ComboBox;
	bFitInWindow = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UMenuAnchor::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyMenuAnchor.Reset();
}

TSharedRef<SWidget> UMenuAnchor::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyMenuAnchor = SNew(SMenuAnchor)
		.Placement(Placement)
		.FitInWindow(bFitInWindow)
		.OnGetMenuContent(BIND_UOBJECT_DELEGATE(FOnGetContent, HandleGetMenuContent))
		.OnMenuOpenChanged(BIND_UOBJECT_DELEGATE(FOnIsOpenChanged, HandleMenuOpenChanged))
		.ShouldDeferPaintingAfterWindowContent(ShouldDeferPaintingAfterWindowContent)
		.UseApplicationMenuStack(UseApplicationMenuStack)
		.ShowMenuBackground(ShowMenuBackground);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if ( GetChildrenCount() > 0 )
	{
		MyMenuAnchor->SetContent(GetContentSlot()->Content ? GetContentSlot()->Content->TakeWidget() : SNullWidget::NullWidget);
	}
	
	return MyMenuAnchor.ToSharedRef();
}

void UMenuAnchor::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if ( MyMenuAnchor.IsValid() )
	{
		MyMenuAnchor->SetContent(InSlot->Content ? InSlot->Content->TakeWidget() : SNullWidget::NullWidget);
	}
}

void UMenuAnchor::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyMenuAnchor.IsValid() )
	{
		MyMenuAnchor->SetContent(SNullWidget::NullWidget);
	}
}

void UMenuAnchor::HandleMenuOpenChanged(bool bIsOpen)
{
	OnMenuOpenChanged.Broadcast(bIsOpen);
}

TSharedRef<SWidget> UMenuAnchor::HandleGetMenuContent()
{
	TSharedPtr<SWidget> SlateMenuWidget;
	
	if ( OnGetUserMenuContentEvent.IsBound() )
	{
		UWidget* MenuWidget = OnGetUserMenuContentEvent.Execute();
		if ( MenuWidget )
		{
			SlateMenuWidget = MenuWidget->TakeWidget();
		}
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if ( OnGetMenuContentEvent.IsBound() )
	{
		// Remove when OnGetMenuContentEvent is fully deprecated.
		UWidget* MenuWidget = OnGetMenuContentEvent.Execute();
		if ( MenuWidget )
		{
			SlateMenuWidget = MenuWidget->TakeWidget();
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	else
	{
		if ( MenuClass != nullptr && !MenuClass->HasAnyClassFlags(CLASS_Abstract) )
		{
			if ( UWorld* World = GetWorld() )
			{
				if ( UUserWidget* MenuWidget = CreateWidget(World, MenuClass) )
				{
					SlateMenuWidget = MenuWidget->TakeWidget();
				}
			}
		}
	}

	return SlateMenuWidget.IsValid() ? SlateMenuWidget.ToSharedRef() : SNullWidget::NullWidget;
}

void UMenuAnchor::ToggleOpen(bool bFocusOnOpen)
{
	if ( MyMenuAnchor.IsValid() )
	{
		MyMenuAnchor->SetIsOpen(!MyMenuAnchor->IsOpen(), bFocusOnOpen);
	}
}

void UMenuAnchor::Open(bool bFocusMenu)
{
	if ( MyMenuAnchor.IsValid() && !MyMenuAnchor->IsOpen() )
	{
		const int32 Index = GetOwningLocalPlayer() ? GetOwningLocalPlayer()->GetLocalPlayerIndex() : 0;
		MyMenuAnchor->SetIsOpen(true, bFocusMenu, Index);
	}
}

void UMenuAnchor::Close()
{
	if ( MyMenuAnchor.IsValid() )
	{
		const int32 Index = GetOwningLocalPlayer() ? GetOwningLocalPlayer()->GetLocalPlayerIndex() : 0;
		return MyMenuAnchor->SetIsOpen(false, false, Index);
	}
}

bool UMenuAnchor::IsOpen() const
{
	if ( MyMenuAnchor.IsValid() )
	{
		return MyMenuAnchor->IsOpen();
	}

	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void UMenuAnchor::SetPlacement(EMenuPlacement InPlacement)
{
	Placement = InPlacement;
	if (MyMenuAnchor.IsValid())
	{
		return MyMenuAnchor->SetMenuPlacement(Placement);
	}
}

EMenuPlacement UMenuAnchor::GetPlacement() const
{
	return Placement;
}

void UMenuAnchor::FitInWindow(bool bFit)
{
	bFitInWindow = bFit;
	if (MyMenuAnchor.IsValid())
	{
		return MyMenuAnchor->SetFitInWindow(bFitInWindow);
	}
}

bool UMenuAnchor::IsFitInWindow() const
{
	return bFitInWindow;
}

bool UMenuAnchor::IsDeferPaintingAfterWindowContent() const
{
	return ShouldDeferPaintingAfterWindowContent;
}

bool UMenuAnchor::IsUseApplicationMenuStack() const
{
	return UseApplicationMenuStack;
}

bool UMenuAnchor::IsShowMenuBackground() const
{
	return ShowMenuBackground;
}

void UMenuAnchor::InitShouldDeferPaintingAfterWindowContent(bool InShouldDeferPaintingAfterWindowContent)
{
	ensureMsgf(!MyMenuAnchor.IsValid(), TEXT("The widget is already created."));
	ShouldDeferPaintingAfterWindowContent = InShouldDeferPaintingAfterWindowContent;
}

void UMenuAnchor::InitUseApplicationMenuStack(bool InUseApplicationMenuStack)
{
	ensureMsgf(!MyMenuAnchor.IsValid(), TEXT("The widget is already created."));
	UseApplicationMenuStack = InUseApplicationMenuStack;
}

void UMenuAnchor::InitShowMenuBackground(bool InShowMenuBackground)
{
	ensureMsgf(!MyMenuAnchor.IsValid(), TEXT("The widget is already created."));
	ShowMenuBackground = InShowMenuBackground;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UMenuAnchor::ShouldOpenDueToClick() const
{
	if ( MyMenuAnchor.IsValid() )
	{
		return MyMenuAnchor->ShouldOpenDueToClick();
	}

	return false;
}

FVector2D UMenuAnchor::GetMenuPosition() const
{
	if ( MyMenuAnchor.IsValid() )
	{
		return MyMenuAnchor->GetMenuPosition();
	}

	return FVector2D(0, 0);
}

bool UMenuAnchor::HasOpenSubMenus() const
{
	if ( MyMenuAnchor.IsValid() )
	{
		return MyMenuAnchor->HasOpenSubMenus();
	}

	return false;
}

#if WITH_EDITOR

const FText UMenuAnchor::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

