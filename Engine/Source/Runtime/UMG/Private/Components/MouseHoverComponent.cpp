// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MouseHoverComponent.h"

#include "Blueprint/UserWidget.h"
#include "Components/Widget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MouseHoverComponent)

class SMouseHoverWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMouseHoverWidget)
		: _Content()
		{
		}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UMouseHoverComponent* InMouseEventsComponent);

protected:

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	TWeakObjectPtr<UMouseHoverComponent> MouseHoverComponent;
};

void SMouseHoverWidget::Construct(const FArguments& InArgs, UMouseHoverComponent* InMouseHoverComponent)
{
	MouseHoverComponent = InMouseHoverComponent;
	ChildSlot
	[
		InArgs._Content.Widget
	];
}

void SMouseHoverWidget::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseHoverComponent.IsValid())
	{
		const bool bWasHovered = IsHovered();

		// Call super class's implementation
		SWidget::OnMouseEnter(MyGeometry, MouseEvent);

		if (!bWasHovered && IsHovered())
		{
			MouseHoverComponent->OnMouseHoverChanged(true);
		}
	}
}

void SMouseHoverWidget::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (MouseHoverComponent.IsValid())
	{
		const bool bWasHovered = IsHovered();

		// Call super class's implementation
		SWidget::OnMouseLeave(MouseEvent);

		if (bWasHovered && !IsHovered())
		{
			MouseHoverComponent->OnMouseHoverChanged(false);
		}
	}
}

void UMouseHoverComponent::OnMouseHoverChanged(bool InbIsHovered)
{
	if (GetIsHovered() != InbIsHovered)
	{
		bIsHovered = InbIsHovered;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::bIsHovered);
	}
}

TSharedRef<SWidget> UMouseHoverComponent::RebuildWidgetWithContent(TSharedRef<SWidget> OwnerContent)
{
	TSharedRef<SWidget> WrapperWidget = 
		SNew(SMouseHoverWidget, this)
		[
			OwnerContent
		];

	return WrapperWidget;
}

