// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MultiLineEditableText.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Font.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiLineEditableText)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UMultiLineEditableText

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UMultiLineEditableText::UMultiLineEditableText(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetTextBlockStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTextBlockStyle();
	}
#endif // WITH_EDITOR

	bIsReadOnly = false;
	SelectAllTextWhenFocused = false;
	ClearTextSelectionOnFocusLoss = true;
	RevertTextOnEscape = false;
	ClearKeyboardFocusOnCommit = true;
	AllowContextMenu = true;
	SetClipping(EWidgetClipping::ClipToBounds);
	VirtualKeyboardDismissAction = EVirtualKeyboardDismissAction::TextChangeOnDismiss;
	AutoWrapText = true;

	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UFont> DefaultFontObj(*UWidget::GetDefaultFontName());
		FSlateFontInfo Font(DefaultFontObj.Object, 12, FName("Bold"));
		//The FSlateFontInfo just created doesn't contain a composite font (while the default from the WidgetStyle does),
		//so in the case the Font object is replaced by a null one, we have to keep the composite one as a fallback.
		Font.CompositeFont = WidgetStyle.Font.CompositeFont;
		WidgetStyle.SetFont(Font);
	}
}

void UMultiLineEditableText::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyMultiLineEditableText.Reset();
}

TSharedRef<SWidget> UMultiLineEditableText::RebuildWidget()
{
	MyMultiLineEditableText = SNew(SMultiLineEditableText)
	.TextStyle(&WidgetStyle)
	.AllowContextMenu(AllowContextMenu)
	.IsReadOnly(bIsReadOnly)
//	.MinDesiredWidth(MinimumDesiredWidth)
//	.IsCaretMovedWhenGainFocus(IsCaretMovedWhenGainFocus)
	.SelectAllTextWhenFocused(SelectAllTextWhenFocused)
	.ClearTextSelectionOnFocusLoss(ClearTextSelectionOnFocusLoss)
	.RevertTextOnEscape(RevertTextOnEscape)
	.ClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit)
//	.SelectAllTextOnCommit(SelectAllTextOnCommit)
//	.BackgroundImageSelected(BackgroundImageSelected ? TAttribute<const FSlateBrush*>(&BackgroundImageSelected->Brush) : TAttribute<const FSlateBrush*>())
//	.BackgroundImageSelectionTarget(BackgroundImageSelectionTarget ? TAttribute<const FSlateBrush*>(&BackgroundImageSelectionTarget->Brush) : TAttribute<const FSlateBrush*>())
//	.BackgroundImageComposing(BackgroundImageComposing ? TAttribute<const FSlateBrush*>(&BackgroundImageComposing->Brush) : TAttribute<const FSlateBrush*>())
//	.CaretImage(CaretImage ? TAttribute<const FSlateBrush*>(&CaretImage->Brush) : TAttribute<const FSlateBrush*>())
	.VirtualKeyboardOptions(VirtualKeyboardOptions)
	.VirtualKeyboardDismissAction(VirtualKeyboardDismissAction)
	.OnTextChanged(BIND_UOBJECT_DELEGATE(FOnTextChanged, HandleOnTextChanged))
	.OnTextCommitted(BIND_UOBJECT_DELEGATE(FOnTextCommitted, HandleOnTextCommitted))
	.OnAllFontFacesFinishLoading(OnAllFontFacesFinishLoading.IsBound() ? BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleAllFontFacesFinishLoading) : FSimpleDelegate())
	;
	
	return MyMultiLineEditableText.ToSharedRef();
}

void UMultiLineEditableText::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyMultiLineEditableText.IsValid())
	{
		return;
	}

	TAttribute<FText> HintTextBinding = PROPERTY_BINDING(FText, HintText);

	MyMultiLineEditableText->SetTextStyle(&WidgetStyle);
	MyMultiLineEditableText->SetText(Text);
	MyMultiLineEditableText->SetHintText(HintTextBinding);
	MyMultiLineEditableText->SetAllowContextMenu(AllowContextMenu);
	MyMultiLineEditableText->SetIsReadOnly(bIsReadOnly);
	MyMultiLineEditableText->SetVirtualKeyboardDismissAction(VirtualKeyboardDismissAction);
	MyMultiLineEditableText->SetSelectAllTextWhenFocused(SelectAllTextWhenFocused);
	MyMultiLineEditableText->SetClearTextSelectionOnFocusLoss(ClearTextSelectionOnFocusLoss);
	MyMultiLineEditableText->SetRevertTextOnEscape(RevertTextOnEscape);
	MyMultiLineEditableText->SetClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit);

	// TODO UMG Complete making all properties settable on SMultiLineEditableText

	Super::SynchronizeTextLayoutProperties(*MyMultiLineEditableText);
}


void UMultiLineEditableText::OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions)
{
	Super::OnShapedTextOptionsChanged(InShapedTextOptions);
	if (MyMultiLineEditableText.IsValid())
	{
		InShapedTextOptions.SynchronizeShapedTextProperties(*MyMultiLineEditableText);
	}
}

void UMultiLineEditableText::OnJustificationChanged(ETextJustify::Type InJustification)
{
	Super::OnJustificationChanged(InJustification);
	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetJustification(InJustification);
	}
}

void UMultiLineEditableText::OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy)
{
	Super::OnWrappingPolicyChanged(InWrappingPolicy);
	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetWrappingPolicy(InWrappingPolicy);
	}
}

void UMultiLineEditableText::OnAutoWrapTextChanged(bool InAutoWrapText)
{
	Super::OnAutoWrapTextChanged(InAutoWrapText);
	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetAutoWrapText(InAutoWrapText);
	}
}

void UMultiLineEditableText::OnWrapTextAtChanged(float InWrapTextAt)
{
	Super::OnWrapTextAtChanged(InWrapTextAt);
	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetWrapTextAt(InWrapTextAt);
	}
}

void UMultiLineEditableText::OnLineHeightPercentageChanged(float InLineHeightPercentage)
{
	Super::OnLineHeightPercentageChanged(InLineHeightPercentage);
	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetLineHeightPercentage(InLineHeightPercentage);
	}
}

void UMultiLineEditableText::OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine)
{
	Super::OnApplyLineHeightToBottomLineChanged(InApplyLineHeightToBottomLine);
	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetApplyLineHeightToBottomLine(InApplyLineHeightToBottomLine);
	}
}

void UMultiLineEditableText::OnMarginChanged(const FMargin& InMargin)
{
	Super::OnMarginChanged(InMargin);
	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetMargin(InMargin);
	}
}

FText UMultiLineEditableText::GetText() const
{
	if ( MyMultiLineEditableText.IsValid() )
	{
		return MyMultiLineEditableText->GetText();
	}

	return Text;
}

void UMultiLineEditableText::SetText(FText InText)
{
	if (SetTextInternal(InText))
	{
		if (MyMultiLineEditableText.IsValid())
		{
			MyMultiLineEditableText->SetText(Text);
		}
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
}

bool UMultiLineEditableText::SetTextInternal(const FText& InText)
{
	if (!Text.IdenticalTo(InText, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		Text = InText;
		return true;
	}

	return false;
}

FText UMultiLineEditableText::GetHintText() const
{
	if (MyMultiLineEditableText.IsValid())
	{
		return MyMultiLineEditableText->GetHintText();
	}

	return HintText;
}

void UMultiLineEditableText::SetHintText(FText InHintText)
{
	HintText = InHintText;
	HintTextDelegate.Clear();
	if (MyMultiLineEditableText.IsValid())
	{
		TAttribute<FText> HintTextBinding = PROPERTY_BINDING(FText, HintText);
		MyMultiLineEditableText->SetHintText(HintTextBinding);
	}
}

void UMultiLineEditableText::SetSelectAllTextWhenFocused(bool bSelectAllTextWhenFocused)
{
	SelectAllTextWhenFocused = bSelectAllTextWhenFocused;

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetSelectAllTextWhenFocused(bSelectAllTextWhenFocused);
	}
}

bool UMultiLineEditableText::GetSelectAllTextWhenFocused() const
{
	return SelectAllTextWhenFocused;
}

void UMultiLineEditableText::SetClearTextSelectionOnFocusLoss(bool bClearTextSelectionOnFocusLoss)
{
	ClearTextSelectionOnFocusLoss = bClearTextSelectionOnFocusLoss;

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetClearTextSelectionOnFocusLoss(bClearTextSelectionOnFocusLoss);
	}
}

bool UMultiLineEditableText::GetClearTextSelectionOnFocusLoss() const
{
	return ClearTextSelectionOnFocusLoss;
}

void UMultiLineEditableText::SetRevertTextOnEscape(bool bRevertTextOnEscape)
{
	RevertTextOnEscape = bRevertTextOnEscape;

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetRevertTextOnEscape(bRevertTextOnEscape);
	}
}

bool UMultiLineEditableText::GetRevertTextOnEscape() const
{
	return RevertTextOnEscape;
}

void UMultiLineEditableText::SetClearKeyboardFocusOnCommit(bool bClearKeyboardFocusOnCommit)
{
	ClearKeyboardFocusOnCommit = bClearKeyboardFocusOnCommit;

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetClearKeyboardFocusOnCommit(bClearKeyboardFocusOnCommit);
	}
}

bool UMultiLineEditableText::GetClearKeyboardFocusOnCommit() const
{
	return ClearKeyboardFocusOnCommit;
}

bool UMultiLineEditableText::GetIsReadOnly() const
{
	return bIsReadOnly;
}

void UMultiLineEditableText::SetIsReadOnly(bool bReadOnly)
{
	bIsReadOnly = bReadOnly;

	if ( MyMultiLineEditableText.IsValid() )
	{
		MyMultiLineEditableText->SetIsReadOnly(bIsReadOnly);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UMultiLineEditableText::SetWidgetStyle(const FTextBlockStyle& InWidgetStyle)
{
	WidgetStyle = InWidgetStyle;

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetTextStyle(&WidgetStyle);
	}
}

const FSlateFontInfo& UMultiLineEditableText::GetFont() const
{
	return WidgetStyle.Font;
}

void UMultiLineEditableText::SetFont(FSlateFontInfo InFontInfo)
{
	WidgetStyle.SetFont(InFontInfo);

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetTextStyle(&WidgetStyle);
	}
}

void UMultiLineEditableText::SetFontMaterial(UMaterialInterface* InMaterial)
{
	WidgetStyle.SetFontMaterial(InMaterial);

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetTextStyle(&WidgetStyle);
	}
}

void UMultiLineEditableText::SetFontOutlineMaterial(UMaterialInterface* InMaterial)
{
	WidgetStyle.SetFontOutlineMaterial(InMaterial);

	if (MyMultiLineEditableText.IsValid())
	{
		MyMultiLineEditableText->SetTextStyle(&WidgetStyle);
	}
}

void UMultiLineEditableText::HandleOnTextChanged(const FText& InText)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
		OnTextChanged.Broadcast(InText);
	}
}

void UMultiLineEditableText::HandleOnTextCommitted(const FText& InText, ETextCommit::Type CommitMethod)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
	OnTextCommitted.Broadcast(InText, CommitMethod);
}

#if WITH_EDITOR

const FText UMultiLineEditableText::GetPaletteCategory()
{
	return LOCTEXT("Input", "Input");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

