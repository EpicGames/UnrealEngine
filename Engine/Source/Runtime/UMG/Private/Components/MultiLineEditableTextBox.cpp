// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/MultiLineEditableTextBox.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Engine/Font.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiLineEditableTextBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UMultiLineEditableTextBox

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UMultiLineEditableTextBox::UMultiLineEditableTextBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetEditableTextBoxStyle();
#if WITH_EDITOR
	TextStyle_DEPRECATED = WidgetStyle.TextStyle;
#endif

	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UFont> DefaultFontObj(*UWidget::GetDefaultFontName());
		FSlateFontInfo Font(DefaultFontObj.Object, 24, FName("Regular"));
		//The FSlateFontInfo just created doesn't contain a composite font (while the default from the WidgetStyle does),
		//so in the case the Font object is replaced by a null one, we have to keep the composite one as a fallback.
		Font.CompositeFont = WidgetStyle.TextStyle.Font.CompositeFont;
	}
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetEditableTextBoxStyle();
		TextStyle_DEPRECATED = WidgetStyle.TextStyle;
	}

	bIsFontDeprecationDone = false;
#endif // WITH_EDITOR

	bIsReadOnly = false;
	AllowContextMenu = true;
	VirtualKeyboardDismissAction = EVirtualKeyboardDismissAction::TextChangeOnDismiss;
	AutoWrapText = true;
}

void UMultiLineEditableTextBox::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsLoading() && !bIsFontDeprecationDone && GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::RemoveDuplicatedStyleInfo)
	{
		TextStyle_DEPRECATED.SetFont(WidgetStyle.Font_DEPRECATED);
		WidgetStyle.SetTextStyle(TextStyle_DEPRECATED);
		bIsFontDeprecationDone = true;
	}
#endif
}

void UMultiLineEditableTextBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyEditableTextBlock.Reset();
}

TSharedRef<SWidget> UMultiLineEditableTextBox::RebuildWidget()
{
	MyEditableTextBlock = SNew(SMultiLineEditableTextBox)
		.Style(&WidgetStyle)
		.AllowContextMenu(AllowContextMenu)
		.IsReadOnly(bIsReadOnly)
//		.MinDesiredWidth(MinimumDesiredWidth)
//		.Padding(Padding)
//		.IsCaretMovedWhenGainFocus(IsCaretMovedWhenGainFocus)
//		.SelectAllTextWhenFocused(SelectAllTextWhenFocused)
//		.RevertTextOnEscape(RevertTextOnEscape)
//		.ClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit)
//		.SelectAllTextOnCommit(SelectAllTextOnCommit)
		.VirtualKeyboardOptions(VirtualKeyboardOptions)
		.VirtualKeyboardDismissAction(VirtualKeyboardDismissAction)
		.OnTextChanged(BIND_UOBJECT_DELEGATE(FOnTextChanged, HandleOnTextChanged))
		.OnTextCommitted(BIND_UOBJECT_DELEGATE(FOnTextCommitted, HandleOnTextCommitted))
		.OnAllFontFacesFinishLoading(OnAllFontFacesFinishLoading.IsBound() ? BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleAllFontFacesFinishLoading) : FSimpleDelegate())
		;

	return MyEditableTextBlock.ToSharedRef();
}

void UMultiLineEditableTextBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyEditableTextBlock.IsValid())
	{
		return;
	}

	TAttribute<FText> HintTextBinding = PROPERTY_BINDING(FText, HintText);

	MyEditableTextBlock->SetStyle(&WidgetStyle);
	MyEditableTextBlock->SetText(Text);
	MyEditableTextBlock->SetHintText(HintTextBinding);
	MyEditableTextBlock->SetAllowContextMenu(AllowContextMenu);
	MyEditableTextBlock->SetIsReadOnly(bIsReadOnly);
	MyEditableTextBlock->SetVirtualKeyboardDismissAction(VirtualKeyboardDismissAction);

//	MyEditableTextBlock->SetIsPassword(IsPassword);
//	MyEditableTextBlock->SetColorAndOpacity(ColorAndOpacity);

	// TODO UMG Complete making all properties settable on SMultiLineEditableTextBox

	Super::SynchronizeTextLayoutProperties(*MyEditableTextBlock);
}

void UMultiLineEditableTextBox::OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions)
{
	Super::OnShapedTextOptionsChanged(InShapedTextOptions);
	if (MyEditableTextBlock.IsValid())
	{
		InShapedTextOptions.SynchronizeShapedTextProperties(*MyEditableTextBlock);
	}
}

void UMultiLineEditableTextBox::OnJustificationChanged(ETextJustify::Type InJustification)
{
	Super::OnJustificationChanged(InJustification);
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetJustification(InJustification);
	}
}

void UMultiLineEditableTextBox::OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy)
{
	Super::OnWrappingPolicyChanged(InWrappingPolicy);
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetWrappingPolicy(InWrappingPolicy);
	}
}

void UMultiLineEditableTextBox::OnAutoWrapTextChanged(bool InAutoWrapText)
{
	Super::OnAutoWrapTextChanged(InAutoWrapText);
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetAutoWrapText(InAutoWrapText);
	}
}

void UMultiLineEditableTextBox::OnWrapTextAtChanged(float InWrapTextAt)
{
	Super::OnWrapTextAtChanged(InWrapTextAt);
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetWrapTextAt(InWrapTextAt);
	}
}

void UMultiLineEditableTextBox::OnLineHeightPercentageChanged(float InLineHeightPercentage)
{
	Super::OnLineHeightPercentageChanged(InLineHeightPercentage);
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetLineHeightPercentage(InLineHeightPercentage);
	}
}

void UMultiLineEditableTextBox::OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine)
{
	Super::OnApplyLineHeightToBottomLineChanged(InApplyLineHeightToBottomLine);
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetApplyLineHeightToBottomLine(InApplyLineHeightToBottomLine);
	}
}

void UMultiLineEditableTextBox::OnMarginChanged(const FMargin& InMargin)
{
	Super::OnMarginChanged(InMargin);
	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetMargin(InMargin);
	}
}

FText UMultiLineEditableTextBox::GetText() const
{
	if ( MyEditableTextBlock.IsValid() )
	{
		return MyEditableTextBlock->GetText();
	}

	return Text;
}

void UMultiLineEditableTextBox::SetText(FText InText)
{
	if (SetTextInternal(InText))
	{
		if (MyEditableTextBlock.IsValid())
		{
			MyEditableTextBlock->SetText(Text);
		}
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
}

bool UMultiLineEditableTextBox::SetTextInternal(const FText& InText)
{
	if (!Text.IdenticalTo(InText, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		Text = InText;
		return true;
	}

	return false;
}

FText UMultiLineEditableTextBox::GetHintText() const
{
	if (MyEditableTextBlock.IsValid())
	{
		return MyEditableTextBlock->GetHintText();
	}

	return HintText;
}

void UMultiLineEditableTextBox::SetHintText(FText InHintText)
{
	HintText = InHintText;
	HintTextDelegate.Clear();
	if ( MyEditableTextBlock.IsValid() )
	{
		TAttribute<FText> HintTextBinding = PROPERTY_BINDING(FText, HintText);
		MyEditableTextBlock->SetHintText(HintTextBinding);
	}
}

void UMultiLineEditableTextBox::SetError(FText InError)
{
	if ( MyEditableTextBlock.IsValid() )
	{
		MyEditableTextBlock->SetError(InError);
	}
}

bool UMultiLineEditableTextBox::GetIsReadOnly() const
{
	return bIsReadOnly;
}

void UMultiLineEditableTextBox::SetIsReadOnly(bool bReadOnly)
{
	bIsReadOnly = bReadOnly;

	if ( MyEditableTextBlock.IsValid() )
	{
		MyEditableTextBlock->SetIsReadOnly(bIsReadOnly);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UMultiLineEditableTextBox::SetTextStyle(const FTextBlockStyle& InTextStyle)
{
	WidgetStyle.SetTextStyle(InTextStyle);

	if (MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetTextStyle(&InTextStyle);
	}
}

void UMultiLineEditableTextBox::SetForegroundColor(FLinearColor color)
{
	if(MyEditableTextBlock.IsValid())
	{
		MyEditableTextBlock->SetForegroundColor(color);
	}
}

void UMultiLineEditableTextBox::HandleOnTextChanged(const FText& InText)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
		OnTextChanged.Broadcast(InText);
	}
}

void UMultiLineEditableTextBox::HandleOnTextCommitted(const FText& InText, ETextCommit::Type CommitMethod)
{
	if (SetTextInternal(InText))
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
	}
	OnTextCommitted.Broadcast(InText, CommitMethod);
}

#if WITH_EDITOR

const FText UMultiLineEditableTextBox::GetPaletteCategory()
{
	return LOCTEXT("Input", "Input");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

