// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/NamedSlot.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamedSlot)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UNamedSlot

UNamedSlot::UNamedSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = true;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UNamedSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyBox.Reset();
}

TSharedRef<SWidget> UNamedSlot::RebuildWidget()
{
	MyBox = SNew(SBox);

	if ( IsDesignTime() )
	{
		MyBox->SetContent(
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromName(GetFName()))
			]
		);
	}

	// Add any existing content to the new slate box
	if ( GetChildrenCount() > 0 )
	{
		UPanelSlot* ContentSlot = GetContentSlot();
		if ( ContentSlot->Content )
		{
			MyBox->SetContent(ContentSlot->Content->TakeWidget());
		}
	}

	return MyBox.ToSharedRef();
}

void UNamedSlot::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if ( MyBox.IsValid() && InSlot->Content )
	{
		MyBox->SetContent(InSlot->Content->TakeWidget());
	}
}

void UNamedSlot::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyBox.IsValid() )
	{
		MyBox->SetContent(SNullWidget::NullWidget);

		if ( IsDesignTime() )
		{
			MyBox->SetContent(
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromName(GetFName()))
				]
			);
		}
	}
}

#if WITH_EDITOR

const FText UNamedSlot::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

FGuid UNamedSlot::GetSlotGUID() const
{
	return SlotGuid;
}
#endif

void UNamedSlot::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

void UNamedSlot::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	const int32 FortniteMainBranchObjectVersion = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (FortniteMainBranchObjectVersion < FFortniteMainBranchObjectVersion::WidgetInheritedNamedSlots)
	{
		bExposeOnInstanceOnly = true;
	}
#endif
}


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

