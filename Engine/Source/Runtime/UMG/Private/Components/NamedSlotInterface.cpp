// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/NamedSlotInterface.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamedSlotInterface)

UNamedSlotInterface::UNamedSlotInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool INamedSlotInterface::ContainsContent(UWidget* Content) const
{
	return FindSlotForContent(Content) != NAME_None;
}

FName INamedSlotInterface::FindSlotForContent(UWidget* Content) const
{
	TArray<FName> SlotNames;
	GetSlotNames(SlotNames);

	for ( const FName& SlotName : SlotNames )
	{
		if ( GetContentForSlot(SlotName) == Content )
		{
			return SlotName;
		}
	}

	return NAME_None;
}

void INamedSlotInterface::ReleaseNamedSlotSlateResources(bool bReleaseChildren)
{
	if ( bReleaseChildren )
	{
		TArray<FName> SlotNames;
		GetSlotNames(SlotNames);

		for ( const FName& SlotName : SlotNames )
		{
			if ( UWidget* Content = GetContentForSlot(SlotName) )
			{
				Content->ReleaseSlateResources(bReleaseChildren);
			}
		}
	}
}

#if WITH_EDITOR

void INamedSlotInterface::SetNamedSlotDesignerFlags(EWidgetDesignFlags NewFlags)
{
	TArray<FName> SlotNames;
	GetSlotNames(SlotNames);

	for (const FName& SlotName : SlotNames)
	{
		if (UWidget* Content = GetContentForSlot(SlotName))
		{
			Content->SetDesignerFlags(NewFlags);
		}
	}
}

#endif
