// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/NativeWidgetHost.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "UMGStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NativeWidgetHost)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UNativeWidgetHost

UNativeWidgetHost::UNativeWidgetHost(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
}

void UNativeWidgetHost::SetContent(TSharedRef<SWidget> InContent)
{
	if (NativeWidget != InContent)
	{
		NativeWidget = InContent;

		TSharedPtr<SWidget> StableMyWidget = MyWidget.Pin();
		if (StableMyWidget.IsValid())
		{
			TSharedPtr<SBox> MyBox = StaticCastSharedPtr<SBox>(StableMyWidget);
			MyBox->SetContent(InContent);
		}
	}
}

void UNativeWidgetHost::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	NativeWidget.Reset();
}

TSharedRef<SWidget> UNativeWidgetHost::RebuildWidget()
{
	return SNew(SBox)
		[
			( NativeWidget.IsValid() ) ? NativeWidget.ToSharedRef() : GetDefaultContent()
		];
}

TSharedRef<SWidget> UNativeWidgetHost::GetDefaultContent()
{
	if ( IsDesignTime() )
	{
		return SNew(SBorder)
			.Visibility(EVisibility::HitTestInvisible)
			.BorderImage(FUMGStyle::Get().GetBrush("MarchingAnts"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NativeWidgetHostText", "Slate Widget Host"))
			];
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#if WITH_EDITOR

const FText UNativeWidgetHost::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

