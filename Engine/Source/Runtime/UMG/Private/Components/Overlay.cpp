// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Overlay.h"
#include "Components/OverlaySlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Overlay)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UOverlay

UOverlay::UOverlay(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UOverlay::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyOverlay.Reset();
}

UOverlaySlot* UOverlay::AddChildToOverlay(UWidget* Content)
{
	return Cast<UOverlaySlot>(Super::AddChild(Content));
}

UClass* UOverlay::GetSlotClass() const
{
	return UOverlaySlot::StaticClass();
}

void UOverlay::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyOverlay.IsValid() )
	{
		CastChecked<UOverlaySlot>(InSlot)->BuildSlot(MyOverlay.ToSharedRef());
	}
}

void UOverlay::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyOverlay.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyOverlay->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

bool UOverlay::ReplaceOverlayChildAt(int32 Index, UWidget* Content)
{
	if (!Slots.IsValidIndex(Index) || Content == nullptr)
	{
		return false;
	}

	UOverlaySlot* OverlaySlot = CastChecked<UOverlaySlot>(Slots[Index]);
	OverlaySlot->ReplaceContent(Content);
	return true;
}

TSharedRef<SWidget> UOverlay::RebuildWidget()
{
	MyOverlay = SNew(SOverlay);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UOverlaySlot* TypedSlot = Cast<UOverlaySlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyOverlay.ToSharedRef());
		}
	}

	return MyOverlay.ToSharedRef();
}

#if WITH_EDITOR

const FText UOverlay::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

