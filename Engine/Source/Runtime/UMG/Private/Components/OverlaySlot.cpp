// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/OverlaySlot.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OverlaySlot)

/////////////////////////////////////////////////////
// UOverlaySlot

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UOverlaySlot::UOverlaySlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	HorizontalAlignment = HAlign_Left;
	VerticalAlignment = VAlign_Top;
	Slot = nullptr;
}

void UOverlaySlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UOverlaySlot::BuildSlot(TSharedRef<SOverlay> Overlay)
{
	Overlay->AddSlot()
		.Expose(Slot)
		.Padding(Padding)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
}

void UOverlaySlot::ReplaceContent(UWidget* NewContent)
{
	if (Content != NewContent)
	{
		if (NewContent)
		{
			NewContent->RemoveFromParent();
		}

		if (UWidget* PreviousWidget = Content)
		{
			// Setting Slot=null before RemoveFromParent to prevent destroying this slot
			PreviousWidget->Slot = nullptr;
			PreviousWidget->RemoveFromParent();
		}

		Content = NewContent;

		if (Content)
		{
			Content->Slot = this;
		}

		if (Slot)
		{
			Slot->AttachWidget(Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget());
		}
	}
}

void UOverlaySlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Slot )
	{
		Slot->SetPadding(InPadding);
	}
}

FMargin UOverlaySlot::GetPadding() const
{
	return Padding;
}

void UOverlaySlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EHorizontalAlignment UOverlaySlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UOverlaySlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Slot )
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}

EVerticalAlignment UOverlaySlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UOverlaySlot::SynchronizeProperties()
{
	SetPadding(Padding);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

