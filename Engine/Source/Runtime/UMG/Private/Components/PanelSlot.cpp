// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PanelSlot.h"
#include "Components/PanelWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PanelSlot)

/////////////////////////////////////////////////////
// UPanelSlot

UPanelSlot::UPanelSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UWidget* UPanelSlot::GetContent() const
{
	return Content;
}

#if WITH_EDITOR
bool UPanelSlot::IsDesignTime() const
{
	return Parent->IsDesignTime();
}
#endif

void UPanelSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	// ReleaseSlateResources for Content unless the content is a UUserWidget as they are responsible for releasing their own content.
	if (IsValid(Content) && !Content->IsA<UUserWidget>())
	{
		Content->ReleaseSlateResources(bReleaseChildren);
	}
}

