// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PanelWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PanelWidget)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UPanelWidget

UPanelWidget::UPanelWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bCanHaveMultipleChildren(true)
{
}

void UPanelWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	if ( bReleaseChildren )
	{
		for ( int32 SlotIndex = 0; SlotIndex < Slots.Num(); SlotIndex++ )
		{
			if ( Slots[SlotIndex] && Slots[SlotIndex]->Content != nullptr )
			{
				Slots[SlotIndex]->ReleaseSlateResources(bReleaseChildren);
			}
		}
	}
}

int32 UPanelWidget::GetChildrenCount() const
{
	return Slots.Num();
}

UWidget* UPanelWidget::GetChildAt(int32 Index) const
{
	
	if (Slots.IsValidIndex(Index))
	{
		// This occasionally is null during garbage collection passes during begin destroy, when we're
		// removing objects.  Consider not removing from parents during GC.
		if (UPanelSlot* ChildSlot = Slots[Index])
		{
			return ChildSlot->Content;
		}
	}

	return nullptr;
}

TArray<UWidget*> UPanelWidget::GetAllChildren() const
{
	TArray<UWidget*> Result;
	Result.Reserve(Slots.Num());

	for(UPanelSlot* ChildSlot : Slots)
	{
		Result.Add(ChildSlot->Content);
	}

	return Result;
}

int32 UPanelWidget::GetChildIndex(const UWidget* Content) const
{
	if (Content)
	{
		const int32 ChildCount = GetChildrenCount();
		for (int32 ChildIndex = 0; ChildIndex < ChildCount; ChildIndex++)
		{
			if (GetChildAt(ChildIndex) == Content)
			{
				return ChildIndex;
			}
		}
	}
	return INDEX_NONE;
}

bool UPanelWidget::HasChild(UWidget* Content) const
{
	if ( !Content )
	{
		return false;
	}
	return ( Content->GetParent() == this );
}

bool UPanelWidget::RemoveChildAt(int32 Index)
{
	if ( Index < 0 || Index >= Slots.Num() )
	{
		return false;
	}

	UPanelSlot* PanelSlot = Slots[Index];
	Slots.RemoveAt(Index);

	if (PanelSlot)
	{
		if (PanelSlot->Content)
		{
			PanelSlot->Content->Slot = nullptr;
		}

		OnSlotRemoved(PanelSlot);

		const bool bReleaseChildren = true;
		PanelSlot->ReleaseSlateResources(bReleaseChildren);
		PanelSlot->Parent = nullptr;
		PanelSlot->Content = nullptr;
	}
	else
	{
		return false;
	}

	InvalidateLayoutAndVolatility();

	return true;
}

UPanelSlot* UPanelWidget::AddChild(UWidget* Content)
{
	return AddChild(Content, nullptr);
}

UPanelSlot* UPanelWidget::AddChild(UWidget* Content, UPanelSlot* SlotTemplate)
{
	if (Content == nullptr)
	{
		return nullptr;
	}

	if ( !bCanHaveMultipleChildren && GetChildrenCount() > 0 )
	{
		return nullptr;
	}

	Content->RemoveFromParent();

	EObjectFlags NewObjectFlags = RF_Transactional;
	if (HasAnyFlags(RF_Transient))
	{
		NewObjectFlags |= RF_Transient;
	}

	UPanelSlot* PanelSlot; 
	if (SlotTemplate && GetSlotClass() == SlotTemplate->GetClass())
	{
		UWidget* TempContent = SlotTemplate->Content;
		UPanelWidget* TempParent = SlotTemplate->Parent;
		SlotTemplate->Content = nullptr;
		SlotTemplate->Parent = nullptr;

		PanelSlot = NewObject<UPanelSlot>(this, GetSlotClass(), NAME_None, NewObjectFlags, SlotTemplate);

		SlotTemplate->Content = TempContent;
		SlotTemplate->Parent = TempParent;
	}
	else
	{
		PanelSlot = NewObject<UPanelSlot>(this, GetSlotClass(), NAME_None, NewObjectFlags);
	}

	PanelSlot->Content = Content;
	PanelSlot->Parent = this;

	Content->Slot = PanelSlot;

	Slots.Add(PanelSlot);

	OnSlotAdded(PanelSlot);

	InvalidateLayoutAndVolatility();

	return PanelSlot;
}

UPanelSlot* UPanelWidget::InsertChildAt(int32 Index, UWidget* Content)
{
	return InsertChildAt(Index, Content, nullptr);
}

UPanelSlot* UPanelWidget::InsertChildAt(int32 Index, UWidget* Content, UPanelSlot* SlotTemplate)
{
	if (Content)
	{
		UPanelSlot* NewSlot = AddChild(Content, SlotTemplate);
		ShiftChild(Index, Content);
		return NewSlot;
	}
	return nullptr;
}

void UPanelWidget::ShiftChild(int32 Index, UWidget* Child)
{
	int32 CurrentIndex = GetChildIndex(Child);
	if (CurrentIndex != INDEX_NONE)
	{
		Slots.RemoveAt(CurrentIndex);
		Slots.Insert(Child->Slot, FMath::Clamp(Index, 0, Slots.Num()));

		InvalidateLayoutAndVolatility();
	}
}

#if WITH_EDITOR

bool UPanelWidget::ReplaceChildAt(int32 Index, UWidget* Content)
{
	if ( Index < 0 || Index >= Slots.Num() || Content == nullptr)
	{
		return false;
	}

	UPanelSlot* PanelSlot = Slots[Index];
	PanelSlot->Content = Content;
	Content->Slot = PanelSlot;

	PanelSlot->SynchronizeProperties();

	return true;
}

bool UPanelWidget::ReplaceChild(UWidget* CurrentChild, UWidget* NewChild)
{
	if (NewChild)
	{
		int32 Index = GetChildIndex(CurrentChild);
		if (Index != INDEX_NONE)
		{
			return ReplaceChildAt(Index, NewChild);
		}
	}

	return false;
}

void UPanelWidget::SetDesignerFlags(EWidgetDesignFlags NewFlags)
{
	Super::SetDesignerFlags(NewFlags);

	// Also mark all children as design time widgets.
	int32 Children = GetChildrenCount();
	for ( int32 SlotIndex = 0; SlotIndex < Children; SlotIndex++ )
	{
		if ( Slots[SlotIndex]->Content != nullptr )
		{
			Slots[SlotIndex]->Content->SetDesignerFlags(NewFlags);
		}
	}
}

#endif

bool UPanelWidget::RemoveChild(UWidget* Content)
{
	int32 ChildIndex = GetChildIndex(Content);
	if ( ChildIndex != INDEX_NONE)
	{
		return RemoveChildAt(ChildIndex);
	}

	return false;
}

bool UPanelWidget::HasAnyChildren() const
{
	return GetChildrenCount() > 0;
}

void UPanelWidget::ClearChildren()
{
	for ( int32 ChildIndex = GetChildrenCount() - 1; ChildIndex >= 0; ChildIndex-- )
	{
		RemoveChildAt(ChildIndex);
	}
}

#if WITH_EDITOR

TSharedRef<SWidget> UPanelWidget::RebuildDesignWidget(TSharedRef<SWidget> Content)
{
	return CreateDesignerOutline(Content);
}

#endif

void UPanelWidget::PostLoad()
{
	Super::PostLoad();

	for ( int32 SlotIndex = 0; SlotIndex < Slots.Num(); SlotIndex++ )
	{
		// Remove any slots where their content is null, we don't support content-less slots.
		if (!Slots[SlotIndex] || Slots[SlotIndex]->Content == nullptr)
		{
			Slots.RemoveAt(SlotIndex);
			SlotIndex--;
		}
	}
}

const TArray<UPanelSlot*>& UPanelWidget::GetSlots() const
{
	return Slots;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

