// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PostBufferBlurUpdater.h"

// Exclude SlateRHIRenderer related includes & implementations from server builds since the module is not a dependency / will not link for UMG on the server
#if !UE_SERVER
#include "FX/SlateRHIPostBufferProcessor.h"
#include "FX/SlateRHIPostBufferBlur.h"
#endif // !UE_SERVER

#include UE_INLINE_GENERATED_CPP_BY_NAME(PostBufferBlurUpdater)

#define LOCTEXT_NAMESPACE "UMG"

TAutoConsoleVariable<int32> CVarUmgMaxPostBufferBlurStrength(
	TEXT("umg.MaxPostBufferBlurStrength"),
	-1,
	TEXT("Limits post buffer blur strength. -1 = unlimited\r\n"),
	ECVF_Default
);


/////////////////////////////////////////////////////
// UPostBufferBlurUpdater

TSharedPtr<FSlatePostProcessorUpdaterProxy> UPostBufferBlurUpdater::GetRenderThreadProxy() const
{
	TSharedPtr<FSlatePostProcessorUpdaterProxy> Proxy = nullptr;
#if !UE_SERVER
	TSharedPtr<FPostBufferBlurUpdaterProxy> BlurProxy = MakeShared<FPostBufferBlurUpdaterProxy>();
	BlurProxy->GaussianBlurStrength_RenderThread = GaussianBlurStrength;
	Proxy = BlurProxy;
#endif // !UE_SERVER
	return Proxy;
}


/////////////////////////////////////////////////////
// FPostBufferBlurUpdaterProxy

void FPostBufferBlurUpdaterProxy::UpdateProcessor_RenderThread(TSharedPtr<FSlateRHIPostBufferProcessorProxy> InProcessor) const
{
#if !UE_SERVER
	TSharedPtr<FSlatePostBufferBlurProxy> BlurRHIProxy = StaticCastSharedPtr<FSlatePostBufferBlurProxy>(InProcessor);
	const int32 MaxBlurStrength = CVarUmgMaxPostBufferBlurStrength.GetValueOnRenderThread();
	BlurRHIProxy->GaussianBlurStrength_RenderThread = (MaxBlurStrength >= 0) ? FMath::Min(GaussianBlurStrength_RenderThread, MaxBlurStrength) : GaussianBlurStrength_RenderThread;
#endif // !UE_SERVER
}


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

