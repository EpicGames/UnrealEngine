// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PostBufferUpdate.h"

#include "Slate/SPostBufferUpdate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PostBufferUpdate)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UPostBufferUpdate

UPostBufferUpdate::UPostBufferUpdate()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bUpdateOnlyPaintArea = true;
	bPerformDefaultPostBufferUpdate = false;
	BuffersToUpdate = {};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	UpdateBufferInfos = {};
}

TSharedRef<SWidget> UPostBufferUpdate::RebuildWidget()
{
	MyPostBufferUpdate = SNew(SPostBufferUpdate)
		.bUsePaintGeometry(true);

	bool bSetBuffersToUpdate = true;

#if WITH_EDITOR
	if (IsDesignTime())
	{
		bSetBuffersToUpdate = false;
	}
#endif // WITH_EDITOR

	if (bSetBuffersToUpdate)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!UpdateBufferInfos.IsEmpty())
		{
			TArray<ESlatePostRT> TempBuffersToUpdate;
			TMap<ESlatePostRT, TSharedPtr<FSlatePostProcessorUpdaterProxy>> TempProcessorUpdaters;

			for (const FSlatePostBufferUpdateInfo& UpdateBufferInfo : UpdateBufferInfos)
			{
				TempBuffersToUpdate.Add(UpdateBufferInfo.BufferToUpdate);
				if (TObjectPtr<USlatePostBufferProcessorUpdater> PostParamUpdater = UpdateBufferInfo.PostParamUpdater)
				{
					TSharedPtr<FSlatePostProcessorUpdaterProxy> PostParamUpdaterProxy = PostParamUpdater->GetRenderThreadProxy();
					TempProcessorUpdaters.Add(UpdateBufferInfo.BufferToUpdate, PostParamUpdaterProxy);
				}
			}

			MyPostBufferUpdate->SetBuffersToUpdate(TempBuffersToUpdate);
			MyPostBufferUpdate->SetProcessorUpdaters(TempProcessorUpdaters);
		}
		else
		{
			MyPostBufferUpdate->SetBuffersToUpdate(BuffersToUpdate);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return MyPostBufferUpdate.ToSharedRef();
}

void UPostBufferUpdate::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyPostBufferUpdate.IsValid())
	{
		return;
	}

	MyPostBufferUpdate->SetUsePaintGeometry(true);

	bool bSetBuffersToUpdate = true;

#if WITH_EDITOR
	if (IsDesignTime())
	{
		bSetBuffersToUpdate = false;
	}
#endif // WITH_EDITOR

	if (bSetBuffersToUpdate)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!UpdateBufferInfos.IsEmpty())
		{
			TArray<ESlatePostRT> TempBuffersToUpdate;
			TMap<ESlatePostRT, TSharedPtr<FSlatePostProcessorUpdaterProxy>> TempProcessorUpdaters;

			for (const FSlatePostBufferUpdateInfo& UpdateBufferInfo : UpdateBufferInfos)
			{
				TempBuffersToUpdate.Add(UpdateBufferInfo.BufferToUpdate);
				if (TObjectPtr<USlatePostBufferProcessorUpdater> PostParamUpdater = UpdateBufferInfo.PostParamUpdater)
				{
					TSharedPtr<FSlatePostProcessorUpdaterProxy> PostParamUpdaterProxy = PostParamUpdater->GetRenderThreadProxy();
					TempProcessorUpdaters.Add(UpdateBufferInfo.BufferToUpdate, PostParamUpdaterProxy);
				}
			}

			MyPostBufferUpdate->SetBuffersToUpdate(TempBuffersToUpdate);
			MyPostBufferUpdate->SetProcessorUpdaters(TempProcessorUpdaters);
		}
		else
		{
			MyPostBufferUpdate->SetBuffersToUpdate(BuffersToUpdate);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

UMG_API void UPostBufferUpdate::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	if (MyPostBufferUpdate.IsValid())
	{
		MyPostBufferUpdate->ReleasePostBufferUpdater();
	}

	MyPostBufferUpdate.Reset();
}

#if WITH_EDITOR
UMG_API bool UPostBufferUpdate::DoesUpdateSlatePostRT(ESlatePostRT SlateBufferBits) const
{
	for (const FSlatePostBufferUpdateInfo& Info : UpdateBufferInfos)
	{
		if (static_cast<uint8>(SlateBufferBits) & static_cast<uint8>(Info.BufferToUpdate))
		{
			return true;
		}
	}

	return false;
}
#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

