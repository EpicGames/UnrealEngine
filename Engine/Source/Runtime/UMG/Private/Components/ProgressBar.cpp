// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ProgressBar.h"
#include "Slate/SlateBrushAsset.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProgressBar)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UProgressBar

UProgressBar::UProgressBar(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetProgressBarStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetProgressBarStyle();
	}
#endif // WITH_EDITOR

	WidgetStyle.FillImage.TintColor = FLinearColor::White;

	BarFillType = EProgressBarFillType::LeftToRight;
	BarFillStyle = EProgressBarFillStyle::Mask;
	bIsMarquee = false;
	Percent = 0;
	FillColorAndOpacity = FLinearColor::White;
	BorderPadding = FVector2D(0, 0);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UProgressBar::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyProgressBar.Reset();
}

TSharedRef<SWidget> UProgressBar::RebuildWidget()
{
	MyProgressBar = SNew(SProgressBar);

	return MyProgressBar.ToSharedRef();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UProgressBar::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyProgressBar.IsValid())
	{
		return;
	}

	TAttribute< TOptional<float> > PercentBinding = OPTIONAL_BINDING_CONVERT(float, Percent, TOptional<float>, ConvertFloatToOptionalFloat);
	TAttribute<FSlateColor> FillColorAndOpacityBinding = PROPERTY_BINDING(FSlateColor, FillColorAndOpacity);

	MyProgressBar->SetStyle(&WidgetStyle);

	MyProgressBar->SetBarFillType(BarFillType);
	MyProgressBar->SetBarFillStyle(BarFillStyle);
	MyProgressBar->SetPercent(bIsMarquee ? TOptional<float>() : PercentBinding);
	MyProgressBar->SetFillColorAndOpacity(FillColorAndOpacityBinding);
	MyProgressBar->SetBorderPadding(BorderPadding);
}

const FProgressBarStyle& UProgressBar::GetWidgetStyle() const
{
	return WidgetStyle;
}

void UProgressBar::SetWidgetStyle(const FProgressBarStyle& InStyle)
{
	WidgetStyle = InStyle;
	if (MyProgressBar.IsValid())
	{
		MyProgressBar->SetStyle(&WidgetStyle);
	}
}

float UProgressBar::GetPercent() const
{
	return Percent;
}

void UProgressBar::SetPercent(float InPercent)
{
	Percent = InPercent;
	if (MyProgressBar.IsValid())
	{
		MyProgressBar->SetPercent(InPercent);
	}
}

EProgressBarFillType::Type UProgressBar::GetBarFillType() const
{
	return BarFillType;
}

void UProgressBar::SetBarFillType(EProgressBarFillType::Type InBarFillType)
{
	BarFillType = InBarFillType;
	if (MyProgressBar.IsValid())
	{
		MyProgressBar->SetBarFillType(BarFillType);
	}
}

EProgressBarFillStyle::Type UProgressBar::GetBarFillStyle() const
{
	return BarFillStyle;
}

void UProgressBar::SetBarFillStyle(EProgressBarFillStyle::Type InBarFillStyle)
{
	BarFillStyle = InBarFillStyle;
	if (MyProgressBar.IsValid())
	{
		MyProgressBar->SetBarFillStyle(BarFillStyle);
	}
}

bool UProgressBar::UseMarquee() const
{
	return bIsMarquee;
}

void UProgressBar::SetIsMarquee(bool InbIsMarquee)
{
	bIsMarquee = InbIsMarquee;
	if (MyProgressBar.IsValid())
	{
		MyProgressBar->SetPercent(bIsMarquee ? TOptional<float>() : Percent);
	}
}

FVector2D UProgressBar::GetBorderPadding() const
{
	return BorderPadding;
}

void UProgressBar::SetBorderPadding(FVector2D InBorderPadding)
{
	BorderPadding = InBorderPadding;
	if (MyProgressBar.IsValid())
	{
		MyProgressBar->SetBorderPadding(BorderPadding);
	}
}

FLinearColor UProgressBar::GetFillColorAndOpacity() const
{
	return FillColorAndOpacity;
}

void UProgressBar::SetFillColorAndOpacity(FLinearColor Color)
{
	FillColorAndOpacity = Color;
	if (MyProgressBar.IsValid())
	{
		MyProgressBar->SetFillColorAndOpacity(FillColorAndOpacity);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UProgressBar::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

void UProgressBar::OnCreationFromPalette()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FillColorAndOpacity = FLinearColor(0.f, 0.5f, 1.0f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

