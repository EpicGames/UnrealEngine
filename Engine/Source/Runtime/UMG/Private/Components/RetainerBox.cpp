// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RetainerBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include "Slate/SRetainerWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RetainerBox)

#define LOCTEXT_NAMESPACE "UMG"

static FName DefaultTextureParameterName("Texture");

/////////////////////////////////////////////////////
// URetainerBox

URetainerBox::URetainerBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetVisibilityInternal(ESlateVisibility::Visible);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Phase = 0;
	PhaseCount = 1;
	RenderOnPhase = true;
	RenderOnInvalidation = false;
	TextureParameter = DefaultTextureParameterName;
#if WITH_EDITOR
	bShowEffectsInDesigner = true;
#endif // WITH_EDITOR
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URetainerBox::SetRenderingPhase(int32 PhaseToRenderOn, int32 TotalRenderingPhases)
{
	Phase = PhaseToRenderOn;
	PhaseCount = TotalRenderingPhases;
	
	if ( PhaseCount < 1 )
	{
		PhaseCount = 1;
	}

	if (MyRetainerWidget.IsValid())
	{
		MyRetainerWidget->SetRenderingPhase(Phase, PhaseCount);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void URetainerBox::RequestRender()
{
	if ( MyRetainerWidget.IsValid() )
	{
		MyRetainerWidget->RequestRender();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const UMaterialInterface* URetainerBox::GetEffectMaterialInterface() const
{
	return EffectMaterial;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UMaterialInstanceDynamic* URetainerBox::GetEffectMaterial() const
{
	if ( MyRetainerWidget.IsValid() )
	{
		return MyRetainerWidget->GetEffectMaterial();
	}

	return nullptr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URetainerBox::SetEffectMaterial(UMaterialInterface* InEffectMaterial)
{
	EffectMaterial = InEffectMaterial;
	if ( MyRetainerWidget.IsValid() )
	{
		MyRetainerWidget->SetEffectMaterial(EffectMaterial);
	}
}

void URetainerBox::SetTextureParameter(FName InTextureParameter)
{
	TextureParameter = InTextureParameter;
	if ( MyRetainerWidget.IsValid() )
	{
		MyRetainerWidget->SetTextureParameter(TextureParameter);
	}
}

const FName& URetainerBox::GetTextureParameter() const
{
	return TextureParameter;
}

void URetainerBox::SetRetainRendering(bool bInRetainRendering)
{
	bRetainRender = bInRetainRendering;

	if (MyRetainerWidget.IsValid())
	{
		MyRetainerWidget->SetRetainedRendering(bRetainRender);
	}
}

bool URetainerBox::IsRetainRendering() const
{
	return bRetainRender;
}

int32 URetainerBox::GetPhase() const
{
	return Phase;
}

int32 URetainerBox::GetPhaseCount() const
{
	return PhaseCount;
}

bool URetainerBox::IsRenderOnInvalidation() const
{
	return RenderOnInvalidation;
}

bool URetainerBox::IsRenderOnPhase() const
{
	return RenderOnPhase;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void URetainerBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyRetainerWidget.Reset();
}

TSharedRef<SWidget> URetainerBox::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyRetainerWidget =
		SNew(SRetainerWidget)
		.RenderOnInvalidation(RenderOnInvalidation)
		.RenderOnPhase(RenderOnPhase)
		.Phase(Phase)
		.PhaseCount(PhaseCount)
#if STATS
		.StatId( FName( *FString::Printf(TEXT("%s [%s]"), *GetFName().ToString(), *GetClass()->GetName() ) ) )
#endif//STATS
	;

#if WITH_EDITOR
	MyRetainerWidget->SetIsDesignTime(IsDesignTime());
	MyRetainerWidget->SetShowEffectsInDesigner(bShowEffectsInDesigner);
#endif // WITH_EDITOR

PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if ( GetChildrenCount() > 0 )
	{
		MyRetainerWidget->SetContent(GetContentSlot()->Content ? GetContentSlot()->Content->TakeWidget() : SNullWidget::NullWidget);
	}
	
	return MyRetainerWidget.ToSharedRef();
}

void URetainerBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyRetainerWidget.IsValid())
	{
		return;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITOR
	MyRetainerWidget->SetIsDesignTime(IsDesignTime());
	MyRetainerWidget->SetShowEffectsInDesigner(bShowEffectsInDesigner);
#endif // WITH_EDITOR
	MyRetainerWidget->SetRetainedRendering(bRetainRender);
	MyRetainerWidget->SetEffectMaterial(EffectMaterial);
	MyRetainerWidget->SetTextureParameter(TextureParameter);
	MyRetainerWidget->SetWorld(GetWorld());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void URetainerBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if ( MyRetainerWidget.IsValid() )
	{
		MyRetainerWidget->SetContent(InSlot->Content ? InSlot->Content->TakeWidget() : SNullWidget::NullWidget);
	}
}

void URetainerBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyRetainerWidget.IsValid() )
	{
		MyRetainerWidget->SetContent(SNullWidget::NullWidget);
	}
}

#if WITH_EDITOR

const FText URetainerBox::GetPaletteCategory()
{
	return LOCTEXT("Optimization", "Optimization");
}

#endif

FGeometry URetainerBox::GetCachedAllottedGeometry() const
{
	if (MyRetainerWidget.IsValid())
	{
		return MyRetainerWidget->GetTickSpaceGeometry();
	}

	static const FGeometry TempGeo;
	return TempGeo;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URetainerBox::InitRenderOnInvalidation(bool InRenderOnInvalidation)
{
	ensureMsgf(!MyRetainerWidget.IsValid(), TEXT("The widget is already created."));
	RenderOnInvalidation = InRenderOnInvalidation;
}

void URetainerBox::InitRenderOnPhase(bool InRenderOnPhase)
{
	ensureMsgf(!MyRetainerWidget.IsValid(), TEXT("The widget is already created."));
	RenderOnPhase = InRenderOnPhase;
}

void URetainerBox::InitPhase(int32 InPhase)
{
	ensureMsgf(!MyRetainerWidget.IsValid(), TEXT("The widget is already created."));
	Phase = InPhase;
}

void URetainerBox::InitPhaseCount(int32 InPhaseCount)
{
	ensureMsgf(!MyRetainerWidget.IsValid(), TEXT("The widget is already created."));
	PhaseCount = InPhaseCount;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
bool URetainerBox::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URetainerBox, Phase)
		|| InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(URetainerBox, PhaseCount))
	{
		return RenderOnPhase && bRetainRender;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	return true;
}
#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

