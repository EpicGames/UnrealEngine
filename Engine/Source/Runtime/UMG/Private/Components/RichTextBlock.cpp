// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RichTextBlock.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Font.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Components/RichTextBlockDecorator.h"
#include "Styling/SlateStyle.h"
#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "Framework/Text/RichTextMarkupProcessing.h"
#include "Framework/Text/IRichTextMarkupParser.h"
#include "Framework/Text/IRichTextMarkupWriter.h"
#include "RenderDeferredCleanup.h"
#include "Editor/WidgetCompilerLog.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RichTextBlock)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// URichTextBlock

template< class ObjectType >
struct FDeferredDeletor : public FDeferredCleanupInterface
{
public:
	FDeferredDeletor(ObjectType* InInnerObjectToDelete)
		: InnerObjectToDelete(InInnerObjectToDelete)
	{
	}

	virtual ~FDeferredDeletor()
	{
		delete InnerObjectToDelete;
	}

private:
	ObjectType* InnerObjectToDelete;
};

template< class ObjectType >
FORCEINLINE TSharedPtr< ObjectType > MakeShareableDeferredCleanup(ObjectType* InObject)
{
	return MakeShareable(InObject, [](ObjectType* ObjectToDelete) { BeginCleanup(new FDeferredDeletor<ObjectType>(ObjectToDelete)); });
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
URichTextBlock::URichTextBlock(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
	TextTransformPolicy = ETextTransformPolicy::None;
	TextOverflowPolicy = ETextOverflowPolicy::Clip;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void URichTextBlock::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyRichTextBlock.Reset();
	StyleInstance.Reset();
	InstanceDecorators.Empty();
}

TSharedRef<SWidget> URichTextBlock::RebuildWidget()
{
	UpdateStyleData();

	TArray< TSharedRef< class ITextDecorator > > CreatedDecorators;
	CreateDecorators(CreatedDecorators);

	TSharedRef<FRichTextLayoutMarshaller> Marshaller = FRichTextLayoutMarshaller::Create(CreateMarkupParser(), CreateMarkupWriter(), CreatedDecorators, StyleInstance.Get());
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyRichTextBlock = SNew(SRichTextBlock)
		.TextStyle(bOverrideDefaultStyle ? &DefaultTextStyleOverride : &DefaultTextStyle)
		.Marshaller(Marshaller)
		.OnAllFontFacesFinishLoading(OnAllFontFacesFinishLoading.IsBound() ? BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleAllFontFacesFinishLoading) : FSimpleDelegate());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return MyRichTextBlock.ToSharedRef();
}

void URichTextBlock::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyRichTextBlock.IsValid())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		MyRichTextBlock->SetText(Text);
		MyRichTextBlock->SetTransformPolicy(TextTransformPolicy);
		MyRichTextBlock->SetMinDesiredWidth(MinDesiredWidth);

		MyRichTextBlock->SetOverflowPolicy(TextOverflowPolicy);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Super::SynchronizeTextLayoutProperties(*MyRichTextBlock);
	}
}

void URichTextBlock::UpdateStyleData()
{
	if (IsDesignTime())
	{
		InstanceDecorators.Reset();
	}

	if (!StyleInstance.IsValid())
	{
		RebuildStyleInstance();

		for (TSubclassOf<URichTextBlockDecorator> DecoratorClass : DecoratorClasses)
		{
			if (UClass* ResolvedClass = DecoratorClass.Get())
			{
				if (!ResolvedClass->HasAnyClassFlags(CLASS_Abstract))
				{
					URichTextBlockDecorator* Decorator = NewObject<URichTextBlockDecorator>(this, ResolvedClass);
					InstanceDecorators.Add(Decorator);
				}
			}
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FText URichTextBlock::GetText() const
{
	if (MyRichTextBlock.IsValid())
	{
		return MyRichTextBlock->GetText();
	}

	return Text;
}

void URichTextBlock::SetText(const FText& InText)
{
	Text = InText;
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetText(InText);
	}
}

void URichTextBlock::RebuildStyleInstance()
{
	StyleInstance = MakeShareableDeferredCleanup(new FSlateStyleSet(TEXT("RichTextStyle")));

	if (TextStyleSet && TextStyleSet->GetRowStruct()->IsChildOf(FRichTextStyleRow::StaticStruct()))
	{
		for (const auto& Entry : TextStyleSet->GetRowMap())
		{
			FName SubStyleName = Entry.Key;
			FRichTextStyleRow* RichTextStyle = (FRichTextStyleRow*)Entry.Value;

			if (SubStyleName == FName("Default"))
			{
				DefaultTextStyle = RichTextStyle->TextStyle;
			}

			StyleInstance->Set(SubStyleName, RichTextStyle->TextStyle);
		}
	}
}

UDataTable* URichTextBlock::GetTextStyleSet() const
{
	return TextStyleSet;
}

void URichTextBlock::SetTextStyleSet(UDataTable* NewTextStyleSet)
{
	if (TextStyleSet != NewTextStyleSet)
	{
		TextStyleSet = NewTextStyleSet;

		RebuildStyleInstance();

		if (MyRichTextBlock.IsValid())
		{
			MyRichTextBlock->SetDecoratorStyleSet(StyleInstance.Get());
			MyRichTextBlock->SetTextStyle(DefaultTextStyle);
		}
	}
}

const FTextBlockStyle& URichTextBlock::GetDefaultTextStyle() const
{
	ensure(StyleInstance.IsValid());
	return DefaultTextStyle;
}

const FTextBlockStyle& URichTextBlock::GetCurrentDefaultTextStyle() const
{
	if (bOverrideDefaultStyle)
	{
		return DefaultTextStyleOverride;
	}
	else
	{
		ensure(StyleInstance.IsValid());
		return DefaultTextStyle;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

URichTextBlockDecorator* URichTextBlock::GetDecoratorByClass(TSubclassOf<URichTextBlockDecorator> DecoratorClass)
{
	for (URichTextBlockDecorator* Decorator : InstanceDecorators)
	{
		if (Decorator->IsA(DecoratorClass))
		{
			return Decorator;
		}
	}

	return nullptr;
}

void URichTextBlock::CreateDecorators(TArray< TSharedRef< class ITextDecorator > >& OutDecorators)
{
	for (URichTextBlockDecorator* Decorator : InstanceDecorators)
	{
		if (Decorator)
		{
			TSharedPtr<ITextDecorator> TextDecorator = Decorator->CreateDecorator(this);
			if (TextDecorator.IsValid())
			{
				OutDecorators.Add(TextDecorator.ToSharedRef());
			}
		}
	}
}

TSharedPtr< IRichTextMarkupParser > URichTextBlock::CreateMarkupParser()
{
	return FDefaultRichTextMarkupParser::GetStaticInstance();
}

TSharedPtr< IRichTextMarkupWriter > URichTextBlock::CreateMarkupWriter()
{
	return FDefaultRichTextMarkupWriter::Create();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URichTextBlock::BeginDefaultStyleOverride()
{
	if (!bOverrideDefaultStyle)
	{
		// If we aren't already overriding, make sure override style starts off matching the existing default
		bOverrideDefaultStyle = true;
		DefaultTextStyleOverride = DefaultTextStyle;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText URichTextBlock::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

void URichTextBlock::OnCreationFromPalette()
{
	//Decorators.Add(NewObject<URichTextBlockDecorator>(this, NAME_None, RF_Transactional));
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URichTextBlock::ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const
{
	Super::ValidateCompiledDefaults(CompileLog);

	if (TextStyleSet && !TextStyleSet->GetRowStruct()->IsChildOf(FRichTextStyleRow::StaticStruct()))
	{
		CompileLog.Warning(FText::Format(
			LOCTEXT("RichTextBlock_InvalidTextStyle", "{0} Text Style Set property expects a Data Table with a Rich Text Style Row structure (currently set to {1})."), 
			FText::FromString(GetName()), 
			FText::AsCultureInvariant(TextStyleSet->GetPathName())));
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif //if WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URichTextBlock::SetDefaultTextStyle(const FTextBlockStyle& InDefaultTextStyle)
{
	BeginDefaultStyleOverride();
	DefaultTextStyleOverride = InDefaultTextStyle;
	ApplyUpdatedDefaultTextStyle();
}

void URichTextBlock::SetDefaultMaterial(UMaterialInterface* InMaterial)
{
	BeginDefaultStyleOverride();
	DefaultTextStyleOverride.Font.FontMaterial = InMaterial;
	ApplyUpdatedDefaultTextStyle();
}

void URichTextBlock::ClearAllDefaultStyleOverrides()
{
	if (bOverrideDefaultStyle)
	{
		bOverrideDefaultStyle = false;
		ApplyUpdatedDefaultTextStyle();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UMaterialInstanceDynamic* URichTextBlock::GetDefaultDynamicMaterial()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TObjectPtr<UObject>& MaterialObject = bOverrideDefaultStyle ? DefaultTextStyleOverride.Font.FontMaterial : DefaultTextStyle.Font.FontMaterial;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (UMaterialInterface* Material = Cast<UMaterialInterface>(MaterialObject.Get()))
	{
		UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);

		if (!DynamicMaterial)
		{
			BeginDefaultStyleOverride();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			DynamicMaterial = UMaterialInstanceDynamic::Create(Material, this);
			DefaultTextStyleOverride.Font.FontMaterial = DynamicMaterial;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			ApplyUpdatedDefaultTextStyle();
		}

		return DynamicMaterial;
	}

	return nullptr;
}

void URichTextBlock::SetDecorators(const TArray<TSubclassOf<URichTextBlockDecorator>>& InDecoratorClasses)
{
	DecoratorClasses = InDecoratorClasses;

	StyleInstance.Reset();
	UpdateStyleData();

	if (MyRichTextBlock.IsValid())
	{
		TArray<TSharedRef<ITextDecorator>> CreatedDecorators;
		CreateDecorators(CreatedDecorators);

		MyRichTextBlock->SetDecoratorStyleSet(StyleInstance.Get());
		MyRichTextBlock->SetDecorators(CreatedDecorators);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URichTextBlock::SetDefaultColorAndOpacity(FSlateColor InColorAndOpacity)
{
	BeginDefaultStyleOverride();
	DefaultTextStyleOverride.ColorAndOpacity = InColorAndOpacity;
	ApplyUpdatedDefaultTextStyle();
}


void URichTextBlock::SetDefaultShadowColorAndOpacity(FLinearColor InShadowColorAndOpacity)
{
	BeginDefaultStyleOverride();
	DefaultTextStyleOverride.ShadowColorAndOpacity = InShadowColorAndOpacity;
	ApplyUpdatedDefaultTextStyle();
}

void URichTextBlock::SetDefaultShadowOffset(FVector2D InShadowOffset)
{
	BeginDefaultStyleOverride();
	DefaultTextStyleOverride.ShadowOffset = InShadowOffset;
	ApplyUpdatedDefaultTextStyle();
}

void URichTextBlock::SetDefaultFont(FSlateFontInfo InFontInfo)
{
	BeginDefaultStyleOverride();
	DefaultTextStyleOverride.Font = InFontInfo;
	ApplyUpdatedDefaultTextStyle();
}

void URichTextBlock::SetDefaultStrikeBrush(const FSlateBrush& InStrikeBrush)
{
	BeginDefaultStyleOverride();
	DefaultTextStyleOverride.StrikeBrush = InStrikeBrush;
	ApplyUpdatedDefaultTextStyle();
}

void URichTextBlock::OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions)
{
	Super::OnShapedTextOptionsChanged(InShapedTextOptions);
	if (MyRichTextBlock.IsValid())
	{
		InShapedTextOptions.SynchronizeShapedTextProperties(*MyRichTextBlock);
	}
}

void URichTextBlock::OnJustificationChanged(ETextJustify::Type InJustification)
{
	Super::OnJustificationChanged(InJustification);
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetJustification(InJustification);
	}
}

void URichTextBlock::OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy)
{
	Super::OnWrappingPolicyChanged(InWrappingPolicy);
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetWrappingPolicy(InWrappingPolicy);
	}
}

void URichTextBlock::OnAutoWrapTextChanged(bool InAutoWrapText)
{
	Super::OnAutoWrapTextChanged(InAutoWrapText);
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetAutoWrapText(InAutoWrapText);
	}
}

void URichTextBlock::OnWrapTextAtChanged(float InWrapTextAt)
{
	Super::OnWrapTextAtChanged(InWrapTextAt);
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetWrapTextAt(InWrapTextAt);
	}
}

void URichTextBlock::OnLineHeightPercentageChanged(float InLineHeightPercentage)
{
	Super::OnLineHeightPercentageChanged(InLineHeightPercentage);
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetLineHeightPercentage(InLineHeightPercentage);
	}
}

void URichTextBlock::OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine)
{
	Super::OnApplyLineHeightToBottomLineChanged(InApplyLineHeightToBottomLine);
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetApplyLineHeightToBottomLine(InApplyLineHeightToBottomLine);
	}
}

void URichTextBlock::OnMarginChanged(const FMargin& InMargin)
{
	Super::OnMarginChanged(InMargin);
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetMargin(InMargin);
	}
}

void URichTextBlock::SetMinDesiredWidth(float InMinDesiredWidth)
{
	MinDesiredWidth = InMinDesiredWidth;
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetMinDesiredWidth(InMinDesiredWidth);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void URichTextBlock::SetAutoWrapText(bool InAutoTextWrap)
{
	AutoWrapText = InAutoTextWrap;
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetAutoWrapText(InAutoTextWrap);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void URichTextBlock::SetTextTransformPolicy(ETextTransformPolicy InTransformPolicy)
{
	TextTransformPolicy = InTransformPolicy;
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetTransformPolicy(TextTransformPolicy);
	}
}

void URichTextBlock::SetTextOverflowPolicy(ETextOverflowPolicy InOverflowPolicy)
{
	TextOverflowPolicy = InOverflowPolicy;

	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetOverflowPolicy(TextOverflowPolicy);
	}
}

const FTextBlockStyle& URichTextBlock::GetDefaultTextStyleOverride() const
{
	return DefaultTextStyleOverride;
}

float URichTextBlock::GetMinDesiredWidth() const
{
	return MinDesiredWidth;
}

ETextTransformPolicy URichTextBlock::GetTransformPolicy() const
{
	return TextTransformPolicy;
}

ETextOverflowPolicy URichTextBlock::GetOverflowPolicy() const
{
	return TextOverflowPolicy;
}

void URichTextBlock::ApplyUpdatedDefaultTextStyle()
{
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->SetTextStyle(bOverrideDefaultStyle ? DefaultTextStyleOverride : DefaultTextStyle);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void URichTextBlock::RefreshTextLayout()
{
	if (MyRichTextBlock.IsValid())
	{
		MyRichTextBlock->Refresh();
	}
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

