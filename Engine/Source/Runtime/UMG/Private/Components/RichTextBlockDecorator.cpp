// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RichTextBlockDecorator.h"
#include "UObject/SoftObjectPtr.h"
#include "Rendering/DrawElements.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateWidgetRun.h"
#include "Framework/Application/SlateApplication.h"
#include "Fonts/FontMeasure.h"
#include "Components/RichTextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RichTextBlockDecorator)

#define LOCTEXT_NAMESPACE "UMG"

FRichTextDecorator::FRichTextDecorator(URichTextBlock* InOwner)
	: Owner(InOwner)
{
}

TSharedRef<ISlateRun> FRichTextDecorator::Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style)
{
	FTextRange ModelRange;
	ModelRange.BeginIndex = InOutModelText->Len();

	FTextRunInfo RunInfo(RunParseResult.Name, FText::FromString(OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.EndIndex - RunParseResult.ContentRange.BeginIndex)));
	for (const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid(Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	const FTextBlockStyle& TextStyle = Owner->GetCurrentDefaultTextStyle();

	TSharedPtr<ISlateRun> SlateRun;
	TSharedPtr<SWidget> DecoratorWidget = CreateDecoratorWidget(RunInfo, TextStyle);
	if (DecoratorWidget.IsValid())
	{
		*InOutModelText += TEXT('\u200B'); // Zero-Width Breaking Space
		ModelRange.EndIndex = InOutModelText->Len();

		// Calculate the baseline of the text within the owning rich text
		// Requested on demand as the font may not be loaded right now
		const FSlateFontInfo Font = TextStyle.Font;
		const float ShadowOffsetY = FMath::Min(0.0f, TextStyle.ShadowOffset.Y);

		TAttribute<int16> GetBaseline = TAttribute<int16>::CreateLambda([Font, ShadowOffsetY]()
		{
			const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			return static_cast<int16>(FontMeasure->GetBaseline(Font) - ShadowOffsetY);
		});

		FSlateWidgetRun::FWidgetRunInfo WidgetRunInfo(DecoratorWidget.ToSharedRef(), GetBaseline);
		SlateRun = FSlateWidgetRun::Create(TextLayout, RunInfo, InOutModelText, WidgetRunInfo, ModelRange);
	}
	else
	{
		// Assume there's a text handler if widget is empty, if there isn't one it will just display an empty string
		FTextBlockStyle TempStyle = TextStyle;
		CreateDecoratorText(RunInfo, TempStyle, *InOutModelText);

		ModelRange.EndIndex = InOutModelText->Len();
		SlateRun = FSlateTextRun::Create(RunInfo, InOutModelText, TempStyle, ModelRange);
	}

	return SlateRun.ToSharedRef();
}

TSharedPtr<SWidget> FRichTextDecorator::CreateDecoratorWidget(const FTextRunInfo& RunInfo, const FTextBlockStyle& DefaultTextStyle) const
{
	return TSharedPtr<SWidget>();
}

void FRichTextDecorator::CreateDecoratorText(const FTextRunInfo& RunInfo, FTextBlockStyle& InOutTextStyle, FString& InOutString) const
{
}

/////////////////////////////////////////////////////
// URichTextBlockDecorator

URichTextBlockDecorator::URichTextBlockDecorator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TSharedPtr<ITextDecorator> URichTextBlockDecorator::CreateDecorator(URichTextBlock* InOwner)
{
	return TSharedPtr<ITextDecorator>();
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

