// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RichTextBlockImageDecorator.h"
#include "UObject/SoftObjectPtr.h"
#include "Rendering/DrawElements.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateTextLayout.h"
#include "Slate/SlateGameResources.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Application/SlateApplication.h"
#include "Fonts/FontMeasure.h"
#include "Math/UnrealMathUtility.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SBox.h"
#include "Misc/DefaultValueHelper.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RichTextBlockImageDecorator)

#define LOCTEXT_NAMESPACE "UMG"


class SRichInlineImage : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRichInlineImage)
	{}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const FSlateBrush* Brush, const FTextBlockStyle& TextStyle, TOptional<int32> Width, TOptional<int32> Height, EStretch::Type Stretch)
	{
		check(Brush);

		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		float IconHeight = FMath::Min((float)FontMeasure->GetMaxCharacterHeight(TextStyle.Font, 1.0f), Brush->ImageSize.Y);

		if (Height.IsSet())
		{
			IconHeight = static_cast<float>(Height.GetValue());
		}

		float IconWidth = IconHeight;
		if (Width.IsSet())
		{
			IconWidth = static_cast<float>(Width.GetValue());
		}

		ChildSlot
		[
			SNew(SBox)
			.HeightOverride(IconHeight)
			.WidthOverride(IconWidth)
			[
				SNew(SScaleBox)
				.Stretch(Stretch)
				.StretchDirection(EStretchDirection::DownOnly)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(Brush)
				]
			]
		];
	}
};


/** 
 * Add an image inline with the text.
 * Usage: Before image <img id="MyId"/>, after image.
 * 
 * A width and height can be specified.
 * By default the width and the height is the same size as the font height.
 * Use "desired" to use the same size as the image brush.
 * Usage: Before image <img id="MyId" height="40" width="desired"/>, after image.
 * 
 * A stretch type can be specified. See EStretch.
 * By default the stretch type is ScaleToFit.
 * Usage: Before image <img id="MyId" stretch="ScaleToFitY"/>, after image.
 */
class FRichInlineImage : public FRichTextDecorator
{
public:
	FRichInlineImage(URichTextBlock* InOwner, URichTextBlockImageDecorator* InDecorator)
		: FRichTextDecorator(InOwner)
		, Decorator(InDecorator)
	{
	}

	virtual bool Supports(const FTextRunParseResults& RunParseResult, const FString& Text) const override
	{
		if (RunParseResult.Name == TEXT("img") && RunParseResult.MetaData.Contains(TEXT("id")))
		{
			const FTextRange& IdRange = RunParseResult.MetaData[TEXT("id")];
			const FString TagId = Text.Mid(IdRange.BeginIndex, IdRange.EndIndex - IdRange.BeginIndex);

			const bool bWarnIfMissing = false;
			return Decorator->FindImageBrush(*TagId, bWarnIfMissing) != nullptr;
		}

		return false;
	}

protected:
	virtual TSharedPtr<SWidget> CreateDecoratorWidget(const FTextRunInfo& RunInfo, const FTextBlockStyle& TextStyle) const override
	{
		const bool bWarnIfMissing = true;
		const FSlateBrush* Brush = Decorator->FindImageBrush(*RunInfo.MetaData[TEXT("id")], bWarnIfMissing);

		if (ensure(Brush))
		{
			TOptional<int32> Width;
			if (const FString* WidthString = RunInfo.MetaData.Find(TEXT("width")))
			{
				int32 WidthTemp;
				if (FDefaultValueHelper::ParseInt(*WidthString, WidthTemp))
				{
					Width = WidthTemp;
				}
				else
				{
					if (FCString::Stricmp(GetData(*WidthString), TEXT("desired")) == 0)
					{
						Width = FMath::TruncToInt32(Brush->ImageSize.X);
					}
				}
			}

			TOptional<int32> Height;
			if (const FString* HeightString = RunInfo.MetaData.Find(TEXT("height")))
			{
				int32 HeightTemp;
				if (FDefaultValueHelper::ParseInt(*HeightString, HeightTemp))
				{
					Height = HeightTemp;
				}
				else
				{
					if (FCString::Stricmp(GetData(*HeightString), TEXT("desired")) == 0)
					{
						Height = FMath::TruncToInt32(Brush->ImageSize.Y);
					}
				}
			}

			EStretch::Type Stretch = EStretch::ScaleToFit;
			if (const FString* SstretchString = RunInfo.MetaData.Find(TEXT("stretch")))
			{
				const UEnum* StretchEnum = StaticEnum<EStretch::Type>();
				int64 StretchValue = StretchEnum->GetValueByNameString(*SstretchString);
				if (StretchValue != INDEX_NONE)
				{
					Stretch = static_cast<EStretch::Type>(StretchValue);
				}
			}

			return SNew(SRichInlineImage, Brush, TextStyle, Width, Height, Stretch);
		}
		return TSharedPtr<SWidget>();
	}

private:
	URichTextBlockImageDecorator* Decorator;
};

/////////////////////////////////////////////////////
// URichTextBlockImageDecorator

URichTextBlockImageDecorator::URichTextBlockImageDecorator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FSlateBrush* URichTextBlockImageDecorator::FindImageBrush(FName TagOrId, bool bWarnIfMissing)
{
	const FRichImageRow* ImageRow = FindImageRow(TagOrId, bWarnIfMissing);
	if (ImageRow)
	{
		return &ImageRow->Brush;
	}

	return nullptr;
}

FRichImageRow* URichTextBlockImageDecorator::FindImageRow(FName TagOrId, bool bWarnIfMissing)
{
	if (ImageSet)
	{
		FString ContextString;
		return ImageSet->FindRow<FRichImageRow>(TagOrId, ContextString, bWarnIfMissing);
	}
	
	return nullptr;
}

TSharedPtr<ITextDecorator> URichTextBlockImageDecorator::CreateDecorator(URichTextBlock* InOwner)
{
	return MakeShareable(new FRichInlineImage(InOwner, this));
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

