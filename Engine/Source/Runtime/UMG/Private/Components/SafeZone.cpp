// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SafeZone.h"
#include "SlateFwd.h"

#include "Components/SafeZoneSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SafeZone)

#define LOCTEXT_NAMESPACE "UMG"

USafeZone::USafeZone()
	: PadLeft(true)
	, PadRight(true)
	, PadTop(true)
	, PadBottom(true)
{
	bCanHaveMultipleChildren = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

#if WITH_EDITOR

const FText USafeZone::GetPaletteCategory()
{
	return LOCTEXT( "Panel", "Panel" );
}

void USafeZone::OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs)
{
	if ( EventArgs.bScreenPreview )
	{
		DesignerSize = EventArgs.Size;
	}
	else
	{
		DesignerSize = FVector2D(0, 0);
	}

	DesignerDpi = EventArgs.DpiScale;

	if ( MySafeZone.IsValid() )
	{
		MySafeZone->SetOverrideScreenInformation(DesignerSize, DesignerDpi);
	}
}

#endif

void USafeZone::OnSlotAdded( UPanelSlot* InSlot )
{
	Super::OnSlotAdded( InSlot );

	if (MySafeZone.IsValid())
	{
		CastChecked<USafeZoneSlot>(InSlot)->BuildSlot(MySafeZone.ToSharedRef());
	}
}

void USafeZone::OnSlotRemoved( UPanelSlot* InSlot )
{
	Super::OnSlotRemoved( InSlot );

	if ( MySafeZone.IsValid() )
	{
		MySafeZone->SetContent( SNullWidget::NullWidget );
	}
}

UClass* USafeZone::GetSlotClass() const
{
	return USafeZoneSlot::StaticClass();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void USafeZone::UpdateWidgetProperties()
{
	if ( MySafeZone.IsValid() && GetChildrenCount() > 0 )
	{
		USafeZoneSlot* SafeSlot = CastChecked< USafeZoneSlot >( Slots[ 0 ] );

		MySafeZone->SetSafeAreaScale( SafeSlot->SafeAreaScale );
		MySafeZone->SetTitleSafe( SafeSlot->bIsTitleSafe );
		MySafeZone->SetHAlign( SafeSlot->HAlign.GetValue() );
		MySafeZone->SetVAlign( SafeSlot->VAlign.GetValue() );
		MySafeZone->SetPadding( SafeSlot->Padding );
		MySafeZone->SetSidesToPad( PadLeft, PadRight, PadTop, PadBottom );
	}
}

void USafeZone::SetSidesToPad(bool InPadLeft, bool InPadRight, bool InPadTop, bool InPadBottom)
{
	PadLeft = InPadLeft;
	PadRight = InPadRight;
	PadTop = InPadTop;
	PadBottom = InPadBottom;

	if (MySafeZone.IsValid() && GetChildrenCount() > 0)
	{
		MySafeZone->SetSidesToPad(PadLeft, PadRight, PadTop, PadBottom);
	}
}

TSharedRef<SWidget> USafeZone::RebuildWidget()
{	
	USafeZoneSlot* SafeSlot = nullptr;
	if (GetChildrenCount() > 0)
	{
		SafeSlot = Cast<USafeZoneSlot>(GetContentSlot());;
	}

	MySafeZone = SNew( SSafeZone )
		.IsTitleSafe(SafeSlot ? SafeSlot->bIsTitleSafe : false)
		.SafeAreaScale(SafeSlot ? SafeSlot->SafeAreaScale : FMargin(1, 1, 1, 1))
		.HAlign(SafeSlot ? SafeSlot->HAlign.GetValue() : HAlign_Fill)
		.VAlign(SafeSlot ? SafeSlot->VAlign.GetValue() : VAlign_Fill)
		.Padding(SafeSlot ? SafeSlot->Padding : FMargin())
		.PadLeft( PadLeft )
		.PadRight( PadRight )
		.PadTop( PadTop )
		.PadBottom( PadBottom )
#if WITH_EDITOR
		.OverrideScreenSize(DesignerSize)
		.OverrideDpiScale(DesignerDpi)
#endif
		[
			GetChildAt( 0 ) ? GetChildAt( 0 )->TakeWidget() : SNullWidget::NullWidget
		];

	if (SafeSlot)
	{
		SafeSlot->BuildSlot(MySafeZone.ToSharedRef());
	}

	return MySafeZone.ToSharedRef();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void USafeZone::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MySafeZone.Reset();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void USafeZone::SetPadLeft(bool InPadLeft)
{
	if (PadLeft != InPadLeft)
	{
		SetSidesToPad(InPadLeft, PadRight, PadTop, PadBottom);
	}
}

bool USafeZone::GetPadLeft() const
{
	return PadLeft;
}

void USafeZone::SetPadRight(bool InPadRight)
{
	if (PadRight != InPadRight)
	{
		SetSidesToPad(PadLeft, InPadRight, PadTop, PadBottom);
	}
}

bool USafeZone::GetPadRight() const
{
	return PadRight;
}

void USafeZone::SetPadTop(bool InPadTop)
{
	if (PadTop != InPadTop)
	{
		SetSidesToPad(PadLeft, PadRight, InPadTop, PadBottom);
	}
}

bool USafeZone::GetPadTop() const
{
	return PadTop;
}

void USafeZone::SetPadBottom(bool InPadBottom)
{
	if (PadBottom != InPadBottom)
	{
		SetSidesToPad(PadLeft, PadRight, PadTop, InPadBottom);
	}
}

bool USafeZone::GetPadBottom() const
{
	return PadBottom;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE

