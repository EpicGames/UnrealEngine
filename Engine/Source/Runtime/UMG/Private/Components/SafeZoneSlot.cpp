// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SafeZoneSlot.h"
#include "Widgets/Layout/SSafeZone.h"

#include "Components/SafeZone.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SafeZoneSlot)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USafeZoneSlot::USafeZoneSlot()
{
	bIsTitleSafe = true;
	SafeAreaScale = FMargin(1, 1, 1, 1);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void USafeZoneSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	SafeZone.Reset();
}

void USafeZoneSlot::BuildSlot(TSharedRef<SSafeZone> InSafeZone)
{
	SafeZone = InSafeZone;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InSafeZone->SetTitleSafe(bIsTitleSafe);
	InSafeZone->SetSafeAreaScale(SafeAreaScale);
	InSafeZone->SetHAlign(HAlign.GetValue());
	InSafeZone->SetVAlign(VAlign.GetValue());
	InSafeZone->SetPadding(Padding);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	InSafeZone->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

void USafeZoneSlot::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if ( IsValid( Parent ) )
	{
		CastChecked< USafeZone >( Parent )->UpdateWidgetProperties();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void USafeZoneSlot::SetIsTitleSafe(bool InIsTitleSafe)
{
	bIsTitleSafe = InIsTitleSafe;
	if (SafeZone.IsValid())
	{
		SafeZone.Pin()->SetTitleSafe(InIsTitleSafe);
	}
}

bool USafeZoneSlot::IsTitleSafe() const
{
	return bIsTitleSafe;
}

void USafeZoneSlot::SetSafeAreaScale(const FMargin& InSafeAreaScale)
{
	SafeAreaScale = InSafeAreaScale;
	if (SafeZone.IsValid())
	{
		SafeZone.Pin()->SetSafeAreaScale(SafeAreaScale);
	}
}

const FMargin& USafeZoneSlot::GetSafeAreaScale() const
{
	return SafeAreaScale;
}

void USafeZoneSlot::SetHorizontalAlignment(EHorizontalAlignment InHAlign)
{
	HAlign = InHAlign;
	if (SafeZone.IsValid())
	{
		SafeZone.Pin()->SetHAlign(HAlign);
	}
}

const EHorizontalAlignment USafeZoneSlot::GetHorizontalAlignment() const
{
	return HAlign.GetValue();
}

void USafeZoneSlot::SetVerticalAlignment(EVerticalAlignment InVAlign)
{
	VAlign = InVAlign;
	if (SafeZone.IsValid())
	{
		SafeZone.Pin()->SetVAlign(VAlign);
	}
}

const EVerticalAlignment USafeZoneSlot::GetVerticalAlignment() const
{
	return VAlign.GetValue();
}

void USafeZoneSlot::SetPadding(const FMargin& InPadding)
{
	Padding = InPadding;
	if (SafeZone.IsValid())
	{
		SafeZone.Pin()->SetPadding(InPadding);
	}
}

const FMargin& USafeZoneSlot::GetPadding() const
{
	return Padding;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
