// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ScaleBox.h"
#include "Components/ScaleBoxSlot.h"
#include "UObject/EditorObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScaleBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UScaleBox

UScaleBox::UScaleBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	StretchDirection = EStretchDirection::Both;
	Stretch = EStretch::ScaleToFit;
	UserSpecifiedScale = 1.0f;
	IgnoreInheritedScale = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UScaleBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyScaleBox.Reset();
}

TSharedRef<SWidget> UScaleBox::RebuildWidget()
{
	MyScaleBox = SNew(SScaleBox)
#if WITH_EDITOR
		.OverrideScreenSize(DesignerSize)
#endif
	;

	if ( GetChildrenCount() > 0 )
	{
		CastChecked<UScaleBoxSlot>(GetContentSlot())->BuildSlot(MyScaleBox.ToSharedRef());
	}

	return MyScaleBox.ToSharedRef();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UScaleBox::SetStretch(EStretch::Type InStretch)
{
	Stretch = InStretch;
	if(MyScaleBox.IsValid())
	{
		MyScaleBox->SetStretch(InStretch);
	}
}

EStretch::Type UScaleBox::GetStretch() const
{
	return Stretch;
}

void UScaleBox::SetStretchDirection(EStretchDirection::Type InStretchDirection)
{
	StretchDirection = InStretchDirection;
	if (MyScaleBox.IsValid())
	{
		MyScaleBox->SetStretchDirection(InStretchDirection);
	}
}

EStretchDirection::Type UScaleBox::GetStretchDirection() const
{
	return StretchDirection;
}

void UScaleBox::SetUserSpecifiedScale(float InUserSpecifiedScale)
{
	UserSpecifiedScale = InUserSpecifiedScale;
	if (MyScaleBox.IsValid())
	{
		MyScaleBox->SetUserSpecifiedScale(InUserSpecifiedScale);
	}
}

float UScaleBox::GetUserSpecifiedScale() const
{
	return UserSpecifiedScale;
}

void UScaleBox::SetIgnoreInheritedScale(bool bInIgnoreInheritedScale)
{
	IgnoreInheritedScale = bInIgnoreInheritedScale;
	if (MyScaleBox.IsValid())
	{
		MyScaleBox->SetIgnoreInheritedScale(bInIgnoreInheritedScale);
	}
}

bool UScaleBox::IsIgnoreInheritedScale() const
{
	return IgnoreInheritedScale;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UScaleBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyScaleBox.IsValid())
	{
		return;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyScaleBox->SetStretchDirection(StretchDirection);
	MyScaleBox->SetStretch(Stretch);
	MyScaleBox->SetUserSpecifiedScale(UserSpecifiedScale);
	MyScaleBox->SetIgnoreInheritedScale(IgnoreInheritedScale);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UClass* UScaleBox::GetSlotClass() const
{
	return UScaleBoxSlot::StaticClass();
}

void UScaleBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if ( MyScaleBox.IsValid() )
	{
		CastChecked<UScaleBoxSlot>(InSlot)->BuildSlot(MyScaleBox.ToSharedRef());
	}
}

void UScaleBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyScaleBox.IsValid() )
	{
		MyScaleBox->SetContent(SNullWidget::NullWidget);
	}
}

void UScaleBox::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
}

#if WITH_EDITOR

const FText UScaleBox::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

bool UScaleBox::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UScaleBox, StretchDirection))
		{
			return Stretch != EStretch::None && Stretch != EStretch::ScaleBySafeZone &&
				Stretch != EStretch::UserSpecified && Stretch != EStretch::UserSpecifiedWithClipping;
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UScaleBox, UserSpecifiedScale))
		{
			return Stretch == EStretch::UserSpecified || Stretch == EStretch::UserSpecifiedWithClipping;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return bIsEditable;
}


void UScaleBox::OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs)
{
	if (EventArgs.bScreenPreview)
	{
		DesignerSize = EventArgs.Size;
	}
	else
	{
		DesignerSize = FVector2D(0, 0);
	}

	if (MyScaleBox.IsValid())
	{
		MyScaleBox->SetOverrideScreenInformation(DesignerSize);
	}
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

