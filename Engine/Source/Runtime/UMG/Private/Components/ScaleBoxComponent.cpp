// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ScaleBoxComponent.h"

#include "Blueprint/UserWidget.h"
#include "Components/Widget.h"
#include "CoreGlobals.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SWidget.h"

#include "UMGPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScaleBoxComponent)

void UScaleBoxComponent::SetStretch(EStretch::Type InStretch)
{
	Stretch = InStretch;
	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
	{
		ScaleBox->SetStretch(InStretch);
	}
}

EStretch::Type UScaleBoxComponent::GetStretch() const
{
	return Stretch;
}

void UScaleBoxComponent::SetStretchDirection(EStretchDirection::Type InStretchDirection)
{
	StretchDirection = InStretchDirection;
	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
	{
		ScaleBox->SetStretchDirection(InStretchDirection);
	}
}

EStretchDirection::Type UScaleBoxComponent::GetStretchDirection() const
{
	return StretchDirection;
}

void UScaleBoxComponent::SetUserSpecifiedScale(float InUserSpecifiedScale)
{
	UserSpecifiedScale = InUserSpecifiedScale;
	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
	{
		ScaleBox->SetUserSpecifiedScale(InUserSpecifiedScale);
	}
}

float UScaleBoxComponent::GetUserSpecifiedScale() const
{
	return UserSpecifiedScale;
}

void UScaleBoxComponent::SetIgnoreInheritedScale(bool bInIgnoreInheritedScale)
{
	IgnoreInheritedScale = bInIgnoreInheritedScale;
	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
	{
		ScaleBox->SetIgnoreInheritedScale(bInIgnoreInheritedScale);
	}
}

bool UScaleBoxComponent::IsIgnoreInheritedScale() const
{
	return IgnoreInheritedScale;
}

EHorizontalAlignment UScaleBoxComponent::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UScaleBoxComponent::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
	{
		ScaleBox->SetHAlign(InHorizontalAlignment);
	}
}

EVerticalAlignment UScaleBoxComponent::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UScaleBoxComponent::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
	{
		ScaleBox->SetVAlign(InVerticalAlignment);
	}
}

#if WITH_EDITOR

bool UScaleBoxComponent::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty)
	{
		const FName PropertyName = InProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UScaleBoxComponent, StretchDirection))
		{
			return Stretch != EStretch::None && Stretch != EStretch::ScaleBySafeZone &&
				Stretch != EStretch::UserSpecified && Stretch != EStretch::UserSpecifiedWithClipping;
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UScaleBoxComponent, UserSpecifiedScale))
		{
			return Stretch == EStretch::UserSpecified || Stretch == EStretch::UserSpecifiedWithClipping;
		}
	}

	return bIsEditable;
}

// TODO vinz: Implement OnDesignerChanged to work for Components. This currently is just a virtual function for UWidgets. Note: this function was adapted from the UScaleBox code.
//void UScaleBoxComponent::OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs)
//{
//	if (EventArgs.bScreenPreview)
//	{
//		DesignerSize = EventArgs.Size;
//	}
//	else
//	{
//		DesignerSize = FVector2D(0, 0);
//	}
//
//	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
//	{
//		ScaleBox->SetOverrideScreenInformation(DesignerSize);
//	}
//}

void UScaleBoxComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const TSharedPtr<SScaleBox> ScaleBox = MyScaleBox.Pin())
	{
		SynchronizeProperties(ScaleBox.ToSharedRef());
	}
}
#endif //WITH_EDITOR

void UScaleBoxComponent::SynchronizeProperties(TSharedRef<SScaleBox> ScaleBox)
{	
	ScaleBox->SetStretchDirection(StretchDirection);
	ScaleBox->SetStretch(Stretch);
	ScaleBox->SetUserSpecifiedScale(UserSpecifiedScale);
	ScaleBox->SetIgnoreInheritedScale(IgnoreInheritedScale);

	// Set child slot properties
	ScaleBox->SetHAlign(HorizontalAlignment);
	ScaleBox->SetVAlign(VerticalAlignment);
}

TSharedRef<SWidget> UScaleBoxComponent::RebuildWidgetWithContent(TSharedRef<SWidget> OwnerContent)
{
	TSharedRef<SScaleBox> ScaleBox = SNew(SScaleBox)
// TODO vinz: Implement OnDesignerChanged to work for Components. This currently is just a virtual function for UWidgets. Also, this probably should be a part of SynchronizeProperties.
//#if WITH_EDITOR
//		.OverrideScreenSize(DesignerSize)
//#endif
		[OwnerContent];

	MyScaleBox = ScaleBox.ToWeakPtr();
	SynchronizeProperties(ScaleBox);
	return ScaleBox;
}
