// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ScaleBoxSlot.h"
#include "Widgets/SNullWidget.h"
#include "Components/Widget.h"
#include "Widgets/Layout/SScaleBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScaleBoxSlot)

/////////////////////////////////////////////////////
// UScaleBoxSlot

UScaleBoxSlot::UScaleBoxSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Center;
	VerticalAlignment = VAlign_Center;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UScaleBoxSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	ScaleBox.Reset();
}

void UScaleBoxSlot::BuildSlot(TSharedRef<SScaleBox> InScaleBox)
{
	ScaleBox = InScaleBox;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InScaleBox->SetHAlign(HorizontalAlignment);
	InScaleBox->SetVAlign(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	InScaleBox->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

void UScaleBoxSlot::SetPadding(FMargin InPadding)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EHorizontalAlignment UScaleBoxSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UScaleBoxSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( ScaleBox.IsValid() )
	{
		ScaleBox.Pin()->SetHAlign(InHorizontalAlignment);
	}
}

EVerticalAlignment UScaleBoxSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UScaleBoxSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( ScaleBox.IsValid() )
	{
		ScaleBox.Pin()->SetVAlign(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UScaleBoxSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

