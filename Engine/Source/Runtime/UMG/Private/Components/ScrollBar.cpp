// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ScrollBar.h"
#include "UObject/EditorObjectVersion.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScrollBar)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UScrollBar

UScrollBar::UScrollBar(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bAlwaysShowScrollbar = true;
	bAlwaysShowScrollbarTrack = true;
	Orientation = Orient_Vertical;
	Thickness = FVector2D(16.0f, 16.0f);
	Padding = FMargin(2.0f);

	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBarStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle();
	}
#endif // WITH_EDITOR
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UScrollBar::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyScrollBar.Reset();
}

TSharedRef<SWidget> UScrollBar::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyScrollBar = SNew(SScrollBar)
		.Style(&WidgetStyle)
		.AlwaysShowScrollbar(bAlwaysShowScrollbar)
		.AlwaysShowScrollbarTrack(bAlwaysShowScrollbarTrack)
		.Orientation(Orientation)
		.Thickness(Thickness)
		.Padding(Padding);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//SLATE_EVENT(FOnUserScrolled, OnUserScrolled)

	return MyScrollBar.ToSharedRef();
}

void UScrollBar::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	//MyScrollBar->SetScrollOffset(DesiredScrollOffset);
}

void UScrollBar::SetState(float InOffsetFraction, float InThumbSizeFraction)
{
	if ( MyScrollBar.IsValid() )
	{
		MyScrollBar->SetState(InOffsetFraction, InThumbSizeFraction);
	}
}

#if WITH_EDITORONLY_DATA

void UScrollBar::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	const bool bDeprecateThickness = Ar.IsLoading() && Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::ScrollBarThicknessChange;
	if (bDeprecateThickness)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Set Thickness property to previous default value.
		Thickness.Set(12.0f, 12.0f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	Super::Serialize(Ar);

	if (bDeprecateThickness)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Implicit padding of 2 was removed, so Thickness value must be incremented by 4.
		Thickness += FVector2D(4.0f, 4.0f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

#endif // if WITH_EDITORONLY_DATA

PRAGMA_DISABLE_DEPRECATION_WARNINGS

const FScrollBarStyle& UScrollBar::GetWidgetStyle() const
{
	return WidgetStyle;
}
void UScrollBar::SetWidgetStyle(const FScrollBarStyle& InWidgetStyle)
{
	WidgetStyle = InWidgetStyle;
	if (MyScrollBar.IsValid())
	{
		MyScrollBar->SetStyle(&InWidgetStyle);
	}
}

bool UScrollBar::IsAlwaysShowScrollbar() const
{
	return bAlwaysShowScrollbar;
}

void UScrollBar::SetAlwaysShowScrollbar(bool bNewValue)
{
	bAlwaysShowScrollbar = bNewValue;
	if (MyScrollBar.IsValid())
	{
		MyScrollBar->SetScrollBarAlwaysVisible(bAlwaysShowScrollbar);
	}
}

bool UScrollBar::IsAlwaysShowScrollbarTrack() const
{
	return bAlwaysShowScrollbarTrack;
}

void UScrollBar::SetAlwaysShowScrollbarTrack(bool bNewValue)
{
	bAlwaysShowScrollbarTrack = bNewValue;
	if (MyScrollBar.IsValid())
	{
		MyScrollBar->SetScrollBarTrackAlwaysVisible(bAlwaysShowScrollbarTrack);
	}
}

EOrientation UScrollBar::GetOrientation() const
{
	return Orientation;
}

FVector2D UScrollBar::GetThickness() const
{
	return Thickness;
}

void UScrollBar::SetThickness(const FVector2D& InThickness)
{
	Thickness = InThickness;
	if (MyScrollBar.IsValid())
	{
		MyScrollBar->SetThickness(Thickness);
	}
}

FMargin UScrollBar::GetPadding() const
{
	return Padding;
}

void UScrollBar::SetPadding(const FMargin& InPadding)
{
	Padding = InPadding;
	if (MyScrollBar.IsValid())
	{
		MyScrollBar->SetPadding(InPadding);
	}
}

void UScrollBar::InitOrientation(EOrientation InOrientation)
{
	ensureMsgf(!MyScrollBar.IsValid(), TEXT("The widget is already created."));
	Orientation = InOrientation;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UScrollBar::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

