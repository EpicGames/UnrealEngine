// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ScrollBox.h"
#include "Containers/Ticker.h"
#include "Components/ScrollBoxSlot.h"
#include "UObject/EditorObjectVersion.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScrollBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UScrollBox

UScrollBox::UScrollBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Orientation(Orient_Vertical)
	, ScrollBarVisibility(ESlateVisibility::Visible)
	, ConsumeMouseWheel(EConsumeMouseWheel::WhenScrollingPossible)
	, ScrollbarThickness(9.0f, 9.0f)
	, ScrollbarPadding(2.0f)
	, AlwaysShowScrollbar(false)
	, AlwaysShowScrollbarTrack(false)
	, AllowOverscroll(true)
	, BackPadScrolling(false)
	, FrontPadScrolling(false)
	, NavigationDestination(EDescendantScrollDestination::IntoView)
	, NavigationScrollPadding(0.0f)
	, ScrollWhenFocusChanges(EScrollWhenFocusChanges::NoScroll)
{
	bIsVariable = false;

	SetVisibilityInternal(ESlateVisibility::Visible);
	SetClipping(EWidgetClipping::ClipToBounds);
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBoxStyle();
	WidgetBarStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetScrollBarStyle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR 
	if (IsEditorWidget())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBoxStyle();
		WidgetBarStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetScrollBarStyle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif // WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bAllowRightClickDragScrolling = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UScrollBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyScrollBox.Reset();
}

UClass* UScrollBox::GetSlotClass() const
{
	return UScrollBoxSlot::StaticClass();
}

void UScrollBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyScrollBox.IsValid() )
	{
		CastChecked<UScrollBoxSlot>(InSlot)->BuildSlot(MyScrollBox.ToSharedRef());
	}
}

void UScrollBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyScrollBox.IsValid() && InSlot->Content)
	{
		const TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyScrollBox->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

TSharedRef<SWidget> UScrollBox::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyScrollBox = SNew(SScrollBox)
		.Style(&WidgetStyle)
		.ScrollBarStyle(&WidgetBarStyle)
		.Orientation(Orientation)
		.ConsumeMouseWheel(ConsumeMouseWheel)
		.NavigationDestination(NavigationDestination)
		.NavigationScrollPadding(NavigationScrollPadding)
		.ScrollWhenFocusChanges(ScrollWhenFocusChanges)
		.BackPadScrolling(BackPadScrolling)
		.FrontPadScrolling(FrontPadScrolling)
		.AnimateWheelScrolling(bAnimateWheelScrolling)
		.ScrollAnimationInterpSpeed(ScrollAnimationInterpolationSpeed)
		.WheelScrollMultiplier(WheelScrollMultiplier)
		.EnableTouchScrolling(bEnableTouchScrolling)
		.ConsumePointerInput(bConsumePointerInput)
		.OnUserScrolled(BIND_UOBJECT_DELEGATE(FOnUserScrolled, SlateHandleUserScrolled))
		.OnScrollBarVisibilityChanged(BIND_UOBJECT_DELEGATE(FOnScrollBarVisibilityChanged, SlateHandleScrollBarVisibilityChanged))
		.OnFocusReceived(BIND_UOBJECT_DELEGATE(FOnScrollBoxFocusReceived, SlateHandleFocusReceived))
		.OnFocusLost(BIND_UOBJECT_DELEGATE(FOnScrollBoxFocusLost, SlateHandleFocusLost))
		.OnFocusChanging(BIND_UOBJECT_DELEGATE(FOnScrollBoxFocusChanging, SlateHandleFocusChanging));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UScrollBoxSlot* TypedSlot = Cast<UScrollBoxSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyScrollBox.ToSharedRef());
		}
	}
	
	return MyScrollBox.ToSharedRef();
}

void UScrollBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyScrollBox.IsValid())
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyScrollBox->SetScrollOffset(DesiredScrollOffset);
	MyScrollBox->SetOrientation(Orientation);
	MyScrollBox->SetScrollBarVisibility(UWidget::ConvertSerializedVisibilityToRuntime(ScrollBarVisibility));
	MyScrollBox->SetScrollBarThickness(ScrollbarThickness);
	MyScrollBox->SetScrollBarPadding(ScrollbarPadding);
	MyScrollBox->SetScrollBarAlwaysVisible(AlwaysShowScrollbar);
	MyScrollBox->SetScrollBarTrackAlwaysVisible(AlwaysShowScrollbarTrack);
	MyScrollBox->SetAllowOverscroll(AllowOverscroll ? EAllowOverscroll::Yes : EAllowOverscroll::No);
	MyScrollBox->SetScrollBarRightClickDragAllowed(bAllowRightClickDragScrolling);
	MyScrollBox->SetConsumeMouseWheel(ConsumeMouseWheel);
	MyScrollBox->SetAnalogMouseWheelKey(AnalogMouseWheelKey);
	MyScrollBox->SetIsFocusable(bIsFocusable);
	MyScrollBox->SetAnimateWheelScrolling(bAnimateWheelScrolling);
	MyScrollBox->SetScrollingAnimationInterpolationSpeed(ScrollAnimationInterpolationSpeed);
	MyScrollBox->SetWheelScrollMultiplier(WheelScrollMultiplier);
	MyScrollBox->SetIsTouchScrollingEnabled(bEnableTouchScrolling);
	MyScrollBox->SetConsumePointerInput(bConsumePointerInput);
	MyScrollBox->SetStyle(&WidgetStyle);
	MyScrollBox->InvalidateStyle();
	MyScrollBox->SetScrollBarStyle(&WidgetBarStyle);
	MyScrollBox->InvalidateScrollBarStyle();

PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

float UScrollBox::GetScrollOffset() const
{
	if ( MyScrollBox.IsValid() )
	{
		return MyScrollBox->GetScrollOffset();
	}

	return 0;
}

float UScrollBox::GetOverscrollOffset() const
{
	if ( MyScrollBox.IsValid() )
	{
		return MyScrollBox->GetOverscrollOffset();
	}

	return 0;
}

float UScrollBox::GetOverscrollPercentage() const
{
	if ( MyScrollBox.IsValid() )
	{
		return MyScrollBox->GetOverscrollPercentage();
	}

	return 0;
}

float UScrollBox::GetScrollOffsetOfEnd() const
{
	if (MyScrollBox.IsValid())
	{
		return MyScrollBox->GetScrollOffsetOfEnd();
	}

	return 0;
}

float UScrollBox::GetViewFraction() const
{
	if ( MyScrollBox.IsValid() )
	{
		return MyScrollBox->GetViewFraction();
	}

	return 0;
}

float UScrollBox::GetViewOffsetFraction() const
{
	if ( MyScrollBox.IsValid() )
	{
		return MyScrollBox->GetViewOffsetFraction();
	}

	return 0;
}

void UScrollBox::SetScrollOffset(float NewScrollOffset)
{
	DesiredScrollOffset = NewScrollOffset;

	if ( MyScrollBox.IsValid() )
	{
		MyScrollBox->SetScrollOffset(NewScrollOffset);
	}
}

void UScrollBox::ScrollToStart()
{
	if ( MyScrollBox.IsValid() )
	{
		MyScrollBox->ScrollToStart();
	}
}

void UScrollBox::ScrollToEnd()
{
	if ( MyScrollBox.IsValid() )
	{
		MyScrollBox->ScrollToEnd();
	}
}

void UScrollBox::ScrollWidgetIntoView(UWidget* WidgetToFind, bool AnimateScroll, EDescendantScrollDestination InScrollDestination, float Padding)
{
	TSharedPtr<SWidget> SlateWidgetToFind;
	if (WidgetToFind)
	{
		SlateWidgetToFind = WidgetToFind->GetCachedWidget();
	}

	if (MyScrollBox.IsValid())
	{
		// NOTE: Pass even if null! This, in effect, cancels a request to scroll which is necessary to avoid warnings/ensures 
		//       when we request to scroll to a widget and later remove that widget!
		MyScrollBox->ScrollDescendantIntoView(SlateWidgetToFind, AnimateScroll, InScrollDestination, Padding);
	}
}

bool UScrollBox::GetIsScrolling() const
{
	return MyScrollBox.IsValid() && MyScrollBox->GetIsScrolling();
}

#if WITH_EDITORONLY_DATA

void UScrollBox::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	
	const bool bDeprecateThickness = Ar.IsLoading() && Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::ScrollBarThicknessChange;
	if (bDeprecateThickness)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Set ScrollbarThickness property to previous default value.
		ScrollbarThickness.Set(5.0f, 5.0f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	Super::Serialize(Ar);

	if (bDeprecateThickness)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Implicit padding of 2 was removed, so ScrollbarThickness value must be incremented by 4.
		ScrollbarThickness += FVector2D(4.0f, 4.0f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

#endif // if WITH_EDITORONLY_DATA

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UScrollBox::SetWidgetStyle(const FScrollBoxStyle& NewWidgetStyle)
{
	WidgetStyle = NewWidgetStyle;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->InvalidateStyle();
	}
}

const FScrollBoxStyle& UScrollBox::GetWidgetStyle() const
{
	return WidgetStyle;
}

void UScrollBox::SetWidgetBarStyle(const FScrollBarStyle& NewWidgetBarStyle)
{
	WidgetBarStyle = NewWidgetBarStyle;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->InvalidateScrollBarStyle();
	}
}

const FScrollBarStyle& UScrollBox::GetWidgetBarStyle() const
{
	return WidgetBarStyle;
}

void UScrollBox::SetNavigationDestination(const EDescendantScrollDestination NewNavigationDestination)
{
	NavigationDestination = NewNavigationDestination;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetNavigationDestination(NewNavigationDestination);
	}
}

EDescendantScrollDestination UScrollBox::GetNavigationDestination() const
{
	return NavigationDestination;
}

void UScrollBox::SetConsumeMouseWheel(EConsumeMouseWheel NewConsumeMouseWheel)
{
	ConsumeMouseWheel = NewConsumeMouseWheel;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetConsumeMouseWheel(NewConsumeMouseWheel);
	}
}

EConsumeMouseWheel UScrollBox::GetConsumeMouseWheel() const
{
	return ConsumeMouseWheel;
}

void UScrollBox::SetOrientation(EOrientation NewOrientation)
{
	Orientation = NewOrientation;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetOrientation(Orientation);
	}
}

EOrientation UScrollBox::GetOrientation() const
{
	return Orientation;
}

void UScrollBox::SetScrollBarVisibility(ESlateVisibility NewScrollBarVisibility)
{
	ScrollBarVisibility = NewScrollBarVisibility;

	if (MyScrollBox.IsValid())
	{
		switch (ScrollBarVisibility)
		{
			case ESlateVisibility::Collapsed:				MyScrollBox->SetScrollBarVisibility(EVisibility::Collapsed); break;
			case ESlateVisibility::Hidden:					MyScrollBox->SetScrollBarVisibility(EVisibility::Hidden); break;
			case ESlateVisibility::HitTestInvisible:		MyScrollBox->SetScrollBarVisibility(EVisibility::HitTestInvisible); break;
			case ESlateVisibility::SelfHitTestInvisible:	MyScrollBox->SetScrollBarVisibility(EVisibility::SelfHitTestInvisible); break;
			case ESlateVisibility::Visible:					MyScrollBox->SetScrollBarVisibility(EVisibility::Visible); break;
		}
	}
}

ESlateVisibility UScrollBox::GetScrollBarVisibility() const
{
	return ScrollBarVisibility;
}

void UScrollBox::SetScrollbarThickness(const FVector2D& NewScrollbarThickness)
{
	ScrollbarThickness = NewScrollbarThickness;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetScrollBarThickness(ScrollbarThickness);
	}
}

FVector2D UScrollBox::GetScrollbarThickness() const
{
	return ScrollbarThickness;
}

void UScrollBox::SetScrollbarPadding(const FMargin& NewScrollbarPadding)
{
	ScrollbarPadding = NewScrollbarPadding;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetScrollBarPadding(ScrollbarPadding);
	}
}

FMargin UScrollBox::GetScrollbarPadding() const
{
	return ScrollbarPadding;
}

void UScrollBox::SetAlwaysShowScrollbar(bool NewAlwaysShowScrollbar)
{
	AlwaysShowScrollbar = NewAlwaysShowScrollbar;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetScrollBarAlwaysVisible(AlwaysShowScrollbar);
	}
}

bool UScrollBox::IsAlwaysShowScrollbar() const
{
	return AlwaysShowScrollbar;
}

void UScrollBox::SetAllowOverscroll(bool NewAllowOverscroll)
{
	AllowOverscroll = NewAllowOverscroll;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetAllowOverscroll(AllowOverscroll ? EAllowOverscroll::Yes : EAllowOverscroll::No);
	}
}

bool UScrollBox::IsAllowOverscroll() const
{
	return AllowOverscroll;
}

void UScrollBox::SetAnimateWheelScrolling(bool bShouldAnimateWheelScrolling)
{
	bAnimateWheelScrolling = bShouldAnimateWheelScrolling;
	if (MyScrollBox)
	{
		MyScrollBox->SetAnimateWheelScrolling(bShouldAnimateWheelScrolling);
	}
}

bool UScrollBox::IsAnimateWheelScrolling() const
{
	return bAnimateWheelScrolling;
}

void UScrollBox::SetScrollAnimationInterpolationSpeed(float NewScrollAnimationInterpolationSpeed)
{
	ScrollAnimationInterpolationSpeed = NewScrollAnimationInterpolationSpeed;
	if (MyScrollBox)
	{
		MyScrollBox->SetScrollingAnimationInterpolationSpeed(ScrollAnimationInterpolationSpeed);
	}
}

float UScrollBox::GetScrollAnimationInterpolationSpeed() const
{
	return ScrollAnimationInterpolationSpeed;
}

void UScrollBox::SetWheelScrollMultiplier(float NewWheelScrollMultiplier)
{
	WheelScrollMultiplier = NewWheelScrollMultiplier;
	if (MyScrollBox)
	{
		MyScrollBox->SetWheelScrollMultiplier(NewWheelScrollMultiplier);
	}
}

float UScrollBox::GetWheelScrollMultiplier() const
{
	return WheelScrollMultiplier;
}

void UScrollBox::SetIsTouchScrollingEnabled(bool bInEnableTouchScrolling)
{
	bEnableTouchScrolling = bInEnableTouchScrolling;
	if (MyScrollBox)
	{
		MyScrollBox->SetIsTouchScrollingEnabled(bInEnableTouchScrolling);
	}
}

bool UScrollBox::GetIsTouchScrollingEnabled() const
{
	return bEnableTouchScrolling;
}

void UScrollBox::SetConsumePointerInput(bool bInConsumePointerInput)
{
	bConsumePointerInput = bInConsumePointerInput;

	if (MyScrollBox.IsValid())
	{
		MyScrollBox->SetConsumePointerInput(bInConsumePointerInput);
	}
}

bool UScrollBox::GetConsumePointerInput() const
{
	return bConsumePointerInput;
}

void UScrollBox::SetScrollWhenFocusChanges(EScrollWhenFocusChanges NewScrollWhenFocusChanges)
{
	ScrollWhenFocusChanges = NewScrollWhenFocusChanges;
	if (MyScrollBox)
	{
		MyScrollBox->SetScrollWhenFocusChanges(NewScrollWhenFocusChanges);
	}
}

EScrollWhenFocusChanges UScrollBox::GetScrollWhenFocusChanges() const
{
	return ScrollWhenFocusChanges;
}

void UScrollBox::EndInertialScrolling()
{
	if (MyScrollBox.IsValid())
	{
		MyScrollBox->EndInertialScrolling();
	}
}

FKey UScrollBox::GetAnalogMouseWheelKey() const
{
	return AnalogMouseWheelKey;
}

void UScrollBox::SetAnalogMouseWheelKey(FKey InMouseWheelKey)
{
	AnalogMouseWheelKey = InMouseWheelKey;
	if (MyScrollBox)
	{
		MyScrollBox->SetAnalogMouseWheelKey(AnalogMouseWheelKey);
	}
}

bool UScrollBox::GetIsFocusable() const
{
	return bIsFocusable;
}

void UScrollBox::SetIsFocusable(bool bInIsFocusable)
{	
	bIsFocusable = bInIsFocusable;
	if (MyScrollBox)
	{
		MyScrollBox->SetIsFocusable(bIsFocusable);
	}
}

void UScrollBox::SetAlwaysShowScrollbarTrack(bool NewAlwaysShowScrollbarTrack)
{
	AlwaysShowScrollbarTrack = NewAlwaysShowScrollbarTrack;
	if (MyScrollBox)
	{
		MyScrollBox->SetScrollBarTrackAlwaysVisible(AlwaysShowScrollbarTrack);
	}
}

bool UScrollBox::IsAlwaysShowScrollbarTrack() const
{
	return AlwaysShowScrollbarTrack;
}

float UScrollBox::GetNavigationScrollPadding() const
{
	return NavigationScrollPadding;
}

void UScrollBox::SetAllowRightClickDragScrolling(bool bShouldAllowRightClickDragScrolling)
{
	bAllowRightClickDragScrolling = bShouldAllowRightClickDragScrolling;
	if (MyScrollBox)
	{
		MyScrollBox->SetScrollBarRightClickDragAllowed(bAllowRightClickDragScrolling);
	}
}

bool UScrollBox::IsAllowRightClickDragScrolling() const
{
	return bAllowRightClickDragScrolling;
}

bool UScrollBox::IsFrontPadScrolling() const
{
	return FrontPadScrolling;
}

bool UScrollBox::IsBackPadScrolling() const
{
	return BackPadScrolling;
}

void UScrollBox::InitBackPadScrolling(bool InBackPadScrolling)
{
	ensureMsgf(!MyScrollBox.IsValid(), TEXT("The widget is already created."));
	BackPadScrolling = InBackPadScrolling;
}

void UScrollBox::InitFrontPadScrolling(bool InFrontPadScrolling)
{
	ensureMsgf(!MyScrollBox.IsValid(), TEXT("The widget is already created."));
	FrontPadScrolling = InFrontPadScrolling;
}

void UScrollBox::InitNavigationScrollPadding(float InNavigationScrollPadding)
{
	ensureMsgf(!MyScrollBox.IsValid(), TEXT("The widget is already created."));
	NavigationScrollPadding = InNavigationScrollPadding;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UScrollBox::SlateHandleUserScrolled(float CurrentOffset)
{
	OnUserScrolled.Broadcast(CurrentOffset);
}

void UScrollBox::SlateHandleScrollBarVisibilityChanged(EVisibility NewVisibility)
{
	OnScrollBarVisibilityChanged.Broadcast(ConvertRuntimeToSerializedVisibility(NewVisibility));
}

void UScrollBox::SlateHandleFocusReceived()
{
	OnFocusReceived.Broadcast();
}

void UScrollBox::SlateHandleFocusLost()
{
	OnFocusLost.Broadcast();
}

UMG_API void UScrollBox::SlateHandleFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath)
{
	if (const SWidget* SlateWidget = this->GetCachedWidget().Get())
	{
		const bool bNewFocusIsInScrollBox = NewWidgetPath.ContainsWidget(SlateWidget);
		const bool bOldFocusIsInScrollBox = PreviousFocusPath.ContainsWidget(SlateWidget);
		if (!bNewFocusIsInScrollBox || !bOldFocusIsInScrollBox)
		{
			OnFocusUpdated.Broadcast(this, !bOldFocusIsInScrollBox);
		}
	}
}

#if WITH_EDITOR

const FText UScrollBox::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

void UScrollBox::OnDescendantSelectedByDesigner( UWidget* DescendantWidget )
{
	UWidget* SelectedChild = UWidget::FindChildContainingDescendant( this, DescendantWidget );
	if ( SelectedChild )
	{
		ScrollWidgetIntoView( SelectedChild, true );

		if ( TickHandle.IsValid() )
		{
			FTSTicker::RemoveTicker( TickHandle );
			TickHandle.Reset();
		}
	}
}

void UScrollBox::OnDescendantDeselectedByDesigner( UWidget* DescendantWidget )
{
	if ( TickHandle.IsValid() )
	{
		FTSTicker::RemoveTicker( TickHandle );
		TickHandle.Reset();
	}

	// because we get a deselect before we get a select, we need to delay this call until we're sure we didn't scroll to another widget.
	TickHandle = FTSTicker::GetCoreTicker().AddTicker( FTickerDelegate::CreateWeakLambda(this, [this]( float ) -> bool
	                                                                                {
                                                                                        QUICK_SCOPE_CYCLE_COUNTER(STAT_UScrollBox_ScrollToStart_LambdaTick);
		                                                                                this->ScrollToStart();
		                                                                                return false;
		                                                                            } ) );
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

