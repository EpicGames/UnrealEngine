// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ScrollBoxSlot.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScrollBoxSlot)

/////////////////////////////////////////////////////
// UScrollBoxSlot

UScrollBoxSlot::UScrollBoxSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	Size = FSlateChildSize(ESlateSizeRule::Automatic);
}

void UScrollBoxSlot::BuildSlot(TSharedRef<SScrollBox> ScrollBox)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ScrollBox->AddSlot()
		.Padding(Padding)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		.Expose(Slot)
		.SizeParam(UWidget::ConvertSerializedSizeParamToRuntime(Size))
	[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UScrollBoxSlot::GetPadding() const
{
	return Slot ? Slot->GetPadding() : Padding;
}

void UScrollBoxSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Slot )
	{
		Slot->SetPadding(InPadding);
	}
}

FSlateChildSize UScrollBoxSlot::GetSize() const
{
	return Size;
}

void UScrollBoxSlot::SetSize(FSlateChildSize InSize)
{
	Size = InSize;
	if (Slot)
	{
		Slot->SetSizeParam(UWidget::ConvertSerializedSizeParamToRuntime(InSize));
	}
}

EHorizontalAlignment UScrollBoxSlot::GetHorizontalAlignment() const
{
	return Slot ? Slot->GetHorizontalAlignment() : HorizontalAlignment.GetValue();
}

void UScrollBoxSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UScrollBoxSlot::GetVerticalAlignment() const
{
	return Slot ? Slot->GetVerticalAlignment() : VerticalAlignment.GetValue();
}

void UScrollBoxSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if (Slot)
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UScrollBoxSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetPadding(Padding);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	SetSize(Size);
}

void UScrollBoxSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	Slot = nullptr;
}

