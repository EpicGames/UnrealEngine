// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SizeBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Components/SizeBoxSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SizeBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// USizeBox

USizeBox::USizeBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MinAspectRatio = 1.f;
	MaxAspectRatio = 1.f;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USizeBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MySizeBox.Reset();
}

TSharedRef<SWidget> USizeBox::RebuildWidget()
{
	MySizeBox = SNew(SBox);

	if (GetChildrenCount() > 0)
	{
		Cast<USizeBoxSlot>(GetContentSlot())->BuildSlot(MySizeBox.ToSharedRef());
	}

	return MySizeBox.ToSharedRef();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float USizeBox::GetWidthOverride() const
{
	return WidthOverride;
}

bool USizeBox::IsWidthOverride() const
{
	return bOverride_WidthOverride;
}

void USizeBox::SetWidthOverride(float InWidthOverride)
{
	bOverride_WidthOverride = true;
	WidthOverride = InWidthOverride;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetWidthOverride(InWidthOverride);
	}
}

void USizeBox::ClearWidthOverride()
{
	bOverride_WidthOverride = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetWidthOverride(FOptionalSize());
	}
}

float USizeBox::GetHeightOverride() const
{
	return HeightOverride;
}

bool USizeBox::IsHeightOverride() const
{
	return bOverride_HeightOverride;
}

void USizeBox::SetHeightOverride(float InHeightOverride)
{
	bOverride_HeightOverride = true;
	HeightOverride = InHeightOverride;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetHeightOverride(InHeightOverride);
	}
}

void USizeBox::ClearHeightOverride()
{
	bOverride_HeightOverride = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetHeightOverride(FOptionalSize());
	}
}

float USizeBox::GetMinDesiredWidth() const
{
	return MinDesiredWidth;
}

bool USizeBox::IsMinDesiredWidthOverride() const
{
	return bOverride_MinDesiredWidth;
}

void USizeBox::SetMinDesiredWidth(float InMinDesiredWidth)
{
	bOverride_MinDesiredWidth = true;
	MinDesiredWidth = InMinDesiredWidth;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMinDesiredWidth(InMinDesiredWidth);
	}
}

void USizeBox::ClearMinDesiredWidth()
{
	bOverride_MinDesiredWidth = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMinDesiredWidth(FOptionalSize());
	}
}

float USizeBox::GetMinDesiredHeight() const
{
	return MinDesiredHeight;
}

bool USizeBox::IsMinDesiredHeightOverride() const
{
	return bOverride_MinDesiredHeight;
}

void USizeBox::SetMinDesiredHeight(float InMinDesiredHeight)
{
	bOverride_MinDesiredHeight = true;
	MinDesiredHeight = InMinDesiredHeight;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMinDesiredHeight(InMinDesiredHeight);
	}
}

void USizeBox::ClearMinDesiredHeight()
{
	bOverride_MinDesiredHeight = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMinDesiredHeight(FOptionalSize());
	}
}

float USizeBox::GetMaxDesiredWidth() const
{
	return MaxDesiredWidth;
}

bool USizeBox::IsMaxDesiredWidthOverride() const
{
	return bOverride_MaxDesiredWidth;
}

void USizeBox::SetMaxDesiredWidth(float InMaxDesiredWidth)
{
	bOverride_MaxDesiredWidth = true;
	MaxDesiredWidth = InMaxDesiredWidth;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMaxDesiredWidth(InMaxDesiredWidth);
	}
}

void USizeBox::ClearMaxDesiredWidth()
{
	bOverride_MaxDesiredWidth = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMaxDesiredWidth(FOptionalSize());
	}
}

float USizeBox::GetMaxDesiredHeight() const
{
	return MaxDesiredHeight;
}

bool USizeBox::IsMaxDesiredHeightOverride() const
{
	return bOverride_MaxDesiredHeight;
}

void USizeBox::SetMaxDesiredHeight(float InMaxDesiredHeight)
{
	bOverride_MaxDesiredHeight = true;
	MaxDesiredHeight = InMaxDesiredHeight;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMaxDesiredHeight(InMaxDesiredHeight);
	}
}

void USizeBox::ClearMaxDesiredHeight()
{
	bOverride_MaxDesiredHeight = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMaxDesiredHeight(FOptionalSize());
	}
}

float USizeBox::GetMinAspectRatio() const
{
	return MinAspectRatio;
}

bool USizeBox::IsMinAspectRatioOverride() const
{
	return bOverride_MinAspectRatio;
}

void USizeBox::SetMinAspectRatio(float InMinAspectRatio)
{
	bOverride_MinAspectRatio = true;
	MinAspectRatio = InMinAspectRatio;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMinAspectRatio(InMinAspectRatio);
	}
}

void USizeBox::ClearMinAspectRatio()
{
	bOverride_MinAspectRatio = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMinAspectRatio(FOptionalSize());
	}
}

float USizeBox::GetMaxAspectRatio() const
{
	return MaxAspectRatio;
}

bool USizeBox::IsMaxAspectRatioOverride() const
{
	return bOverride_MaxAspectRatio;
}

void USizeBox::SetMaxAspectRatio(float InMaxAspectRatio)
{
	bOverride_MaxAspectRatio = true;
	MaxAspectRatio = InMaxAspectRatio;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMaxAspectRatio(InMaxAspectRatio);
	}
}

void USizeBox::ClearMaxAspectRatio()
{
	bOverride_MaxAspectRatio = false;
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetMaxAspectRatio(FOptionalSize());
	}
}

void USizeBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (bOverride_WidthOverride)
	{
		SetWidthOverride(WidthOverride);
	}
	else
	{
		ClearWidthOverride();
	}

	if (bOverride_HeightOverride)
	{
		SetHeightOverride(HeightOverride);
	}
	else
	{
		ClearHeightOverride();
	}

	if (bOverride_MinDesiredWidth)
	{
		SetMinDesiredWidth(MinDesiredWidth);
	}
	else
	{
		ClearMinDesiredWidth();
	}

	if (bOverride_MinDesiredHeight)
	{
		SetMinDesiredHeight(MinDesiredHeight);
	}
	else
	{
		ClearMinDesiredHeight();
	}

	if (bOverride_MaxDesiredWidth)
	{
		SetMaxDesiredWidth(MaxDesiredWidth);
	}
	else
	{
		ClearMaxDesiredWidth();
	}

	if (bOverride_MaxDesiredHeight)
	{
		SetMaxDesiredHeight(MaxDesiredHeight);
	}
	else
	{
		ClearMaxDesiredHeight();
	}

	if (bOverride_MinAspectRatio)
	{
		SetMinAspectRatio(MinAspectRatio);
	}
	else
	{
		ClearMinAspectRatio();
	}

	if (bOverride_MaxAspectRatio)
	{
		SetMaxAspectRatio(MaxAspectRatio);
	}
	else
	{
		ClearMaxAspectRatio();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UClass* USizeBox::GetSlotClass() const
{
	return USizeBoxSlot::StaticClass();
}

void USizeBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if (MySizeBox.IsValid())
	{
		CastChecked<USizeBoxSlot>(InSlot)->BuildSlot(MySizeBox.ToSharedRef());
	}
}

void USizeBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if (MySizeBox.IsValid())
	{
		MySizeBox->SetContent(SNullWidget::NullWidget);
	}
}

#if WITH_EDITOR

const FText USizeBox::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
