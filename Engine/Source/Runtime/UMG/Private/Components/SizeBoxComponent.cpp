// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SizeBoxComponent.h"

#include "Blueprint/UserWidget.h"
#include "Components/Widget.h"
#include "CoreGlobals.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWidget.h"

#include "UMGPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SizeBoxComponent)

float USizeBoxComponent::GetWidthOverride() const
{
	return WidthOverride;
}

bool USizeBoxComponent::IsWidthOverride() const
{
	return bOverride_WidthOverride;
}

void USizeBoxComponent::SetWidthOverride(float InWidthOverride)
{
	bOverride_WidthOverride = true;
	WidthOverride = InWidthOverride;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetWidthOverride(InWidthOverride);
	}
}

void USizeBoxComponent::ClearWidthOverride()
{
	bOverride_WidthOverride = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetWidthOverride(FOptionalSize());
	}
}

float USizeBoxComponent::GetHeightOverride() const
{
	return HeightOverride;
}

bool USizeBoxComponent::IsHeightOverride() const
{
	return bOverride_HeightOverride;
}

void USizeBoxComponent::SetHeightOverride(float InHeightOverride)
{
	bOverride_HeightOverride = true;
	HeightOverride = InHeightOverride;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetHeightOverride(InHeightOverride);
	}
}

void USizeBoxComponent::ClearHeightOverride()
{
	bOverride_HeightOverride = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetHeightOverride(FOptionalSize());
	}
}

float USizeBoxComponent::GetMinDesiredWidth() const
{
	return MinDesiredWidth;
}

bool USizeBoxComponent::IsMinDesiredWidthOverride() const
{
	return bOverride_MinDesiredWidth;
}

void USizeBoxComponent::SetMinDesiredWidth(float InMinDesiredWidth)
{
	bOverride_MinDesiredWidth = true;
	MinDesiredWidth = InMinDesiredWidth;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMinDesiredWidth(InMinDesiredWidth);
	}
}

void USizeBoxComponent::ClearMinDesiredWidth()
{
	bOverride_MinDesiredWidth = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMinDesiredWidth(FOptionalSize());
	}
}

float USizeBoxComponent::GetMinDesiredHeight() const
{
	return MinDesiredHeight;
}

bool USizeBoxComponent::IsMinDesiredHeightOverride() const
{
	return bOverride_MinDesiredHeight;
}

void USizeBoxComponent::SetMinDesiredHeight(float InMinDesiredHeight)
{
	bOverride_MinDesiredHeight = true;
	MinDesiredHeight = InMinDesiredHeight;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMinDesiredHeight(InMinDesiredHeight);
	}
}

void USizeBoxComponent::ClearMinDesiredHeight()
{
	bOverride_MinDesiredHeight = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMinDesiredHeight(FOptionalSize());
	}
}

float USizeBoxComponent::GetMaxDesiredWidth() const
{
	return MaxDesiredWidth;
}

bool USizeBoxComponent::IsMaxDesiredWidthOverride() const
{
	return bOverride_MaxDesiredWidth;
}

void USizeBoxComponent::SetMaxDesiredWidth(float InMaxDesiredWidth)
{
	bOverride_MaxDesiredWidth = true;
	MaxDesiredWidth = InMaxDesiredWidth;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMaxDesiredWidth(InMaxDesiredWidth);
	}
}

void USizeBoxComponent::ClearMaxDesiredWidth()
{
	bOverride_MaxDesiredWidth = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMaxDesiredWidth(FOptionalSize());
	}
}

float USizeBoxComponent::GetMaxDesiredHeight() const
{
	return MaxDesiredHeight;
}

bool USizeBoxComponent::IsMaxDesiredHeightOverride() const
{
	return bOverride_MaxDesiredHeight;
}

void USizeBoxComponent::SetMaxDesiredHeight(float InMaxDesiredHeight)
{
	bOverride_MaxDesiredHeight = true;
	MaxDesiredHeight = InMaxDesiredHeight;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMaxDesiredHeight(InMaxDesiredHeight);
	}
}

void USizeBoxComponent::ClearMaxDesiredHeight()
{
	bOverride_MaxDesiredHeight = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMaxDesiredHeight(FOptionalSize());
	}
}

float USizeBoxComponent::GetMinAspectRatio() const
{
	return MinAspectRatio;
}

bool USizeBoxComponent::IsMinAspectRatioOverride() const
{
	return bOverride_MinAspectRatio;
}

void USizeBoxComponent::SetMinAspectRatio(float InMinAspectRatio)
{
	bOverride_MinAspectRatio = true;
	MinAspectRatio = InMinAspectRatio;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMinAspectRatio(InMinAspectRatio);
	}
}

void USizeBoxComponent::ClearMinAspectRatio()
{
	bOverride_MinAspectRatio = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMinAspectRatio(FOptionalSize());
	}
}

float USizeBoxComponent::GetMaxAspectRatio() const
{
	return MaxAspectRatio;
}

bool USizeBoxComponent::IsMaxAspectRatioOverride() const
{
	return bOverride_MaxAspectRatio;
}

void USizeBoxComponent::SetMaxAspectRatio(float InMaxAspectRatio)
{
	bOverride_MaxAspectRatio = true;
	MaxAspectRatio = InMaxAspectRatio;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMaxAspectRatio(InMaxAspectRatio);
	}
}

void USizeBoxComponent::ClearMaxAspectRatio()
{
	bOverride_MaxAspectRatio = false;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetMaxAspectRatio(FOptionalSize());
	}
}

FMargin USizeBoxComponent::GetPadding() const
{
	return Padding;
}

void USizeBoxComponent::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetPadding(InPadding);
	}
}

EHorizontalAlignment USizeBoxComponent::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void USizeBoxComponent::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetHAlign(InHorizontalAlignment);
	}
}

EVerticalAlignment USizeBoxComponent::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void USizeBoxComponent::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SizeBox->SetVAlign(InVerticalAlignment);
	}
}

#if WITH_EDITOR
void USizeBoxComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	
	if (const TSharedPtr<SBox> SizeBox = MySizeBox.Pin())
	{
		SynchronizeProperties(SizeBox.ToSharedRef());
	}
}
#endif //WITH_EDITOR

void USizeBoxComponent::SynchronizeProperties(TSharedRef<SBox> SizeBox)
{
	SizeBox->SetWidthOverride(bOverride_WidthOverride? WidthOverride : FOptionalSize());
	SizeBox->SetHeightOverride(bOverride_HeightOverride ? HeightOverride : FOptionalSize());
	SizeBox->SetMinDesiredWidth(bOverride_MinDesiredWidth? MinDesiredWidth : FOptionalSize());
	SizeBox->SetMinDesiredHeight(bOverride_MinDesiredHeight ? MinDesiredHeight : FOptionalSize());
	SizeBox->SetMaxDesiredWidth(bOverride_MaxDesiredWidth ? MaxDesiredWidth : FOptionalSize());
	SizeBox->SetMaxDesiredHeight(bOverride_MaxDesiredHeight ? MaxDesiredHeight : FOptionalSize());
	SizeBox->SetMinAspectRatio(bOverride_MinAspectRatio ? MinAspectRatio : FOptionalSize());
	SizeBox->SetMaxAspectRatio(bOverride_MaxAspectRatio ? MaxAspectRatio : FOptionalSize());
	
	// Set child slot properties
	SizeBox->SetPadding(Padding);
	SizeBox->SetHAlign(HorizontalAlignment);
	SizeBox->SetVAlign(VerticalAlignment);
}

TSharedRef<SWidget> USizeBoxComponent::RebuildWidgetWithContent(TSharedRef<SWidget> OwnerContent)
{
	TSharedRef<SBox> SizeBox = SNew(SBox)
		[OwnerContent];

	MySizeBox = SizeBox.ToWeakPtr();
	SynchronizeProperties(SizeBox);
	return SizeBox;
}

