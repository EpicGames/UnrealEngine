// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SizeBoxSlot.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SizeBoxSlot)

/////////////////////////////////////////////////////
// USizeBoxSlot

USizeBoxSlot::USizeBoxSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Padding = FMargin(0.f, 0.f);

	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USizeBoxSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	SizeBox.Reset();
}

void USizeBoxSlot::BuildSlot(TSharedRef<SBox> InSizeBox)
{
	SizeBox = InSizeBox;

	SynchronizeProperties();

	SizeBox.Pin()->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin USizeBoxSlot::GetPadding() const
{
	return Padding;
}

void USizeBoxSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( SizeBox.IsValid() )
	{
		SizeBox.Pin()->SetPadding(InPadding);
	}
}

EHorizontalAlignment USizeBoxSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void USizeBoxSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( SizeBox.IsValid() )
	{
		SizeBox.Pin()->SetHAlign(InHorizontalAlignment);
	}
}

EVerticalAlignment USizeBoxSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void USizeBoxSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( SizeBox.IsValid() )
	{
		SizeBox.Pin()->SetVAlign(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void USizeBoxSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetPadding(Padding);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

