// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Slider.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSlider.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Slider)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// USlider

USlider::USlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MinValue = 0.0f;
	MaxValue = 1.0f;
	Orientation = EOrientation::Orient_Horizontal;
	SliderBarColor = FLinearColor::White;
	SliderHandleColor = FLinearColor::White;
	StepSize = 0.01f;
	IsFocusable = true;
	MouseUsesStep = false;
	RequiresControllerLock = true;

	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetSliderStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetSliderStyle();
	}
#endif // WITH_EDITOR

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

TSharedRef<SWidget> USlider::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MySlider = SNew(SSlider)
		.Style(&WidgetStyle)
		.IsFocusable(IsFocusable)
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd))
		.OnControllerCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureBegin))
		.OnControllerCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureEnd))
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.PreventThrottling(bPreventThrottling);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return MySlider.ToSharedRef();
}

void USlider::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MySlider.IsValid())
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TAttribute<float> ValueBinding = PROPERTY_BINDING(float, Value);
	
	MySlider->SetOrientation(Orientation);
	MySlider->SetMouseUsesStep(MouseUsesStep);
	MySlider->SetRequiresControllerLock(RequiresControllerLock);
	MySlider->SetSliderBarColor(SliderBarColor);
	MySlider->SetSliderHandleColor(SliderHandleColor);
	MySlider->SetValue(ValueBinding);
	MySlider->SetMinAndMaxValues(MinValue, MaxValue);
	MySlider->SetLocked(Locked);
	MySlider->SetIndentHandle(IndentHandle);
	MySlider->SetStepSize(StepSize);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USlider::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MySlider.Reset();
}

void USlider::HandleOnValueChanged(float InValue)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Value = InValue;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	OnValueChanged.Broadcast(InValue);
	BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Value);
}

void USlider::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void USlider::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

void USlider::HandleOnControllerCaptureBegin()
{
	OnControllerCaptureBegin.Broadcast();
}

void USlider::HandleOnControllerCaptureEnd()
{
	OnControllerCaptureEnd.Broadcast();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float USlider::GetValue() const
{
	if ( MySlider.IsValid() )
	{
		return MySlider->GetValue();
	}

	return Value;
}

float USlider::GetNormalizedValue() const
{
	if (MySlider.IsValid())
	{
		return MySlider->GetNormalizedValue();
	}

	if (MinValue == MaxValue)
	{
		return 1.0f;
	}
	else
	{
		return (Value - MinValue) / (MaxValue - MinValue);
	}
}

void USlider::SetValue(float InValue)
{
	if (MySlider.IsValid())
	{
		MySlider->SetValue(InValue);
	}

	if (Value != InValue)
	{
		Value = InValue;
		HandleOnValueChanged(InValue);
	}
}

float USlider::GetMinValue() const
{
	if (MySlider.IsValid())
	{
		return MySlider->GetMinValue();
	}
	return MinValue;
}

void USlider::SetMinValue(float InValue)
{
	MinValue = InValue;
	if (MySlider.IsValid())
	{
		// Because SSlider clamps min/max values upon setting them,
		// we have to send both values together to ensure that they
		// don't get out of sync.
		MySlider->SetMinAndMaxValues(MinValue, MaxValue);
	}
}

float USlider::GetMaxValue() const
{
	if (MySlider.IsValid())
	{
		return MySlider->GetMaxValue();
	}
	return MaxValue;
}

void USlider::SetMaxValue(float InValue)
{
	MaxValue = InValue;
	if (MySlider.IsValid())
	{
		MySlider->SetMinAndMaxValues(MinValue, MaxValue);
	}
}

const FSliderStyle& USlider::GetWidgetStyle() const
{
	return WidgetStyle;
}

void USlider::SetWidgetStyle(const FSliderStyle& InStyle)
{
	WidgetStyle = InStyle;
	if (MySlider.IsValid())
	{
		MySlider->SetStyle(&WidgetStyle);
	}
}

EOrientation USlider::GetOrientation() const
{
	return Orientation;
}

void USlider::SetOrientation(EOrientation InOrientation)
{
	Orientation = InOrientation;
	if (MySlider.IsValid())
	{
		MySlider->SetOrientation(Orientation);
	}
}

bool USlider::HasIndentHandle() const
{
	return IndentHandle;
}

void USlider::SetIndentHandle(bool InIndentHandle)
{
	IndentHandle = InIndentHandle;
	if ( MySlider.IsValid() )
	{
		MySlider->SetIndentHandle(InIndentHandle);
	}
}

bool USlider::IsLocked() const
{
	return Locked;
}

void USlider::SetLocked(bool InLocked)
{
	Locked = InLocked;
	if ( MySlider.IsValid() )
	{
		MySlider->SetLocked(InLocked);
	}
}

float USlider::GetStepSize() const
{
	if (MySlider.IsValid())
	{
		return MySlider->GetStepSize();
	}
	return StepSize;
}

void USlider::SetStepSize(float InValue)
{
	StepSize = InValue;
	if (MySlider.IsValid())
	{
		MySlider->SetStepSize(InValue);
	}
}

FLinearColor USlider::GetSliderHandleColor() const
{
	return SliderHandleColor;
}

void USlider::SetSliderHandleColor(FLinearColor InValue)
{
	SliderHandleColor = InValue;
	if (MySlider.IsValid())
	{
		MySlider->SetSliderHandleColor(InValue);
	}
}

FLinearColor USlider::GetSliderBarColor() const
{
	return SliderBarColor;
}

void USlider::SetSliderBarColor(FLinearColor InValue)
{
	SliderBarColor = InValue;
	if (MySlider.IsValid())
	{
		MySlider->SetSliderBarColor(InValue);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> USlider::GetAccessibleWidget() const
{
	return MySlider;
}
#endif

#if WITH_EDITOR

const FText USlider::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

