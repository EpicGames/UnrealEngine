// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Spacer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SSpacer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Spacer)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// USpacer

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USpacer::USpacer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Size(1.0f, 1.0f)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void USpacer::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MySpacer.Reset();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVector2D USpacer::GetSize() const
{
	if (MySpacer.IsValid())
	{
		return MySpacer->GetSize();
	}
	
	return Size;
}

void USpacer::SetSize(FVector2D InSize)
{
	Size = InSize;

	if ( MySpacer.IsValid() )
	{
		MySpacer->SetSize(InSize);
	}
}

TSharedRef<SWidget> USpacer::RebuildWidget()
{
	MySpacer = SNew(SSpacer)
		.Size(Size);
	
	return MySpacer.ToSharedRef();
}

void USpacer::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MySpacer.IsValid())
	{
		return;
	}
	
	MySpacer->SetSize(Size);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText USpacer::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

