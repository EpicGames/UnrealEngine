// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/SpinBox.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Font.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpinBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// USpinBox

USpinBox::USpinBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UFont> RobotoFontObj(*UWidget::GetDefaultFontName());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Font = FSlateFontInfo(RobotoFontObj.Object, 12, FName("Bold"));
	}

	Value = 0;
	MinValue = 0;
	MaxValue = 0;
	MinSliderValue = 0;
	MaxSliderValue = 0;	
	MinFractionalDigits = 1;
	MaxFractionalDigits = 6;
	bAlwaysUsesDeltaSnap = false;
	bEnableSlider = true;
	Delta = 0;
	SliderExponent = 1;
	MinDesiredWidth = 0;
	ClearKeyboardFocusOnCommit = false;
	SelectAllTextOnCommit = true;
	KeyboardType = EVirtualKeyboardType::Number;
	VirtualKeyboardDismissAction = EVirtualKeyboardDismissAction::TextChangeOnDismiss;

	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetSpinBoxStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetSpinBoxStyle();
	}
#endif // WITH_EDITOR

	ForegroundColor = WidgetStyle.ForegroundColor;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USpinBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MySpinBox.Reset();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedRef<SWidget> USpinBox::RebuildWidget()
{
	MySpinBox = SNew(SSpinBox<float>)
	.Style(&WidgetStyle)
	.Font(Font)
	.ClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit)
	.SelectAllTextOnCommit(SelectAllTextOnCommit)
	.Justification(Justification)
	.KeyboardType(EVirtualKeyboardType::AsKeyboardType(KeyboardType.GetValue()))
	.VirtualKeyboardDismissAction(VirtualKeyboardDismissAction)
	.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
	.OnValueCommitted(BIND_UOBJECT_DELEGATE(FOnFloatValueCommitted, HandleOnValueCommitted))
	.OnBeginSliderMovement(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnBeginSliderMovement))
	.OnEndSliderMovement(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnEndSliderMovement))
	;
	
	return MySpinBox.ToSharedRef();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void USpinBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();
	
	if (!MySpinBox.IsValid())
	{
		return;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MySpinBox->SetDelta(Delta);
	MySpinBox->SetSliderExponent(SliderExponent);
	MySpinBox->SetMinDesiredWidth(MinDesiredWidth);

	MySpinBox->SetForegroundColor(ForegroundColor);

	MySpinBox->SetMinFractionalDigits(MinFractionalDigits);
	MySpinBox->SetMaxFractionalDigits(MaxFractionalDigits);
	MySpinBox->SetAlwaysUsesDeltaSnap(bAlwaysUsesDeltaSnap);
	MySpinBox->SetEnableSlider(bEnableSlider);

	// Set optional values
	bOverride_MinValue ? SetMinValue(MinValue) : ClearMinValue();
	bOverride_MaxValue ? SetMaxValue(MaxValue) : ClearMaxValue();
	bOverride_MinSliderValue ? SetMinSliderValue(MinSliderValue) : ClearMinSliderValue();
	bOverride_MaxSliderValue ? SetMaxSliderValue(MaxSliderValue) : ClearMaxSliderValue();

	MySpinBox->SetWidgetStyle(&WidgetStyle);
	MySpinBox->InvalidateStyle();

	MySpinBox->SetTextJustification(Justification);
	MySpinBox->SetTextBlockFont(Font);
	MySpinBox->SetTextClearKeyboardFocusOnCommit(ClearKeyboardFocusOnCommit);
	MySpinBox->SetTextSelectAllTextOnCommit(ClearKeyboardFocusOnCommit);

	// Always set the value last so that the max/min values are taken into account.
	TAttribute<float> ValueBinding = PROPERTY_BINDING(float, Value);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	MySpinBox->SetValue(ValueBinding);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float USpinBox::GetValue() const
{
	return Value;
}

void USpinBox::SetValue(float InValue)
{
	if (Value != InValue)
	{
		Value = InValue;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Value);
		if (MySpinBox.IsValid())
		{
			MySpinBox->SetValue(InValue);
		}
	}
}

const FSpinBoxStyle& USpinBox::GetWidgetStyle() const
{
	return WidgetStyle;
}

void USpinBox::SetWidgetStyle(const FSpinBoxStyle& InWidgetStyle)
{
	WidgetStyle = InWidgetStyle;
	if (MySpinBox.IsValid())
	{
		MySpinBox->InvalidateStyle();
	}
}

int32 USpinBox::GetMinFractionalDigits() const
{
	return MinFractionalDigits;
}

void USpinBox::SetMinFractionalDigits(int32 NewValue)
{
	MinFractionalDigits = FMath::Max(0, NewValue);
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMinFractionalDigits(MinFractionalDigits);
	}
}

int32 USpinBox::GetMaxFractionalDigits() const
{
	return MaxFractionalDigits;
}

void USpinBox::SetMaxFractionalDigits(int32 NewValue)
{
	MaxFractionalDigits = FMath::Max(0, NewValue);
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMaxFractionalDigits(MaxFractionalDigits);
	}
}

bool USpinBox::GetAlwaysUsesDeltaSnap() const
{
	if (MySpinBox.IsValid())
	{
		return MySpinBox->GetAlwaysUsesDeltaSnap();
	}

	return bAlwaysUsesDeltaSnap;
}

void USpinBox::SetAlwaysUsesDeltaSnap(bool bNewValue)
{
	bAlwaysUsesDeltaSnap = bNewValue;

	if (MySpinBox.IsValid())
	{
		MySpinBox->SetAlwaysUsesDeltaSnap(bNewValue);
	}
}

bool USpinBox::GetEnableSlider() const
{
	return bEnableSlider;
}

void USpinBox::SetEnableSlider(bool bNewValue)
{
	bEnableSlider = bNewValue;

	if (MySpinBox.IsValid())
	{
		MySpinBox->SetEnableSlider(bNewValue);
	}
}

float USpinBox::GetDelta() const
{
	return Delta;
}

void USpinBox::SetDelta(float NewValue)
{
	Delta = NewValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetDelta(NewValue);
	}
}

float USpinBox::GetSliderExponent() const
{
	return SliderExponent;
}

void USpinBox::SetSliderExponent(float NewValue)
{
	SliderExponent = NewValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetSliderExponent(NewValue);
	}
}

const FSlateFontInfo& USpinBox::GetFont() const
{
	return Font;
}

void USpinBox::SetFont(const FSlateFontInfo& InFont)
{
	Font = InFont;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetTextBlockFont(InFont);
	}
}

const ETextJustify::Type USpinBox::GetJustification() const
{
	return Justification;
}

void USpinBox::SetJustification(ETextJustify::Type InJustification)
{
	Justification = InJustification;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetTextJustification(InJustification);
	}
}

float USpinBox::GetMinDesiredWidth() const
{
	return MinDesiredWidth;
}

void USpinBox::SetMinDesiredWidth(float NewValue)
{
	MinDesiredWidth = NewValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMinDesiredWidth(NewValue);
	}
}

bool USpinBox::GetClearKeyboardFocusOnCommit() const
{
	return ClearKeyboardFocusOnCommit;
}

void USpinBox::SetClearKeyboardFocusOnCommit(bool bNewValue)
{
	ClearKeyboardFocusOnCommit = bNewValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetTextClearKeyboardFocusOnCommit(bNewValue);
	}
}

bool USpinBox::GetSelectAllTextOnCommit() const
{
	return SelectAllTextOnCommit;
}

void USpinBox::SetSelectAllTextOnCommit(bool bNewValue)
{
	SelectAllTextOnCommit = bNewValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetTextSelectAllTextOnCommit(bNewValue);
	}
}

// MIN VALUE
float USpinBox::GetMinValue() const
{
	float ReturnVal = TNumericLimits<float>::Lowest();

	if (bOverride_MinValue)
	{
		ReturnVal = MinValue;
	}

	return ReturnVal;
}

void USpinBox::SetMinValue(float InMinValue)
{
	bOverride_MinValue = true;
	MinValue = InMinValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMinValue(InMinValue);
	}
}

void USpinBox::ClearMinValue()
{
	bOverride_MinValue = false;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMinValue(TOptional<float>());
	}
}

// MAX VALUE
float USpinBox::GetMaxValue() const
{
	float ReturnVal = TNumericLimits<float>::Max();

	if (bOverride_MaxValue)
	{
		ReturnVal = MaxValue;
	}

	return ReturnVal;
}

void USpinBox::SetMaxValue(float InMaxValue)
{
	bOverride_MaxValue = true;
	MaxValue = InMaxValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMaxValue(InMaxValue);
	}
}
void USpinBox::ClearMaxValue()
{
	bOverride_MaxValue = false;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMaxValue(TOptional<float>());
	}
}

// MIN SLIDER VALUE
float USpinBox::GetMinSliderValue() const
{
	float ReturnVal = TNumericLimits<float>::Min();

	if (bOverride_MinSliderValue)
	{
		ReturnVal = MinSliderValue;
	}

	return ReturnVal;
}

void USpinBox::SetMinSliderValue(float InMinSliderValue)
{
	bOverride_MinSliderValue = true;
	MinSliderValue = InMinSliderValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMinSliderValue(InMinSliderValue);
	}
}

void USpinBox::ClearMinSliderValue()
{
	bOverride_MinSliderValue = false;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMinSliderValue(TOptional<float>());
	}
}

// MAX SLIDER VALUE
float USpinBox::GetMaxSliderValue() const
{
	float ReturnVal = TNumericLimits<float>::Max();

	if (bOverride_MaxSliderValue)
	{
		ReturnVal = MaxSliderValue;
	}

	return ReturnVal;
}

void USpinBox::SetMaxSliderValue(float InMaxSliderValue)
{
	bOverride_MaxSliderValue = true;
	MaxSliderValue = InMaxSliderValue;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMaxSliderValue(InMaxSliderValue);
	}
}

void USpinBox::ClearMaxSliderValue()
{
	bOverride_MaxSliderValue = false;
	if (MySpinBox.IsValid())
	{
		MySpinBox->SetMaxSliderValue(TOptional<float>());
	}
}

void USpinBox::SetForegroundColor(FSlateColor InForegroundColor)
{
	ForegroundColor = InForegroundColor;
	if ( MySpinBox.IsValid() )
	{
		MySpinBox->SetForegroundColor(ForegroundColor);
	}
}

FSlateColor USpinBox::GetForegroundColor() const
{
	return ForegroundColor;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Event handlers
void USpinBox::HandleOnValueChanged(float InValue)
{
	if ( !IsDesignTime() )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Value != InValue)
		{
			Value = InValue;
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Value);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OnValueChanged.Broadcast(InValue);
	}
}

void USpinBox::HandleOnValueCommitted(float InValue, ETextCommit::Type CommitMethod)
{
	if ( !IsDesignTime() )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Value != InValue)
		{
			Value = InValue;
			BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Value);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OnValueCommitted.Broadcast(InValue, CommitMethod);
	}
}

void USpinBox::HandleOnBeginSliderMovement()
{
	if ( !IsDesignTime() )
	{
		OnBeginSliderMovement.Broadcast();
	}
}

void USpinBox::HandleOnEndSliderMovement(float InValue)
{
	if ( !IsDesignTime() )
	{
		OnEndSliderMovement.Broadcast(InValue);
	}
}

#if WITH_EDITOR

const FText USpinBox::GetPaletteCategory()
{
	return LOCTEXT("Input", "Input");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

