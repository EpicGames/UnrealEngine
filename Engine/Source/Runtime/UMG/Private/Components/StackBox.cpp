// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StackBox.h"
#include "Components/StackBoxSlot.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StackBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UStackBox

UStackBox::UStackBox()
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UStackBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyBox.Reset();
}

UClass* UStackBox::GetSlotClass() const
{
	return UStackBoxSlot::StaticClass();
}

void UStackBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if (MyBox.IsValid() )
	{
		CastChecked<UStackBoxSlot>(InSlot)->BuildSlot(MyBox.ToSharedRef());
	}
}

void UStackBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if (MyBox.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyBox->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

EOrientation UStackBox::GetOrientation() const
{
	return Orientation;
}

void UStackBox::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		if (MyBox)
		{
			MyBox->SetOrientation(InOrientation);
		}
	}
}

UStackBoxSlot* UStackBox::AddChildToStackBox(UWidget* Content)
{
	return Cast<UStackBoxSlot>(Super::AddChild(Content));
}

bool UStackBox::ReplaceStackBoxChildAt(int32 Index, UWidget* Content)
{
	if (!Slots.IsValidIndex(Index) || Content == nullptr)
	{
		return false;
	}

	UStackBoxSlot* StackBoxSlot = CastChecked<UStackBoxSlot>(Slots[Index]);
	StackBoxSlot->ReplaceContent(Content);
	return true;
}

TSharedRef<SWidget> UStackBox::RebuildWidget()
{
	MyBox = SNew(SStackBox)
		.Orientation(Orientation);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UStackBoxSlot* TypedSlot = Cast<UStackBoxSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyBox.ToSharedRef());
		}
	}

	return MyBox.ToSharedRef();
}

void UStackBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyBox.IsValid())
	{
		return;
	}

	MyBox->SetOrientation(Orientation);
}

#if WITH_EDITOR

const FText UStackBox::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

