// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StackBoxSlot.h"
#include "Components/Widget.h"
#include "Components/StackBox.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StackBoxSlot)

/////////////////////////////////////////////////////
// UStackBoxSlot

void UStackBoxSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UStackBoxSlot::BuildSlot(TSharedRef<SStackBox> StackBox)
{
	StackBox->AddSlot()
		.Expose(Slot)
		.Padding(Padding)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		.SizeParam(UWidget::ConvertSerializedSizeParamToRuntime(Size))
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
}

void UStackBoxSlot::ReplaceContent(UWidget* NewContent)
{
	if (Content != NewContent)
	{
		if (NewContent)
		{
			NewContent->RemoveFromParent();
		}

		if (UWidget* PreviousWidget = Content)
		{
			// Setting Slot=null before RemoveFromParent to prevent destroying this slot
			PreviousWidget->Slot = nullptr;
			PreviousWidget->RemoveFromParent();
		}

		Content = NewContent;

		if (Content)
		{
			Content->Slot = this;
		}

		if (Slot)
		{
			Slot->AttachWidget(Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget());
		}
	}
}

FMargin UStackBoxSlot::GetPadding() const
{
	return Padding;
}

void UStackBoxSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if (Slot)
	{
		Slot->SetPadding(InPadding);
	}
}

FSlateChildSize UStackBoxSlot::GetSize() const
{
	return Size;
}

void UStackBoxSlot::SetSize(FSlateChildSize InSize)
{
	Size = InSize;
	if (Slot)
	{
		Slot->SetSizeParam(UWidget::ConvertSerializedSizeParamToRuntime(InSize));
	}
}

EHorizontalAlignment UStackBoxSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UStackBoxSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if (Slot)
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UStackBoxSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UStackBoxSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if (Slot)
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}

void UStackBoxSlot::SynchronizeProperties()
{
	SetPadding(Padding);
	SetSize(Size);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
}

#if WITH_EDITOR
bool UStackBoxSlot::NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize)
{
	const FVector2D ClampedDirection = NudgeDirection.ClampAxes(-1, 1);
	UStackBox* ParentStackBox = CastChecked<UStackBox>(Parent);
	const EOrientation Orientation = ParentStackBox->GetOrientation();
	const int32 UnitDirection = Orientation == EOrientation::Orient_Horizontal ? (int32)ClampedDirection.X : (int32)ClampedDirection.Y;
	if (UnitDirection == 0)
	{
		return false;
	}

	const int32 CurrentIndex = ParentStackBox->GetChildIndex(Content);

	if ((CurrentIndex == 0 && UnitDirection < 0) ||
		(CurrentIndex + 1 >= ParentStackBox->GetChildrenCount() && UnitDirection > 0))
	{
		return false;
	}

	ParentStackBox->Modify();
	ParentStackBox->ShiftChild(CurrentIndex + UnitDirection, Content);

	return true;
}

void UStackBoxSlot::SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot)
{
	const ThisClass* const TemplateStackBoxSlot = CastChecked<ThisClass>(TemplateSlot);
	const int32 CurrentIndex = TemplateStackBoxSlot->Parent->GetChildIndex(TemplateStackBoxSlot->Content);

	UStackBox* ParentStackBox = CastChecked<UStackBox>(Parent);
	ParentStackBox->ShiftChild(CurrentIndex, Content);
}
#endif

