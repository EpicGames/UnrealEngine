// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/TextBlock.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Font.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SInvalidationPanel.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Engine/UserInterfaceSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextBlock)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UTextBlock

UTextBlock::UTextBlock(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	bWrapWithInvalidationPanel = false;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ShadowOffset = FVector2D(1.0f, 1.0f);
	ColorAndOpacity = FLinearColor::White;
	ShadowColorAndOpacity = FLinearColor::Transparent;
	TextTransformPolicy = ETextTransformPolicy::None;
	TextOverflowPolicy = ETextOverflowPolicy::Clip;

	if (!IsRunningDedicatedServer())
	{
		static ConstructorHelpers::FObjectFinder<UFont> RobotoFontObj(*UWidget::GetDefaultFontName());
		Font = FSlateFontInfo(RobotoFontObj.Object, 24, FName("Bold"));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::Auto;
	bCanChildrenBeAccessible = false;
#endif
}

void UTextBlock::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyTextBlock.Reset();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSlateColor UTextBlock::GetColorAndOpacity() const
{
	if (ColorAndOpacityDelegate.IsBound() && !IsDesignTime())
	{
		return ColorAndOpacityDelegate.Execute();
	}
	else
	{
		return ColorAndOpacity;
	}
}

void UTextBlock::SetColorAndOpacity(FSlateColor InColorAndOpacity)
{
	ColorAndOpacity = InColorAndOpacity;
	if( MyTextBlock.IsValid() )
	{
		MyTextBlock->SetColorAndOpacity( InColorAndOpacity );
	}
}

void UTextBlock::SetOpacity(float InOpacity)
{
	FLinearColor CurrentColor = ColorAndOpacity.GetSpecifiedColor();
	CurrentColor.A = InOpacity;
	
	SetColorAndOpacity(FSlateColor(CurrentColor));
}

FLinearColor UTextBlock::GetShadowColorAndOpacity() const
{
	if (ShadowColorAndOpacityDelegate.IsBound() && !IsDesignTime())
	{
		return ShadowColorAndOpacityDelegate.Execute();
	}
	else
	{
		return ShadowColorAndOpacity;
	}
}

void UTextBlock::SetShadowColorAndOpacity(FLinearColor InShadowColorAndOpacity)
{
	ShadowColorAndOpacity = InShadowColorAndOpacity;
	if( MyTextBlock.IsValid() )
	{
		MyTextBlock->SetShadowColorAndOpacity(InShadowColorAndOpacity);
	}
}

FVector2D UTextBlock::GetShadowOffset() const
{
	return ShadowOffset;
}

void UTextBlock::SetShadowOffset(FVector2D InShadowOffset)
{
	ShadowOffset = InShadowOffset;
	if( MyTextBlock.IsValid() )
	{
		MyTextBlock->SetShadowOffset(ShadowOffset);
	}
}

const FSlateFontInfo& UTextBlock::GetFont() const
{
	return Font;
}

void UTextBlock::SetFont(FSlateFontInfo InFontInfo)
{
	Font = InFontInfo;
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetFont(Font);
	}
	OnFontChanged();
}

const FSlateBrush& UTextBlock::GetStrikeBrush() const
{
	return StrikeBrush;
}

void UTextBlock::SetStrikeBrush(FSlateBrush InStrikeBrush)
{
	StrikeBrush = InStrikeBrush;
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetStrikeBrush(&StrikeBrush);
	}
}

void UTextBlock::OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions)
{
	Super::OnShapedTextOptionsChanged(InShapedTextOptions);
	if (MyTextBlock.IsValid())
	{
		InShapedTextOptions.SynchronizeShapedTextProperties(*MyTextBlock);
	}
}

void UTextBlock::OnJustificationChanged(ETextJustify::Type InJustification)
{
	Super::OnJustificationChanged(InJustification);
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetJustification(InJustification);
	}
}

void UTextBlock::OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy)
{
	Super::OnWrappingPolicyChanged(InWrappingPolicy);
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetWrappingPolicy(InWrappingPolicy);
	}
}

void UTextBlock::OnAutoWrapTextChanged(bool InAutoWrapText)
{
	Super::OnAutoWrapTextChanged(InAutoWrapText);
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetAutoWrapText(InAutoWrapText);
	}
}

void UTextBlock::OnWrapTextAtChanged(float InWrapTextAt)
{
	Super::OnWrapTextAtChanged(InWrapTextAt);
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetWrapTextAt(InWrapTextAt);
	}
}

void UTextBlock::OnLineHeightPercentageChanged(float InLineHeightPercentage)
{
	Super::OnLineHeightPercentageChanged(InLineHeightPercentage);
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetLineHeightPercentage(InLineHeightPercentage);
	}
}

void UTextBlock::OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine)
{
	Super::OnApplyLineHeightToBottomLineChanged(InApplyLineHeightToBottomLine);
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetApplyLineHeightToBottomLine(InApplyLineHeightToBottomLine);
	}
}

void UTextBlock::OnMarginChanged(const FMargin& InMargin)
{
	Super::OnMarginChanged(InMargin);
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetMargin(InMargin);
	}
}

float UTextBlock::GetMinDesiredWidth() const
{
	return MinDesiredWidth;
}

void UTextBlock::SetMinDesiredWidth(float InMinDesiredWidth)
{
	MinDesiredWidth = InMinDesiredWidth;
	if (MyTextBlock.IsValid())
	{
		MyTextBlock->SetMinDesiredWidth(MinDesiredWidth);
	}
}

void UTextBlock::SetAutoWrapText(bool InAutoWrapText)
{
	AutoWrapText = InAutoWrapText;
	if(MyTextBlock.IsValid())
	{
		MyTextBlock->SetAutoWrapText(InAutoWrapText);
	}
}

ETextTransformPolicy UTextBlock::GetTextTransformPolicy() const
{
	return TextTransformPolicy;
}

void UTextBlock::SetTextTransformPolicy(ETextTransformPolicy InTransformPolicy)
{
	TextTransformPolicy = InTransformPolicy;
	if(MyTextBlock.IsValid())
	{
		MyTextBlock->SetTransformPolicy(TextTransformPolicy);
	}
}

ETextOverflowPolicy UTextBlock::GetTextOverflowPolicy() const
{
	return TextOverflowPolicy;
}

void UTextBlock::SetTextOverflowPolicy(ETextOverflowPolicy InOverflowPolicy)
{
	TextOverflowPolicy = InOverflowPolicy;
	SynchronizeProperties();
}

void UTextBlock::SetFontMaterial(UMaterialInterface* InMaterial)
{
	Font.FontMaterial = InMaterial;
	SetFont(Font);
}

void UTextBlock::SetFontOutlineMaterial(UMaterialInterface* InMaterial)
{
	Font.OutlineSettings.OutlineMaterial = InMaterial;
	SetFont(Font);
}

void UTextBlock::SetFontSize(float DisplayFontSize)
{
	// Convert from your project FontResolution setting DPI to the native default render DPI
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	if (UISettings)
	{

		Font.Size = UISettings->ConvertFontSizeFromDisplayToNative(DisplayFontSize);
		SetFont(Font);
	}
}

UMG_API float UTextBlock::GetFontSize() const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	if (UISettings)
	{
		return UISettings->ConvertFontSizeFromNativeToDisplay(Font.Size);
	}
	return 0.0f;
}

UMaterialInstanceDynamic* UTextBlock::GetDynamicFontMaterial()
{
	if (Font.FontMaterial)
	{
		UMaterialInterface* Material = CastChecked<UMaterialInterface>(Font.FontMaterial);

		UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);

		if (!DynamicMaterial)
		{
			DynamicMaterial = UMaterialInstanceDynamic::Create(Material, this);
			Font.FontMaterial = DynamicMaterial;

			SetFont(Font);
		}

		return DynamicMaterial;
	}

	return nullptr;
}

UMaterialInstanceDynamic* UTextBlock::GetDynamicOutlineMaterial()
{
	if (Font.OutlineSettings.OutlineMaterial)
	{
		UMaterialInterface* Material = CastChecked<UMaterialInterface>(Font.OutlineSettings.OutlineMaterial);

		UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Material);

		if (!DynamicMaterial)
		{
			DynamicMaterial = UMaterialInstanceDynamic::Create(Material, this);
			Font.OutlineSettings.OutlineMaterial = DynamicMaterial;

			SetFont(Font);
		}

		return DynamicMaterial;
	}

	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


TSharedRef<SWidget> UTextBlock::RebuildWidget()
{
 	if (bWrapWithInvalidationPanel && !IsDesignTime())
 	{
 		TSharedPtr<SWidget> RetWidget = SNew(SInvalidationPanel)
 		[
 			SAssignNew(MyTextBlock, STextBlock)
			.SimpleTextMode(bSimpleTextMode)
			.OnAllFontFacesFinishLoading(OnAllFontFacesFinishLoading.IsBound() ? BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleAllFontFacesFinishLoading) : FSimpleDelegate())
 		];
 		return RetWidget.ToSharedRef();
 	}
 	else
	{
		MyTextBlock =
			SNew(STextBlock)
			.SimpleTextMode(bSimpleTextMode)
			.OnAllFontFacesFinishLoading(OnAllFontFacesFinishLoading.IsBound() ? BIND_UOBJECT_DELEGATE(FSimpleDelegate, SlateHandleAllFontFacesFinishLoading) : FSimpleDelegate());

		//if (IsDesignTime())
		//{
		//	return SNew(SOverlay)

		//	+ SOverlay::Slot()
		//	[
		//		MyTextBlock.ToSharedRef()
		//	]

		//	+ SOverlay::Slot()
		//	.VAlign(VAlign_Top)
		//	.HAlign(HAlign_Right)
		//	[
		//		SNew(SImage)
		//		.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
		//		.Visibility_UObject(this, &ThisClass::GetTextWarningImageVisibility)
		//		.ToolTipText(LOCTEXT("TextNotLocalizedWarningToolTip", "This text is marked as 'culture invariant' and won't be gathered for localization.\nYou can change this by editing the advanced text settings."))
		//	];
		//}
		
		return MyTextBlock.ToSharedRef();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EVisibility UTextBlock::GetTextWarningImageVisibility() const
{
	return Text.IsCultureInvariant() ? EVisibility::Visible : EVisibility::Collapsed;
}

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> UTextBlock::GetAccessibleWidget() const
{
	return MyTextBlock;
}
#endif

void UTextBlock::OnBindingChanged(const FName& Property)
{
	Super::OnBindingChanged(Property);

	if ( MyTextBlock.IsValid() )
	{
		static const FName TextProperty("TextDelegate");
		static const FName ColorAndOpacityProperty("ColorAndOpacityDelegate");
		static const FName ShadowColorAndOpacityProperty("ShadowColorAndOpacityDelegate");

		if ( Property == TextProperty )
		{
			TAttribute<FText> TextBinding = GetDisplayText();
			MyTextBlock->SetText(TextBinding);
		}
		else if ( Property == ColorAndOpacityProperty )
		{
			TAttribute<FSlateColor> ColorAndOpacityBinding = PROPERTY_BINDING(FSlateColor, ColorAndOpacity);
			MyTextBlock->SetColorAndOpacity(ColorAndOpacityBinding);
		}
		else if ( Property == ShadowColorAndOpacityProperty )
		{
			TAttribute<FLinearColor> ShadowColorAndOpacityBinding = PROPERTY_BINDING(FLinearColor, ShadowColorAndOpacity);
			MyTextBlock->SetShadowColorAndOpacity(ShadowColorAndOpacityBinding);
		}
	}
}

void UTextBlock::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	TAttribute<FText> TextBinding = GetDisplayText();
	TAttribute<FSlateColor> ColorAndOpacityBinding = PROPERTY_BINDING(FSlateColor, ColorAndOpacity);
	TAttribute<FLinearColor> ShadowColorAndOpacityBinding = PROPERTY_BINDING(FLinearColor, ShadowColorAndOpacity);

	if ( MyTextBlock.IsValid() )
	{
		MyTextBlock->SetText( TextBinding );
		MyTextBlock->SetFont( Font );
		MyTextBlock->SetStrikeBrush( &StrikeBrush );
		MyTextBlock->SetColorAndOpacity( ColorAndOpacityBinding );
		MyTextBlock->SetShadowOffset( ShadowOffset );
		MyTextBlock->SetShadowColorAndOpacity( ShadowColorAndOpacityBinding );
		MyTextBlock->SetMinDesiredWidth( MinDesiredWidth );
		MyTextBlock->SetTransformPolicy( TextTransformPolicy );
		MyTextBlock->SetOverflowPolicy(TextOverflowPolicy);

		Super::SynchronizeTextLayoutProperties( *MyTextBlock );
	}
}

/// @cond DOXYGEN_WARNINGS

FText UTextBlock::GetText() const
{
	if (MyTextBlock.IsValid())
	{
		return MyTextBlock->GetText();
	}

	return Text;
}

/// @endcond

void UTextBlock::SetText(FText InText)
{
	Text = InText;
	TextDelegate.Unbind();
	if ( MyTextBlock.IsValid() )
	{
		MyTextBlock->SetText(GetDisplayText());
	}
	OnTextChanged();
}

TAttribute<FText> UTextBlock::GetDisplayText()
{
	return PROPERTY_BINDING(FText, Text);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

FString UTextBlock::GetLabelMetadata() const
{
	const int32 MaxSampleLength = 15;

	FString TextStr = GetText().ToString().Replace(TEXT("\n"), TEXT(" "));
	TextStr = TextStr.Len() <= MaxSampleLength ? TextStr : TextStr.Left(MaxSampleLength - 2) + TEXT("..");
	return TEXT(" \"") + TextStr + TEXT("\"");
}

void UTextBlock::HandleTextCommitted(const FText& InText, ETextCommit::Type CommitteType)
{
	//TODO UMG How will this migrate to the template?  Seems to me we need the previews to have access to their templates!
	//TODO UMG How will the user click the editable area?  There is an overlay blocking input so that other widgets don't get them.
	//     Need a way to recognize one particular widget and forward things to them!
}

const FText UTextBlock::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

void UTextBlock::OnCreationFromPalette()
{
	SetText(LOCTEXT("TextBlockDefaultValue", "Text Block"));
}

bool UTextBlock::CanEditChange(const FProperty* InProperty) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bSimpleTextMode && InProperty)
	{
		static TArray<FName> InvalidPropertiesInSimpleMode =
		{
			GET_MEMBER_NAME_CHECKED(UTextBlock, ShapedTextOptions),
			GET_MEMBER_NAME_CHECKED(UTextBlock, Justification),
			GET_MEMBER_NAME_CHECKED(UTextBlock, WrappingPolicy),
			GET_MEMBER_NAME_CHECKED(UTextBlock, AutoWrapText),
			GET_MEMBER_NAME_CHECKED(UTextBlock, WrapTextAt),
			GET_MEMBER_NAME_CHECKED(UTextBlock, Margin),
			GET_MEMBER_NAME_CHECKED(UTextBlock, LineHeightPercentage),
			GET_MEMBER_NAME_CHECKED(UTextBlock, AutoWrapText),
		};

		return !InvalidPropertiesInSimpleMode.Contains(InProperty->GetFName());
	}

	return Super::CanEditChange(InProperty);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif //if WITH_EDITOR

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

