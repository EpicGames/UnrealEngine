// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/TextWidgetTypes.h"
#include "Fonts/FontCache.h"
#include "Framework/Text/SlateAsyncFontLoadingTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextWidgetTypes)

FShapedTextOptions::FShapedTextOptions()
{
	bOverride_TextShapingMethod = false;
	TextShapingMethod = ETextShapingMethod::Auto;

	bOverride_TextFlowDirection = false;
	TextFlowDirection = ETextFlowDirection::Auto;
}

UTextLayoutWidget::UTextLayoutWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Justification = ETextJustify::Left;
	AutoWrapText = false;
	WrapTextAt = 0.0f;
	WrappingPolicy = ETextWrappingPolicy::DefaultWrapping;
	Margin = FMargin(0.0f);
	LineHeightPercentage = 1.0f;
	ApplyLineHeightToBottomLine = true;
	FontFacesLoadingPaintPolicy = EFontFacesLoadingPaintPolicy::DoNotPaint;
}

void UTextLayoutWidget::SetJustification(ETextJustify::Type InJustification)
{
	if (Justification != InJustification)
	{
		Justification = InJustification;
		OnJustificationChanged(InJustification);
	}
}

void UTextLayoutWidget::SetShapedTextOptions(FShapedTextOptions InShapedTextOptions)
{
	if (ShapedTextOptions != InShapedTextOptions)
	{
		ShapedTextOptions = InShapedTextOptions;
		OnShapedTextOptionsChanged(InShapedTextOptions);
	}
}

void UTextLayoutWidget::SetWrappingPolicy(ETextWrappingPolicy InWrappingPolicy)
{
	if (WrappingPolicy != InWrappingPolicy)
	{
		WrappingPolicy = InWrappingPolicy;
		OnWrappingPolicyChanged(InWrappingPolicy);
	}
}

void UTextLayoutWidget::SetAutoWrapText(bool InAutoWrapText)
{
	if (AutoWrapText != InAutoWrapText)
	{
		AutoWrapText = InAutoWrapText;
		OnAutoWrapTextChanged(InAutoWrapText);
	}
}

void UTextLayoutWidget::SetWrapTextAt(float InWrapTextAt)
{
	if (WrapTextAt != InWrapTextAt)
	{
		WrapTextAt = InWrapTextAt;
		OnWrapTextAtChanged(InWrapTextAt);
	}
}

void UTextLayoutWidget::SetLineHeightPercentage(float InLineHeightPercentage)
{
	if (LineHeightPercentage != InLineHeightPercentage)
	{
		LineHeightPercentage = InLineHeightPercentage;
		OnLineHeightPercentageChanged(InLineHeightPercentage);
	}
}

void UTextLayoutWidget::SetApplyLineHeightToBottomLine(bool InApplyLineHeightToBottomLine)
{
	if (ApplyLineHeightToBottomLine != InApplyLineHeightToBottomLine)
	{
		ApplyLineHeightToBottomLine = InApplyLineHeightToBottomLine;
		OnApplyLineHeightToBottomLineChanged(InApplyLineHeightToBottomLine);
	}
}

void UTextLayoutWidget::SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy)
{
	if (FontFacesLoadingPaintPolicy != InFontFacesLoadingPaintPolicy)
	{
		FontFacesLoadingPaintPolicy = InFontFacesLoadingPaintPolicy;
		OnFontFacesLoadingPaintPolicyChange(InFontFacesLoadingPaintPolicy);
	}
}

void UTextLayoutWidget::SetMargin(const FMargin& InMargin)
{
	if (Margin != InMargin)
	{
		Margin = InMargin;
		OnMarginChanged(Margin);
	}
}
