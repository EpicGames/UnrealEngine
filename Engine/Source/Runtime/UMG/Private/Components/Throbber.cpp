// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Throbber.h"
#include "SlateFwd.h"
#include "SlateGlobals.h"
#include "Slate/SlateBrushAsset.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Throbber)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UThrobber

UThrobber::UThrobber(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NumberOfPieces = 3;

	bAnimateVertically = true;
	bAnimateHorizontally = true;
	bAnimateOpacity = true;
	
	Image = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetThrobberBrush();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		Image = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetThrobberBrush();
	}
#endif // WITH_EDITOR
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UThrobber::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyThrobber.Reset();
}

TSharedRef<SWidget> UThrobber::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyThrobber = SNew(SThrobber)
		.PieceImage(&Image)
		.NumPieces(FMath::Clamp(NumberOfPieces, 1, 25))
		.Animate(GetAnimation());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return MyThrobber.ToSharedRef();
}

void UThrobber::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyThrobber.IsValid())
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyThrobber->SetPieceImage(&Image);
	MyThrobber->InvalidatePieceImage();
	MyThrobber->SetNumPieces(FMath::Clamp(NumberOfPieces, 1, 25));
	MyThrobber->SetAnimate(GetAnimation());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
SThrobber::EAnimation UThrobber::GetAnimation() const
{
	const int32 AnimationParams = (bAnimateVertically ? SThrobber::Vertical : 0) |
		(bAnimateHorizontally ? SThrobber::Horizontal : 0) |
		(bAnimateOpacity ? SThrobber::Opacity : 0);

	return static_cast<SThrobber::EAnimation>(AnimationParams);
}

void UThrobber::SetNumberOfPieces(int32 InNumberOfPieces)
{
	int32 NewNumberOfPieces = FMath::Clamp(InNumberOfPieces, 1, 25);
	if (NewNumberOfPieces != InNumberOfPieces)
	{
		UE_LOGF(LogSlate, Warning, "The number of Pieces was clamped between 1 and 25");
	}

	NumberOfPieces = NewNumberOfPieces;
	if (MyThrobber.IsValid())
	{
		MyThrobber->SetNumPieces(NumberOfPieces);
	}
}

int32 UThrobber::GetNumberOfPieces() const
{
	return NumberOfPieces;
}

void UThrobber::SetAnimateHorizontally(bool bInAnimateHorizontally)
{
	bAnimateHorizontally = bInAnimateHorizontally;
	if (MyThrobber.IsValid())
	{
		MyThrobber->SetAnimate(GetAnimation());
	}
}

bool UThrobber::IsAnimateHorizontally() const
{
	return bAnimateHorizontally;
}

void UThrobber::SetAnimateVertically(bool bInAnimateVertically)
{
	bAnimateVertically = bInAnimateVertically;
	if (MyThrobber.IsValid())
	{
		MyThrobber->SetAnimate(GetAnimation());
	}
}

bool UThrobber::IsAnimateVertically() const
{
	return bAnimateVertically;
}

void UThrobber::SetAnimateOpacity(bool bInAnimateOpacity)
{
	bAnimateOpacity = bInAnimateOpacity;
	if (MyThrobber.IsValid())
	{
		MyThrobber->SetAnimate(GetAnimation());
	}
}

bool UThrobber::IsAnimateOpacity() const
{
	return bAnimateOpacity;
}

void UThrobber::SetImage(const FSlateBrush& Brush)
{
	Image = Brush;
	if (MyThrobber.IsValid())
	{
		MyThrobber->InvalidatePieceImage();
	}
}

const FSlateBrush& UThrobber::GetImage() const
{
	return Image;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UThrobber::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

