// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/TileView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TileView)

/////////////////////////////////////////////////////
// UTileView

UTileView::UTileView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = true;
}

TSharedRef<STableViewBase> UTileView::RebuildListWidget()
{
	return ConstructTileView<STileView>();
}

FMargin UTileView::GetDesiredEntryPadding(UObject* Item) const
{
	return FMargin(GetHorizontalEntrySpacing() * 0.5f, GetVerticalEntrySpacing() * 0.5f);
}

float UTileView::GetTotalEntryHeight() const
{
	if (IsAligned() && !bEntrySizeIncludesEntrySpacing)
	{
		return EntryHeight + GetVerticalEntrySpacing();
	}
	else
	{
		return EntryHeight + GetVerticalEntrySpacing() * 0.5f;
	}
}

float UTileView::GetTotalEntryWidth() const
{
	if (IsAligned() && !bEntrySizeIncludesEntrySpacing)
	{
		return EntryWidth + GetHorizontalEntrySpacing();
	}
	else
	{
		return EntryWidth + GetHorizontalEntrySpacing() * 0.5f;
	}
}

void UTileView::SetEntryHeight(float NewHeight)
{
	EntryHeight = NewHeight;
	if (MyTileView.IsValid())
	{
		MyTileView->SetItemHeight(GetTotalEntryHeight());
	}
}

void UTileView::SetEntryWidth(float NewWidth)
{
	EntryWidth = NewWidth;
	if (MyTileView.IsValid())
	{
		MyTileView->SetItemWidth(GetTotalEntryWidth());
	}
}

bool UTileView::IsAligned() const
{
	return TileAlignment == EListItemAlignment::LeftAligned
		|| TileAlignment == EListItemAlignment::RightAligned
		|| TileAlignment == EListItemAlignment::CenterAligned;
}

void UTileView::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyTileView.Reset();
}


/////////////////////////////////////////////////////
