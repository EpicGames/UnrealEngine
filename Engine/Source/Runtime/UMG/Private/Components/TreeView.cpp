// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/TreeView.h"
#include "Styling/DefaultStyleCache.h"
#include "Styling/UMGCoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TreeView)

/////////////////////////////////////////////////////
// UTreeView

UTreeView::UTreeView(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetRuntime().GetTreeViewStyle();
	
#if WITH_EDITOR 
	if (IsEditorWidget())
	{
		WidgetStyle = UE::Slate::Private::FDefaultStyleCache::GetEditor().GetTreeViewStyle();
	}
#endif // WITH_EDITOR
}

TSharedRef<STableViewBase> UTreeView::RebuildListWidget()
{
	return ConstructTreeView<STreeView>();
}

void UTreeView::OnItemExpansionChangedInternal(UObject* Item, bool bIsExpanded)
{
	BP_OnItemExpansionChanged.Broadcast(Item, bIsExpanded);
}

void UTreeView::OnGetChildrenInternal(UObject* Item, TArray<UObject*>& OutChildren) const
{
#if WITH_EDITORONLY_DATA
	//@todo DanH TreeView: Ideally it'd be nice to support previewing children/nesting in some way
	if (!IsDesignTime())
#endif
	{
		if (OnGetItemChildren.IsBound())
		{
			OnGetItemChildren.Execute(Item, OutChildren);
		}
		else if (BP_OnGetItemChildren.IsBound())
		{
			BP_OnGetItemChildren.Execute(Item, OutChildren);
		}
	}
}

void UTreeView::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyTreeView.Reset();
}

void UTreeView::SetItemExpansion(UObject* Item, bool bExpandItem)
{
	if (MyTreeView.IsValid())
	{
		MyTreeView->SetItemExpansion(Item, bExpandItem);
	}
}

void UTreeView::ExpandAll()
{
	if (MyTreeView.IsValid())
	{
		//@todo DanH TreeView: Recursive expansion - as written this will only expand the root items
		for (UObject* ListItem : GetListItems())
		{
			MyTreeView->SetItemExpansion(ListItem, true);
		}
	}
}

void UTreeView::CollapseAll()
{
	if (MyTreeView.IsValid())
	{
		MyTreeView->ClearExpandedItems();
	}
}

void UTreeView::OnItemClickedInternal(UObject* ListItem)
{
	// If the clicked item has children, expand it now as part of the click
	if (ensure(MyTreeView.IsValid()))
	{
		// The item was clicked, implying that there should certainly be a widget representing this item right now
		TSharedPtr<ITableRow> RowWidget = MyTreeView->WidgetFromItem(ListItem);
		if (ensure(RowWidget.IsValid()) && RowWidget->DoesItemHaveChildren() > 0)
		{
			const bool bNewExpansionState = !MyTreeView->IsItemExpanded(ListItem);
			MyTreeView->SetItemExpansion(ListItem, bNewExpansionState);
		}
	}
	Super::OnItemClickedInternal(ListItem);
}

/////////////////////////////////////////////////////
