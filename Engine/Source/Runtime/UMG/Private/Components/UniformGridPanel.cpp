// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/UniformGridPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Components/UniformGridSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UniformGridPanel)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UUniformGridPanel

UUniformGridPanel::UUniformGridPanel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UUniformGridPanel::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyUniformGridPanel.Reset();
}

UClass* UUniformGridPanel::GetSlotClass() const
{
	return UUniformGridSlot::StaticClass();
}

void UUniformGridPanel::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyUniformGridPanel.IsValid() )
	{
		CastChecked<UUniformGridSlot>(InSlot)->BuildSlot(MyUniformGridPanel.ToSharedRef());
	}
}

void UUniformGridPanel::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyUniformGridPanel.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyUniformGridPanel->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

TSharedRef<SWidget> UUniformGridPanel::RebuildWidget()
{
	MyUniformGridPanel = SNew(SUniformGridPanel);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UUniformGridSlot* TypedSlot = Cast<UUniformGridSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyUniformGridPanel.ToSharedRef());
		}
	}

	return MyUniformGridPanel.ToSharedRef();
}

UUniformGridSlot* UUniformGridPanel::AddChildToUniformGrid(UWidget* Content, int32 InRow, int32 InColumn)
{
	UUniformGridSlot* GridSlot = Cast<UUniformGridSlot>(Super::AddChild(Content));

	if (GridSlot != nullptr)
	{
		GridSlot->SetRow(InRow);
		GridSlot->SetColumn(InColumn);
	}

	return GridSlot;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UUniformGridPanel::GetSlotPadding() const
{
	return SlotPadding;
}

void UUniformGridPanel::SetSlotPadding(FMargin InSlotPadding)
{
	SlotPadding = InSlotPadding;
	if ( MyUniformGridPanel.IsValid() )
	{
		MyUniformGridPanel->SetSlotPadding(InSlotPadding);
	}
}

float UUniformGridPanel::GetMinDesiredSlotWidth() const
{
	return MinDesiredSlotWidth;
}

void UUniformGridPanel::SetMinDesiredSlotWidth(float InMinDesiredSlotWidth)
{
	MinDesiredSlotWidth = InMinDesiredSlotWidth;
	if ( MyUniformGridPanel.IsValid() )
	{
		MyUniformGridPanel->SetMinDesiredSlotWidth(InMinDesiredSlotWidth);
	}
}

float UUniformGridPanel::GetMinDesiredSlotHeight() const
{
	return MinDesiredSlotHeight;
}

void UUniformGridPanel::SetMinDesiredSlotHeight(float InMinDesiredSlotHeight)
{
	MinDesiredSlotHeight = InMinDesiredSlotHeight;
	if ( MyUniformGridPanel.IsValid() )
	{
		MyUniformGridPanel->SetMinDesiredSlotHeight(InMinDesiredSlotHeight);
	}
}

void UUniformGridPanel::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyUniformGridPanel.IsValid())
	{
		return;
	}

	MyUniformGridPanel->SetSlotPadding(SlotPadding);
	MyUniformGridPanel->SetMinDesiredSlotWidth(MinDesiredSlotWidth);
	MyUniformGridPanel->SetMinDesiredSlotHeight(MinDesiredSlotHeight);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UUniformGridPanel::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

