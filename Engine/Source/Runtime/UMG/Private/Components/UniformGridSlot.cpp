// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/UniformGridSlot.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UniformGridSlot)

/////////////////////////////////////////////////////
// UUniformGridSlot

UUniformGridSlot::UUniformGridSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Left;
	VerticalAlignment = VAlign_Top;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UUniformGridSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UUniformGridSlot::BuildSlot(TSharedRef<SUniformGridPanel> GridPanel)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GridPanel->AddSlot(Column, Row)
		.Expose(Slot)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 UUniformGridSlot::GetRow() const
{
	return Row;
}

void UUniformGridSlot::SetRow(int32 InRow)
{
	Row = InRow;
	if ( Slot )
	{
		Slot->SetRow(InRow);
	}
}

int32 UUniformGridSlot::GetColumn() const
{
	return Column;
}

void UUniformGridSlot::SetColumn(int32 InColumn)
{
	Column = InColumn;
	if ( Slot )
	{
		Slot->SetColumn(InColumn);
	}
}

EHorizontalAlignment UUniformGridSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UUniformGridSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UUniformGridSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UUniformGridSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Slot )
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UUniformGridSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetRow(Row);
	SetColumn(Column);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

bool UUniformGridSlot::NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize)
{
	const FVector2D ClampedDirection = NudgeDirection.ClampAxes(-1.0f, 1.0f);
	const int32 NewColumn = GetColumn() + FMath::TruncToInt32(ClampedDirection.X);
	const int32 NewRow = GetRow() + FMath::TruncToInt32(ClampedDirection.Y);

	if (NewColumn < 0 || NewRow < 0 || (NewColumn == GetColumn() && NewRow == GetRow()))
	{
		return false;
	}

	Modify();

	SetRow(NewRow);
	SetColumn(NewColumn);

	return true;
}

void UUniformGridSlot::SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot)
{
	const ThisClass* const TemplateUniformGridSlot = CastChecked<ThisClass>(TemplateSlot);
	SetRow(TemplateUniformGridSlot->GetRow());
	SetColumn(TemplateUniformGridSlot->GetColumn());
}

#endif

