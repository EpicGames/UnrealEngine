// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/VerticalBox.h"
#include "Components/VerticalBoxSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VerticalBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UVerticalBox

UVerticalBox::UVerticalBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UVerticalBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyVerticalBox.Reset();
}

UClass* UVerticalBox::GetSlotClass() const
{
	return UVerticalBoxSlot::StaticClass();
}

void UVerticalBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyVerticalBox.IsValid() )
	{
		CastChecked<UVerticalBoxSlot>(InSlot)->BuildSlot(MyVerticalBox.ToSharedRef());
	}
}

void UVerticalBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyVerticalBox.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyVerticalBox->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

UVerticalBoxSlot* UVerticalBox::AddChildToVerticalBox(UWidget* Content)
{
	return Cast<UVerticalBoxSlot>(Super::AddChild(Content));
}

TSharedRef<SWidget> UVerticalBox::RebuildWidget()
{
	MyVerticalBox = SNew(SVerticalBox);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UVerticalBoxSlot* TypedSlot = Cast<UVerticalBoxSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyVerticalBox.ToSharedRef());
		}
	}

	return MyVerticalBox.ToSharedRef();
}

#if WITH_EDITOR

const FText UVerticalBox::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

