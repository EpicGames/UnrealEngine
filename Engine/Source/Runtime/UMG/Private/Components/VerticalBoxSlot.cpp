// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/VerticalBoxSlot.h"
#include "Components/Widget.h"
#include "Components/VerticalBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VerticalBoxSlot)

/////////////////////////////////////////////////////
// UVerticalBoxSlot

UVerticalBoxSlot::UVerticalBoxSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
	Size = FSlateChildSize(ESlateSizeRule::Automatic);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UVerticalBoxSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UVerticalBoxSlot::BuildSlot(TSharedRef<SVerticalBox> VerticalBox)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	VerticalBox->AddSlot()
		.Expose(Slot)
		.Padding(Padding)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		.SizeParam(UWidget::ConvertSerializedSizeParamToRuntime(Size))
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UVerticalBoxSlot::GetPadding() const
{
	return Padding;
}

void UVerticalBoxSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Slot )
	{
		Slot->SetPadding(InPadding);
	}
}

FSlateChildSize UVerticalBoxSlot::GetSize() const
{
	return Size;
}

void UVerticalBoxSlot::SetSize(FSlateChildSize InSize)
{
	Size = InSize;
	if ( Slot )
	{
		Slot->SetSizeParam(UWidget::ConvertSerializedSizeParamToRuntime(InSize));
	}
}

EHorizontalAlignment UVerticalBoxSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UVerticalBoxSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UVerticalBoxSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UVerticalBoxSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Slot )
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UVerticalBoxSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetPadding(Padding);
	SetSize(Size);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR

bool UVerticalBoxSlot::NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize)
{
	if (NudgeDirection.Y == 0)
	{
		return false;
	}
	
	const FVector2D ClampedDirection = NudgeDirection.ClampAxes(-1, 1);
	UVerticalBox* ParentVerticalBox = CastChecked<UVerticalBox>(Parent);

	const int32 CurrentIndex = ParentVerticalBox->GetChildIndex(Content);

	if ((CurrentIndex == 0 && ClampedDirection.Y < 0.0f) ||
		(CurrentIndex + 1 >= ParentVerticalBox->GetChildrenCount() && ClampedDirection.Y > 0.0f))
	{
		return false;
	}

	ParentVerticalBox->Modify();
	ParentVerticalBox->ShiftChild(CurrentIndex + FMath::TruncToInt32(ClampedDirection.Y), Content);

	return true;
}

void UVerticalBoxSlot::SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot)
{
	const ThisClass* const TemplateVerticalBoxSlot = CastChecked<ThisClass>(TemplateSlot);
	const int32 CurrentIndex = TemplateVerticalBoxSlot->Parent->GetChildIndex(TemplateVerticalBoxSlot->Content);

	UVerticalBox* ParentVerticalBox = CastChecked<UVerticalBox>(Parent);
	ParentVerticalBox->ShiftChild(CurrentIndex, Content);
}

#endif

