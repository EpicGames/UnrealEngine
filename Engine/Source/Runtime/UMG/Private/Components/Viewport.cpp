// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Viewport.h"
#include "Misc/App.h"
#include "CanvasTypes.h"
#include "Components/LineBatchComponent.h"
#include "Engine/LocalPlayer.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SViewport.h"
#include "PreviewScene.h"
#include "EngineModule.h"
#include "Slate/SceneViewport.h"
#include "LegacyScreenPercentageDriver.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Viewport)

#define LOCTEXT_NAMESPACE "UMG"

namespace FocusConstants
{
	const float TransitionTime = 0.25f;
}

FUMGViewportCameraTransform::FUMGViewportCameraTransform()
	: TransitionStartTime(0)
	, ViewLocation(FVector::ZeroVector)
	, ViewRotation(FRotator::ZeroRotator)
	, DesiredLocation(FVector::ZeroVector)
	, LookAt(FVector::ZeroVector)
	, StartLocation(FVector::ZeroVector)
	, OrthoZoom(DEFAULT_ORTHOZOOM)
{}

void FUMGViewportCameraTransform::SetLocation(const FVector& Position)
{
	ViewLocation = Position;
	DesiredLocation = ViewLocation;
}

void FUMGViewportCameraTransform::TransitionToLocation(const FVector& InDesiredLocation, bool bInstant)
{
	if ( bInstant )
	{
		SetLocation(InDesiredLocation);
		TransitionStartTime = FSlateApplication::Get().GetCurrentTime() - FocusConstants::TransitionTime;
	}
	else
	{
		DesiredLocation = InDesiredLocation;
		StartLocation = ViewLocation;

		TransitionStartTime = FSlateApplication::Get().GetCurrentTime();
	}
}

bool FUMGViewportCameraTransform::UpdateTransition()
{
	bool bIsAnimating = false;
	double TransitionProgress = FMath::Clamp(( FSlateApplication::Get().GetCurrentTime() - TransitionStartTime ) / FocusConstants::TransitionTime, 0.0, 1.0);
	if (TransitionProgress < 1.0 || ViewLocation != DesiredLocation)
	{
		const float Offset = (float)TransitionProgress - 1.0f;
		float LerpWeight = Offset * Offset * Offset + 1.0f;

		if (LerpWeight == 1.0f)
		{
			// Failsafe for the value not being exact on lerps
			ViewLocation = DesiredLocation;
		}
		else
		{
			ViewLocation = FMath::Lerp(StartLocation, DesiredLocation, LerpWeight);
		}


		bIsAnimating = true;
	}

	return bIsAnimating;
}

FMatrix FUMGViewportCameraTransform::ComputeOrbitMatrix() const
{
	FTransform Transform =
		FTransform(-LookAt) *
		FTransform(FRotator(0, ViewRotation.Yaw, 0)) *
		FTransform(FRotator(0, 0, ViewRotation.Pitch)) *
		FTransform(FVector(0, ( ViewLocation - LookAt ).Size(), 0));

	return Transform.ToMatrixNoScale() * FInverseRotationMatrix(FRotator(0, 90.f, 0));
}

FUMGViewportClient::FUMGViewportClient(FPreviewScene* InPreviewScene)
	: PreviewScene(InPreviewScene)
	, Viewport(nullptr)
	, EngineShowFlags(ESFIM_Game)
{
	FSceneInterface* Scene = GetScene();
	ViewState.Allocate(Scene ? Scene->GetFeatureLevel() : GMaxRHIFeatureLevel);

	BackgroundColor = FColor(55, 55, 55);
}

FUMGViewportClient::~FUMGViewportClient()
{
	if (PreviewScene->GetWorld())
	{
		PreviewScene->GetWorld()->EndPlay(EEndPlayReason::RemovedFromWorld);
	}
}

void FUMGViewportClient::Tick(float InDeltaTime)
{
	if ( !GIntraFrameDebuggingGameThread )
	{
		// Begin Play
		UWorld* PreviewWorld = PreviewScene->GetWorld();
		if ( !PreviewWorld->GetBegunPlay() )
		{
			for ( FActorIterator It(PreviewWorld); It; ++It )
			{
				It->DispatchBeginPlay();
			}
			PreviewWorld->SetBegunPlay(true);
		}

		// Tick
		PreviewWorld->Tick(LEVELTICK_All, InDeltaTime);
	}
}

void FUMGViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	FViewport* ViewportBackup = Viewport;
	Viewport = InViewport ? InViewport : Viewport;

	const bool bIsRealTime = true;

	UWorld* World = GetWorld();
	FGameTime Time;
	if ( bIsRealTime || GetScene() != World->Scene )
	{
		Time = FGameTime::GetTimeSinceAppStart();
	}
	else
	{
		Time = World->GetTime();
	}

	// Setup a FSceneViewFamily/FSceneView for the viewport.
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		Canvas->GetRenderTarget(),
		GetScene(),
		EngineShowFlags)
		.SetTime(Time)
		.SetRealtimeUpdate(bIsRealTime));

	// Get DPI derived view fraction.
	float GlobalResolutionFraction = GetDPIDerivedResolutionFraction();

	// Force screen percentage show flag for High DPI.
	ViewFamily.EngineShowFlags.ScreenPercentage = true;

	//UpdateLightingShowFlags(ViewFamily.EngineShowFlags);

	//ViewFamily.ExposureSettings = ExposureSettings;

	//ViewFamily.LandscapeLODOverride = LandscapeLODOverride;

	FSceneView* View = CalcSceneView(&ViewFamily);

	ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
		ViewFamily, GlobalResolutionFraction));

	CachedViewProjectionMatrix = View->ViewMatrices.GetWorldToClip();
	//SetupViewForRendering(ViewFamily, *View);

	FSlateRect SafeFrame;
	View->CameraConstrainedViewRect = View->UnscaledViewRect;
	//if ( CalculateEditorConstrainedViewRect(SafeFrame, Viewport) )
	//{
	//	View->CameraConstrainedViewRect = FIntRect(SafeFrame.Left, SafeFrame.Top, SafeFrame.Right, SafeFrame.Bottom);
	//}

	if ( IsAspectRatioConstrained() )
	{
		// Clear the background to black if the aspect ratio is constrained, as the scene view won't write to all pixels.
		Canvas->Clear(FLinearColor::Black);
	}

	Canvas->Clear(BackgroundColor);

	// workaround for hacky renderer code that uses GFrameNumber to decide whether to resize render targets
	--GFrameNumber;
	GetRendererModule().BeginRenderingViewFamily(Canvas, &ViewFamily);

	// Remove temporary debug lines.
	// Possibly a hack. Lines may get added without the scene being rendered etc.
	if (World)
	{
		constexpr const UWorld::ELineBatcherType LineBatchersToFlush[] = { UWorld::ELineBatcherType::World, UWorld::ELineBatcherType::Foreground };
		World->FlushLineBatchers(LineBatchersToFlush);
	}
	
	Viewport = ViewportBackup;
}

FSceneInterface* FUMGViewportClient::GetScene() const
{
	UWorld* World = GetWorld();
	if ( World )
	{
		return World->Scene;
	}

	return NULL;
}

UWorld* FUMGViewportClient::GetWorld() const
{
	UWorld* OutWorldPtr = NULL;
	// If we have a valid scene get its world
	if ( PreviewScene )
	{
		OutWorldPtr = PreviewScene->GetWorld();
	}
	if ( OutWorldPtr == NULL )
	{
		OutWorldPtr = GWorld;
	}
	return OutWorldPtr;
}

bool FUMGViewportClient::IsAspectRatioConstrained() const
{
	return ViewInfo.bConstrainAspectRatio;
}

void FUMGViewportClient::SetBackgroundColor(FLinearColor InBackgroundColor)
{
	BackgroundColor = InBackgroundColor;
}

FLinearColor FUMGViewportClient::GetBackgroundColor() const
{
	return BackgroundColor;
}

float FUMGViewportClient::GetOrthoUnitsPerPixel(const FViewport* InViewport) const
{
	const int32 SizeX = InViewport->GetSizeXY().X;

	// 15.0f was coming from the CAMERA_ZOOM_DIV marco, seems it was chosen arbitrarily
	return ( GetOrthoZoom() / ( SizeX * 15.f ) )/* * ComputeOrthoZoomFactor(SizeX)*/;
}

FMatrix FUMGViewportClient::GetViewProjectionMatrix() const
{
	return CachedViewProjectionMatrix;
}

FSceneView* FUMGViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily)
{
	FSceneViewInitOptions ViewInitOptions;

	const FVector& ViewLocation = GetViewLocation();
	const FRotator& ViewRotation = GetViewRotation();

	const FIntPoint ViewportSizeXY = Viewport->GetSizeXY();

	FIntRect ViewRect = FIntRect(0, 0, ViewportSizeXY.X, ViewportSizeXY.Y);
	ViewInitOptions.SetViewRectangle(ViewRect);

	ViewInitOptions.ViewOrigin = ViewLocation;

	ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(ViewRotation);
	ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * FMatrix(
		FPlane(0, 0, 1, 0),
		FPlane(1, 0, 0, 0),
		FPlane(0, 1, 0, 0),
		FPlane(0, 0, 0, 1));

	//@TODO: Should probably be locally configurable (or just made into a FMinimalViewInfo property)
	const EAspectRatioAxisConstraint AspectRatioAxisConstraint = GetDefault<ULocalPlayer>()->AspectRatioAxisConstraint;

	FMinimalViewInfo::CalculateProjectionMatrixGivenView(ViewInfo, AspectRatioAxisConstraint, Viewport, /*inout*/ ViewInitOptions);

	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.SceneViewStateInterface = ViewState.GetReference();
	ViewInitOptions.ViewElementDrawer = this;

	ViewInitOptions.BackgroundColor = GetBackgroundColor();

	//ViewInitOptions.EditorViewBitflag = 0, // send the bit for this view - each actor will check it's visibility bits against this
	//ViewInitOptions.CursorPos = CurrentMousePos;

	FSceneView* View = new FSceneView(ViewInitOptions);

	ViewFamily->Views.Add(View);

	View->StartFinalPostprocessSettings(ViewLocation);

	//OverridePostProcessSettings(*View);

	View->EndFinalPostprocessSettings(ViewInitOptions);

	return View;
}

/////////////////////////////////////////////////////
// SAutoRefreshViewport

class SAutoRefreshViewport : public SViewport
{
	SLATE_BEGIN_ARGS( SAutoRefreshViewport )
		: _bIsEditorPreview(false)
	{
	}
	SLATE_ARGUMENT( bool, bIsEditorPreview )
	SLATE_END_ARGS()
	
	SAutoRefreshViewport() = default;
	

	void Construct(const FArguments& InArgs)
	{
		SViewport::FArguments ParentArgs;
		ParentArgs.IgnoreTextureAlpha(false);
		ParentArgs.EnableBlending(false);
		//ParentArgs.RenderDirectlyToWindow(true);
		SViewport::Construct(ParentArgs);
		
		bIsEditorPreview = InArgs._bIsEditorPreview;
		
		PreviewScene = MakeUnique<FPreviewScene>(FPreviewScene::ConstructionValues().SetEditor(bIsEditorPreview));
		

		ViewportClient = MakeShareable(new FUMGViewportClient(PreviewScene.Get()));
		Viewport = FSceneViewport::Create(ViewportClient, SharedThis(this));
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		Viewport->Invalidate();
		//Viewport->InvalidateDisplay();

		Viewport->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
		ViewportClient->Tick(InDeltaTime);
	}

public:
	/** preview scene */
	
	TUniquePtr<FPreviewScene> PreviewScene;
	
	bool bIsEditorPreview = false;

	TSharedPtr<FUMGViewportClient> ViewportClient;
	
	TSharedPtr<FSceneViewport> Viewport;

};

/////////////////////////////////////////////////////
// UViewport

UViewport::UViewport(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ShowFlags(ESFIM_Game)
{
	bIsVariable = true;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BackgroundColor = FLinearColor::Black;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ShowFlags.DisableAdvancedFeatures();
	//ParentArgs.IgnoreTextureAlpha(false);
	//ParentArgs.EnableBlending(true);
	////ParentArgs.RenderDirectlyToWindow(true);
}

void UViewport::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	ViewportWidget.Reset();
}

TSharedRef<SWidget> UViewport::RebuildWidget()
{
	if ( IsDesignTime() )
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Viewport", "Viewport"))
			];
	}
	else
	{
		ViewportWidget = SNew(SAutoRefreshViewport)
			.bIsEditorPreview(bIsEditorPreview);

		if ( GetChildrenCount() > 0 )
		{
			ViewportWidget->SetContent(GetContentSlot()->Content ? GetContentSlot()->Content->TakeWidget() : SNullWidget::NullWidget);
		}

		return ViewportWidget.ToSharedRef();
	}
}

void UViewport::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if ( ViewportWidget.IsValid() )
	{
		check(ViewportWidget->ViewportClient.IsValid());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ViewportWidget->ViewportClient->SetBackgroundColor(BackgroundColor);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		ViewportWidget->ViewportClient->SetEngineShowFlags(ShowFlags);
	}
}

void UViewport::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( ViewportWidget.IsValid() )
	{
		ViewportWidget->SetContent(InSlot->Content ? InSlot->Content->TakeWidget() : SNullWidget::NullWidget);
	}
}

void UViewport::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( ViewportWidget.IsValid() )
	{
		ViewportWidget->SetContent(SNullWidget::NullWidget);
	}
}

UWorld* UViewport::GetViewportWorld() const
{
	if ( ViewportWidget.IsValid() )
	{
		return ViewportWidget->PreviewScene.Get()->GetWorld();
	}

	return NULL;
}

FVector UViewport::GetViewLocation() const
{
	if ( ViewportWidget.IsValid() )
	{
		return ViewportWidget->ViewportClient->GetViewLocation();
	}

	return FVector();
}

void UViewport::SetViewLocation(FVector Vector)
{
	if ( ViewportWidget.IsValid() )
	{
		ViewportWidget->ViewportClient->SetViewLocation(Vector);
	}
}

FRotator UViewport::GetViewRotation() const
{
	if ( ViewportWidget.IsValid() )
	{
		return ViewportWidget->ViewportClient->GetViewRotation();
	}

	return FRotator();
}

void UViewport::SetViewRotation(FRotator Rotator)
{
	if ( ViewportWidget.IsValid() )
	{
		ViewportWidget->ViewportClient->SetViewRotation(Rotator);
	}
}

AActor* UViewport::Spawn(TSubclassOf<AActor> ActorClass)
{
	if ( ViewportWidget.IsValid() )
	{
		UWorld* World = GetViewportWorld();
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

		return World->SpawnActor<AActor>(ActorClass, FVector::ZeroVector, FRotator::ZeroRotator, SpawnParameters);
	}

	// TODO UMG Report spawning actor error before the world is ready.

	return NULL;
}

void UViewport::SetShowFlag(FString InShowFlagName, bool InValue)
{
	if (ShowFlags.IsNameThere(*InShowFlagName, 0))
	{
		int32 FlagIndex = ShowFlags.FindIndexByName(*InShowFlagName);
		ShowFlags.SetSingleFlag(FlagIndex, InValue);
		ViewportWidget->ViewportClient->SetEngineShowFlags(ShowFlags);
	}
}

FMatrix UViewport::GetViewProjectionMatrix() const
{
	FMatrix ProjectionMatrix = FMatrix::Identity;
	if (ViewportWidget.IsValid())
	{
		check(ViewportWidget->ViewportClient.IsValid());
		ProjectionMatrix = ViewportWidget->ViewportClient->GetViewProjectionMatrix();
	}
	return ProjectionMatrix;
}

void UViewport::SetEnableAdvancedFeatures(bool InEnableAdvancedFeatures)
{
	ShowFlags.DisableAdvancedFeatures();
	if (ViewportWidget.IsValid())
	{
		check(ViewportWidget->ViewportClient.IsValid());
		if (InEnableAdvancedFeatures)
		{
			ShowFlags.EnableAdvancedFeatures();
			ViewportWidget->ViewportClient->SetEngineShowFlags(ShowFlags);
		}
		else
		{
			ShowFlags.DisableAdvancedFeatures();
			ViewportWidget->ViewportClient->SetEngineShowFlags(ShowFlags);
		}

	}
}

void UViewport::SetLightIntensity(float InLightIntensity)
{
	ViewportWidget->PreviewScene.Get()->SetLightBrightness(InLightIntensity);
}

void UViewport::SetSkyIntensity(float InLightIntensity)
{
	ViewportWidget->PreviewScene.Get()->SetSkyBrightness(InLightIntensity);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FLinearColor& UViewport::GetBackgroundColor() const
{
	return BackgroundColor;
}

void UViewport::SetBackgroundColor(const FLinearColor& InColor)
{
	BackgroundColor = InColor;
	if (ViewportWidget.IsValid())
	{
		check(ViewportWidget->ViewportClient.IsValid());
		ViewportWidget->ViewportClient->SetBackgroundColor(BackgroundColor);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UViewport::GetPaletteCategory()
{
	return LOCTEXT("Primitive", "Primitive");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

