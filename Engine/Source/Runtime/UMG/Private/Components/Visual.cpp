// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Visual.h"
#include "Engine/UserInterfaceSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Visual)

/////////////////////////////////////////////////////
// UVisual

UVisual::UVisual(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UVisual::ReleaseSlateResources(bool bReleaseChildren)
{
}

void UVisual::BeginDestroy()
{
	Super::BeginDestroy();

	const bool bReleaseChildren = false;
	ReleaseSlateResources(bReleaseChildren);
}

bool UVisual::NeedsLoadForServer() const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	return UISettings->bLoadWidgetsOnDedicatedServer;
}
