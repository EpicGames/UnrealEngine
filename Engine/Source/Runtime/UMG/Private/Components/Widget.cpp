// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/Widget.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/UObjectToken.h"
#include "CoreGlobals.h"
#include "Widgets/SNullWidget.h"
#include "Types/NavigationMetaData.h"
#include "Widgets/IToolTip.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SOverlay.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "Engine/LocalPlayer.h"
#include "Engine/UserInterfaceSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SToolTip.h"
#include "Binding/PropertyBinding.h"
#include "Binding/States/WidgetStateBitfield.h"
#include "Binding/States/WidgetStateSettings.h"
#include "Binding/States/WidgetStateRegistration.h"
#include "Binding/WidgetFieldNotificationExtension.h"
#include "Logging/MessageLog.h"
#include "Blueprint/GameViewportSubsystem.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/UserWidgetBlueprint.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Slate/SObjectWidget.h"
#include "Blueprint/WidgetTree.h"
#include "UMGStyle.h"
#include "Types/ReflectionMetadata.h"
#include "Trace/SlateMemoryTags.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "Components/NamedSlotInterface.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "HAL/LowLevelMemStats.h"
#include "Extensions/UIComponentContainer.h"
#include "Extensions/UIComponent.h"
#include "Extensions/UIComponentUserWidgetExtension.h"
#include "Types/InvisibleToWidgetReflectorMetaData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Widget)

#define LOCTEXT_NAMESPACE "UMG"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total Created UWidgets"), STAT_SlateUTotalWidgets, STATGROUP_SlateMemory);

/**
* Interface for tool tips.
*/
class FDelegateToolTip : public IToolTip
{
public:

	/**
	* Gets the widget that this tool tip represents.
	*
	* @return The tool tip widget.
	*/
	virtual TSharedRef<class SWidget> AsWidget() override
	{
		return GetContentWidget();
	}

	/**
	* Gets the tool tip's content widget.
	*
	* @return The content widget.
	*/
	virtual TSharedRef<SWidget> GetContentWidget() override
	{
		if ( CachedToolTip.IsValid() )
		{
			return CachedToolTip.ToSharedRef();
		}

		UWidget* Widget = ToolTipWidgetDelegate.Execute();
		if ( Widget )
		{
			CachedToolTip = Widget->TakeWidget();
			return CachedToolTip.ToSharedRef();
		}

		return SNullWidget::NullWidget;
	}

	/**
	* Sets the tool tip's content widget.
	*
	* @param InContentWidget The new content widget to set.
	*/
	virtual void SetContentWidget(const TSharedRef<SWidget>& InContentWidget) override
	{
		CachedToolTip = InContentWidget;
	}

	/**
	* Checks whether this tool tip has no content to display right now.
	*
	* @return true if the tool tip has no content to display, false otherwise.
	*/
	virtual bool IsEmpty() const override
	{
		return !ToolTipWidgetDelegate.IsBound();
	}

	/**
	* Checks whether this tool tip can be made interactive by the user (by holding Ctrl).
	*
	* @return true if it is an interactive tool tip, false otherwise.
	*/
	virtual bool IsInteractive() const override
	{
		return false;
	}

	virtual void OnClosed() override
	{
		//TODO Notify interface implementing widget of closure

		CachedToolTip.Reset();
	}

	virtual void OnOpening() override
	{
		//TODO Notify interface implementing widget of opening
	}

public:
	UWidget::FGetWidget ToolTipWidgetDelegate;

private:
	TSharedPtr<SWidget> CachedToolTip;
};


#if WITH_EDITORONLY_DATA
namespace
{
	void GatherWidgetForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		const UWidget* const Widget = CastChecked<UWidget>(Object);

		EPropertyLocalizationGathererTextFlags WidgetGatherTextFlags = GatherTextFlags;

		// If we've instanced this widget from another asset, then we only want to process the widget itself (to process any overrides against the archetype), but skip all of its children
		if (UObject* WidgetGenerator = Widget->WidgetGeneratedBy.Get())
		{
			if (WidgetGenerator->GetOutermost() != Widget->GetOutermost())
			{
				WidgetGatherTextFlags |= EPropertyLocalizationGathererTextFlags::SkipSubObjects;
			}
		}

		PropertyLocalizationDataGatherer.GatherLocalizationDataFromObject(Widget, WidgetGatherTextFlags);
	}
}
#endif


/////////////////////////////////////////////////////
// UWidget

TArray<TSubclassOf<UPropertyBinding>> UWidget::BinderClasses;


/////////////////////////////////////////////////////
void UWidget::FFieldNotificationClassDescriptor::ForEachField(const UClass* Class, TFunctionRef<bool(::UE::FieldNotification::FFieldId FielId)> Callback) const
{
	for (int32 Index = 0; Index < Max_IndexOf_; ++Index)
	{
		if (!Callback(*AllFields[Index]))
		{
			return;
		}
	}
	if (const UBlueprintGeneratedClass* BPClass = Cast<const UBlueprintGeneratedClass>(Class))
	{
		BPClass->ForEachFieldNotify(Callback, true);
	}
}
UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_ThreeFields(UWidget, ToolTipText, Visibility, bIsEnabled);


UWidget::UWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bIsEnabled = true;
	bIsVariable = true;
	bIsManagedByGameViewportSubsystem = false;
#if WITH_EDITOR
	DesignerFlags = static_cast<uint8>(EWidgetDesignFlags::None);
#endif
	Visibility = ESlateVisibility::Visible;
	RenderOpacity = 1.0f;
	RenderTransformPivot = FVector2D(0.5f, 0.5f);
	Cursor = EMouseCursor::Default;
	PixelSnapping = EWidgetPixelSnapping::Inherit;

#if WITH_EDITORONLY_DATA
	bOverrideAccessibleDefaults = false;
	AccessibleBehavior = ESlateAccessibleBehavior::NotAccessible;
	AccessibleSummaryBehavior = ESlateAccessibleBehavior::Auto;
	bCanChildrenBeAccessible = true;
#endif
	AccessibleWidgetData = nullptr;

	bShouldBroadcastState = true;
	bWidgetStateInitialized = false;
	bWrappedByComponent = false;

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UWidget::StaticClass(), &GatherWidgetForLocalization); }
#endif

	INC_DWORD_STAT(STAT_SlateUTotalWidgets);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FWidgetTransform& UWidget::GetRenderTransform() const
{
	return RenderTransform;
}

void UWidget::SetRenderTransform(FWidgetTransform Transform)
{
	RenderTransform = Transform;
	UpdateRenderTransform();
}

void UWidget::SetRenderScale(FVector2D Scale)
{
	RenderTransform.Scale = Scale;
	UpdateRenderTransform();
}

void UWidget::SetRenderShear(FVector2D Shear)
{
	RenderTransform.Shear = Shear;
	UpdateRenderTransform();
}

void UWidget::SetRenderTransformAngle(float Angle)
{
	RenderTransform.Angle = Angle;
	UpdateRenderTransform();
}

float UWidget::GetRenderTransformAngle() const
{
	return RenderTransform.Angle;
}

void UWidget::SetRenderTranslation(FVector2D Translation)
{
	RenderTransform.Translation = Translation;
	UpdateRenderTransform();
}

void UWidget::UpdateRenderTransform()
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		if (RenderTransform.IsIdentity())
		{
			SafeWidget->SetRenderTransform(TOptional<FSlateRenderTransform>());
		}
		else
		{
			SafeWidget->SetRenderTransform(RenderTransform.ToSlateRenderTransform());
		}
	}
}

FVector2D UWidget::GetRenderTransformPivot() const
{
	return RenderTransformPivot;
}

void UWidget::SetRenderTransformPivot(FVector2D Pivot)
{
	RenderTransformPivot = Pivot;

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SafeWidget->SetRenderTransformPivot(Pivot);
	}
}

EFlowDirectionPreference UWidget::GetFlowDirectionPreference() const
{
	return FlowDirectionPreference;
}

void UWidget::SetFlowDirectionPreference(EFlowDirectionPreference FlowDirection)
{
	FlowDirectionPreference = FlowDirection;
	
	if (TSharedPtr<SWidget> SafeWidget = GetCachedWidget())
	{
		SafeWidget->SetFlowDirectionPreference(FlowDirectionPreference);
	}
}

bool UWidget::GetIsEnabled() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	return SafeWidget.IsValid() ? SafeWidget->IsEnabled() : bIsEnabled;
}

void UWidget::SetIsEnabled(bool bInIsEnabled)
{
	bool bValueChanged = false;
	if (bIsEnabled != bInIsEnabled)
	{
		bIsEnabled = bInIsEnabled;
		bValueChanged = true;
	}

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SafeWidget->SetEnabled(bInIsEnabled);
	}

	if (bValueChanged)
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::bIsEnabled);
		// Note: State is disabled, so we broadcast !bIsEnabled
		BroadcastBinaryPostStateChange(UWidgetDisabledStateRegistration::Bit, !bIsEnabled);
	}
}

bool UWidget::IsInViewport() const
{
	if (bIsManagedByGameViewportSubsystem)
	{
		if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
		{
			return Subsystem->IsWidgetAdded(this);
		}
	}
	return false;
}

EMouseCursor::Type UWidget::GetCursor() const
{
	return Cursor;
}

void UWidget::SetCursor(EMouseCursor::Type InCursor)
{
	bOverride_Cursor = true;
	Cursor = InCursor;

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		SafeWidget->SetCursor(Cursor);
	}
}

void UWidget::ResetCursor()
{
	bOverride_Cursor = false;

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		SafeWidget->SetCursor(TOptional<EMouseCursor::Type>());
	}
}

bool UWidget::IsRendered() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		return SafeWidget->GetVisibility().IsVisible() && SafeWidget->GetRenderOpacity() > 0.0f;
	}

	return false;
}

bool UWidget::IsVisible() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		return SafeWidget->GetVisibility().IsVisible();
	}

	return false;
}

ESlateVisibility UWidget::GetVisibility() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		return UWidget::ConvertRuntimeToSerializedVisibility(SafeWidget->GetVisibility());
	}

	return Visibility;
}

void UWidget::SetVisibility(ESlateVisibility InVisibility)
{
	SetVisibilityInternal(InVisibility);
}

void UWidget::SetVisibilityInternal(ESlateVisibility InVisibility)
{
	const bool bVisibilityChanged = Visibility != InVisibility;
	if (bVisibilityChanged)
	{
		Visibility = InVisibility;
	}

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		EVisibility SlateVisibility = UWidget::ConvertSerializedVisibilityToRuntime(InVisibility);
		SafeWidget->SetVisibility(SlateVisibility);
	}

	if (bVisibilityChanged)
	{
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Visibility);
	}
}

float UWidget::GetRenderOpacity() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		return SafeWidget->GetRenderOpacity();
	}

	return RenderOpacity;
}

void UWidget::SetRenderOpacity(float InRenderOpacity)
{
	RenderOpacity = InRenderOpacity;

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SafeWidget->SetRenderOpacity(InRenderOpacity);
	}
}

EWidgetClipping UWidget::GetClipping() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		return SafeWidget->GetClipping();
	}

	return Clipping;
}

void UWidget::SetClipping(EWidgetClipping InClipping)
{
	Clipping = InClipping;

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SafeWidget->SetClipping(InClipping);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

EWidgetPixelSnapping UWidget::GetPixelSnapping() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		return SafeWidget->GetPixelSnapping();
	}

	return PixelSnapping;
}

void UWidget::SetPixelSnapping(EWidgetPixelSnapping InPixelSnappingMethod)
{
	PixelSnapping = InPixelSnappingMethod;

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SafeWidget->SetPixelSnapping(InPixelSnappingMethod);
	}
}

void UWidget::ForceVolatile(bool bForce)
{
	bIsVolatile = bForce;
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		SafeWidget->ForceVolatile(bForce);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FText UWidget::GetToolTipText() const
{
	return ToolTipText;
}

void UWidget::SetToolTipText(const FText& InToolTipText)
{
	ToolTipText = InToolTipText;
	BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::ToolTipText);

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SafeWidget->SetToolTipText(InToolTipText);
	}
}

UWidget* UWidget::GetToolTip() const
{
	return ToolTipWidget;
}

void UWidget::SetToolTip(UWidget* InToolTipWidget)
{
	ToolTipWidget = InToolTipWidget;

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		if ( ToolTipWidget )
		{
			TSharedRef<SToolTip> ToolTip = SNew(SToolTip)
				.TextMargin(FMargin(0))
				.BorderImage(nullptr)
				[
					ToolTipWidget->TakeWidget()
				];

			SafeWidget->SetToolTip(ToolTip);
		}
		else
		{
			SafeWidget->SetToolTip(TSharedPtr<IToolTip>());
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UWidget::IsHovered() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	return SafeWidget.IsValid() && SafeWidget->IsHovered();
}

bool UWidget::HasKeyboardFocus() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	return SafeWidget.IsValid() && SafeWidget->HasKeyboardFocus();
}

bool UWidget::SupportsKeyboardFocus() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	return SafeWidget.IsValid() && SafeWidget->SupportsKeyboardFocus();
}

bool UWidget::HasMouseCapture() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	return SafeWidget.IsValid() && SafeWidget->HasMouseCapture();
}

bool UWidget::HasMouseCaptureByUser(int32 UserIndex, int32 PointerIndex) const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	return SafeWidget.IsValid() && SafeWidget->HasMouseCaptureByUser(UserIndex, PointerIndex >= 0 ? PointerIndex : TOptional<int32>());
}

void UWidget::SetKeyboardFocus()
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if ( !SafeWidget->SupportsKeyboardFocus() )
		{
			FMessageLog("PIE").Warning(FText::Format(LOCTEXT("ThisWidgetDoesntSupportFocus", "The widget {0} does not support focus. If this is a UserWidget, you should set bIsFocusable to true."), FText::FromString(GetNameSafe(this))));
		}
#endif

		if ( !FSlateApplication::Get().SetKeyboardFocus(SafeWidget) )
		{
			if ( UWorld* World = GetWorld() )
			{
				if ( ULocalPlayer* LocalPlayer = World->GetFirstLocalPlayerFromController() )
				{
					LocalPlayer->GetSlateOperations().SetUserFocus(SafeWidget.ToSharedRef(), EFocusCause::SetDirectly);
				}
			}
		}
	}
}

bool UWidget::HasUserFocus(APlayerController* PlayerController) const
{
	if (PlayerController == nullptr || PlayerController->Player == nullptr || !PlayerController->IsLocalPlayerController())
	{
		return false;
	}

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		FLocalPlayerContext Context(PlayerController);

		if ( ULocalPlayer* LocalPlayer = Context.GetLocalPlayer() )
		{
			TOptional<int32> UserIndex = FSlateApplication::Get().GetUserIndexForController(LocalPlayer->GetControllerId());
			if (UserIndex.IsSet())
			{
				TOptional<EFocusCause> FocusCause = SafeWidget->HasUserFocus(UserIndex.GetValue());
				return FocusCause.IsSet();
			}
		}
	}

	return false;
}

bool UWidget::HasAnyUserFocus() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		TOptional<EFocusCause> FocusCause = SafeWidget->HasAnyUserFocus();
		return FocusCause.IsSet();
	}

	return false;
}

bool UWidget::HasFocusedDescendants() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		return SafeWidget->HasFocusedDescendants();
	}

	return false;
}

bool UWidget::HasUserFocusedDescendants(APlayerController* PlayerController) const
{
	if ( PlayerController == nullptr || !PlayerController->IsLocalPlayerController() )
	{
		return false;
	}

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		FLocalPlayerContext Context(PlayerController);

		if ( ULocalPlayer* LocalPlayer = Context.GetLocalPlayer() )
		{
			TOptional<int32> UserIndex = FSlateApplication::Get().GetUserIndexForController(LocalPlayer->GetControllerId());
			if (UserIndex.IsSet())
			{
				return SafeWidget->HasUserFocusedDescendants(UserIndex.GetValue());
			}
		}
	}

	return false;
}

void UWidget::SetFocus()
{
	SetUserFocus(GetOwningPlayer());
}

void UWidget::SetUserFocus(APlayerController* PlayerController)
{
	if ( PlayerController == nullptr || !PlayerController->IsLocalPlayerController() || PlayerController->Player == nullptr )
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FMessageLog("PIE").Error()->AddToken(FTextToken::Create(LOCTEXT("NoPlayerControllerToFocus", "The PlayerController is not a valid local player so it can't focus on ")))->AddToken(FUObjectToken::Create(this));
#endif
		return;
	}

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if ( !SafeWidget->SupportsKeyboardFocus() )
		{
			TSharedRef<FTokenizedMessage> Message = FMessageLog("PIE").Warning()->AddToken(FUObjectToken::Create(this));
#if WITH_EDITORONLY_DATA
			if(UObject* GeneratedBy = WidgetGeneratedBy.Get())
			{
				Message->AddToken(FTextToken::Create(FText::FromString(TEXT(" in "))))->AddToken(FUObjectToken::Create(GeneratedBy));
			}
#endif
			if (IsA(UUserWidget::StaticClass()))
			{
				Message->AddToken(FTextToken::Create(LOCTEXT("UserWidgetDoesntSupportFocus", " does not support focus, you should set bIsFocusable to true.")));
			}
			else
			{
				Message->AddToken(FTextToken::Create(LOCTEXT("NonUserWidgetDoesntSupportFocus", " does not support focus.")));
			}
			
		}
#endif

		if ( ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer() )
		{
			TOptional<int32> UserIndex = FSlateApplication::Get().GetUserIndexForController(LocalPlayer->GetControllerId());
			if (UserIndex.IsSet())
			{
				FReply& DelayedSlateOperations = LocalPlayer->GetSlateOperations();
				if (FSlateApplication::Get().SetUserFocus(UserIndex.GetValue(), SafeWidget))
				{
					DelayedSlateOperations.CancelFocusRequest();
				}
				else
				{
					DelayedSlateOperations.SetUserFocus(SafeWidget.ToSharedRef());
				}
				
			}
		}
	}
}

void UWidget::ForceLayoutPrepass()
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SafeWidget->MarkPrepassAsDirty();
		SafeWidget->SlatePrepass(SafeWidget->GetTickSpaceGeometry().Scale);
	}
}

void UWidget::InvalidateLayoutAndVolatility()
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		SafeWidget->Invalidate(EInvalidateWidgetReason::LayoutAndVolatility);
	}
}

FVector2D UWidget::GetDesiredSize() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		return SafeWidget->GetDesiredSize();
	}

	return FVector2D(0, 0);
}

void UWidget::SetNavigationRuleInternal(EUINavigation Direction, EUINavigationRule Rule, FName WidgetToFocus/* = NAME_None*/, UWidget* InWidget/* = nullptr*/, FCustomWidgetNavigationDelegate InCustomDelegate/* = FCustomWidgetNavigationDelegate()*/)
{
	if (Navigation == nullptr)
	{
		Navigation = NewObject<UWidgetNavigation>(this);
	}

	FWidgetNavigationData NavigationData;
	NavigationData.Rule = Rule;
	NavigationData.WidgetToFocus = WidgetToFocus;
	NavigationData.Widget = InWidget;
	NavigationData.CustomDelegate = InCustomDelegate;
	switch(Direction)
	{
		case EUINavigation::Up:
			Navigation->Up = NavigationData;
			break;
		case EUINavigation::Down:
			Navigation->Down = NavigationData;
			break;
		case EUINavigation::Left:
			Navigation->Left = NavigationData;
			break;
		case EUINavigation::Right:
			Navigation->Right = NavigationData;
			break;
		case EUINavigation::Next:
			Navigation->Next = NavigationData;
			break;
		case EUINavigation::Previous:
			Navigation->Previous = NavigationData;
			break;
		default:
			break;
	}
}

void UWidget::SetNavigationRule(EUINavigation Direction, EUINavigationRule Rule, FName WidgetToFocus)
{
	SetNavigationRuleInternal(Direction, Rule, WidgetToFocus);
	BuildNavigation();
}

void UWidget::SetNavigationRuleBase(EUINavigation Direction, EUINavigationRule Rule)
{
	if (Rule == EUINavigationRule::Explicit || Rule == EUINavigationRule::Custom || Rule == EUINavigationRule::CustomBoundary)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FMessageLog("PIE").Error(LOCTEXT("SetNavigationRuleBaseWrongRule", "Cannot use SetNavigationRuleBase with an Explicit or a Custom or a CustomBoundary Rule."));
#endif
		return;
	}
	SetNavigationRuleInternal(Direction, Rule);
	BuildNavigation();
}

void UWidget::SetNavigationRuleExplicit(EUINavigation Direction, UWidget* InWidget)
{
	SetNavigationRuleInternal(Direction, EUINavigationRule::Explicit, NAME_None, InWidget);
	BuildNavigation();
}

void UWidget::SetNavigationRuleCustom(EUINavigation Direction, FCustomWidgetNavigationDelegate InCustomDelegate)
{
	SetNavigationRuleInternal(Direction, EUINavigationRule::Custom, NAME_None, nullptr, InCustomDelegate);
	BuildNavigation();
}

void UWidget::SetNavigationRuleCustomBoundary(EUINavigation Direction, FCustomWidgetNavigationDelegate InCustomDelegate)
{
	SetNavigationRuleInternal(Direction, EUINavigationRule::CustomBoundary, NAME_None, nullptr, InCustomDelegate);
	BuildNavigation();
}

void UWidget::SetAllNavigationRules(EUINavigationRule Rule, FName WidgetToFocus)
{
	SetNavigationRuleInternal(EUINavigation::Up, Rule, WidgetToFocus);
	SetNavigationRuleInternal(EUINavigation::Down, Rule, WidgetToFocus);
	SetNavigationRuleInternal(EUINavigation::Left, Rule, WidgetToFocus);
	SetNavigationRuleInternal(EUINavigation::Right, Rule, WidgetToFocus);
	SetNavigationRuleInternal(EUINavigation::Next, Rule, WidgetToFocus);
	SetNavigationRuleInternal(EUINavigation::Previous, Rule, WidgetToFocus);
	BuildNavigation();
}

void UWidget::SetNavigationMethod(const TInstancedStruct<FNavigationMethod>& NavigationMethod)
{
	if (Navigation == nullptr)
	{
		Navigation = NewObject<UWidgetNavigation>(this);
	}

	Navigation->NavigationMethod = NavigationMethod;
	
	BuildNavigation();
}

UPanelWidget* UWidget::GetParent() const
{
	if ( Slot )
	{
		return Slot->Parent;
	}

	return nullptr;
}

void UWidget::RemoveFromParent()
{
	if (!HasAnyFlags(RF_BeginDestroyed))
	{
		if (bIsManagedByGameViewportSubsystem)
		{
			if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
			{
				Subsystem->RemoveWidget(this);
			}
		}
		else if (UPanelWidget* CurrentParent = GetParent())
		{
			CurrentParent->RemoveChild(this);
		}
		else
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (GetCachedWidget().IsValid() && GetCachedWidget()->GetParentWidget().IsValid() && !IsDesignTime())
			{
				FText WarningMessage = FText::Format(LOCTEXT("RemoveFromParentWithNoParent", "UWidget::RemoveFromParent() called on '{0}' which has no UMG parent (if it was added directly to a native Slate widget via TakeWidget() then it must be removed explicitly rather than via RemoveFromParent())"), FText::AsCultureInvariant(GetPathName()));
				// @todo: nickd - we need to switch this back to a warning in engine, but info for games
				FMessageLog("PIE").Info(WarningMessage);
			}
#endif
		}
	}
}

const FGeometry& UWidget::GetCachedGeometry() const
{
	return GetTickSpaceGeometry();
}

const FGeometry& UWidget::GetTickSpaceGeometry() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( SafeWidget.IsValid() )
	{
		return SafeWidget->GetTickSpaceGeometry();
	}

	return SNullWidget::NullWidget->GetTickSpaceGeometry();
}

const FGeometry& UWidget::GetPaintSpaceGeometry() const
{
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		return SafeWidget->GetPaintSpaceGeometry();
	}

	return SNullWidget::NullWidget->GetPaintSpaceGeometry();
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void UWidget::VerifySynchronizeProperties()
{
	ensureMsgf(bRoutedSynchronizeProperties, TEXT("%s failed to route SynchronizeProperties.  Please call Super::SynchronizeProperties() in your <className>::SynchronizeProperties() function."), *GetFullName());
}
#endif

void UWidget::OnWidgetRebuilt()
{
}

TSharedRef<SWidget> UWidget::TakeWidget()
{
	LLM_SCOPE_BYTAG(UI_UMG);

#if WIDGET_INCLUDE_RELFECTION_METADATA
	UObject* SourceAsset = GetSourceAssetOrClass();
	UClass* WidgetClass = GetClass();
	if(SourceAsset && WidgetClass)
	{
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(SourceAsset->GetPackage(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(WidgetClass, ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET(SourceAsset, WidgetClass);

		return TakeWidget_Private([](UUserWidget* Widget, TSharedRef<SWidget> Content) -> TSharedPtr<SObjectWidget> {
			return SNew(SObjectWidget, Widget)[Content];
			});
	}
#endif
	return TakeWidget_Private([](UUserWidget* Widget, TSharedRef<SWidget> Content) -> TSharedPtr<SObjectWidget> {
		return SNew(SObjectWidget, Widget)[Content];
		});
}

TSharedRef<SWidget> UWidget::TakeWidget_Private(ConstructMethodType ConstructMethod)
{
	bool bNewlyCreated = false;
	TSharedPtr<SWidget> PublicWidget;

	// If the underlying widget doesn't exist we need to construct and cache the widget for the first run.
	if (!MyWidget.IsValid())
	{
		PublicWidget = RebuildWidget();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		ensureMsgf(PublicWidget.Get() != &SNullWidget::NullWidget.Get(), TEXT("Don't return SNullWidget from RebuildWidget, because we mutate the state of the return.  Return a SSpacer if you need to return a no-op widget."));
#endif

		MyWidget = PublicWidget;

		bNewlyCreated = true;
	}
	else
	{
		PublicWidget = MyWidget.Pin();
	}

	// If it is a user widget wrap it in a SObjectWidget to keep the instance from being GC'ed
	if (IsA(UUserWidget::StaticClass()))
	{
		TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();

		// If the GC Widget is still valid we still exist in the slate hierarchy, so just return the GC Widget.
		if (SafeGCWidget.IsValid())
		{
			ensure(bNewlyCreated == false);
			PublicWidget = SafeGCWidget;
		}
		else // Otherwise we need to recreate the wrapper widget
		{
			SafeGCWidget = ConstructMethod(Cast<UUserWidget>(this), PublicWidget.ToSharedRef());

			MyGCWidget = SafeGCWidget;
			PublicWidget = SafeGCWidget;
		}
	}
	if (bWrappedByComponent)
	{
		// Now, if the Widget instance is wrapped by a component, add it to the hierarchy
		const UWidgetTree* WidgetTree = Cast<UWidgetTree>(GetOuter());
		if (const UUserWidget* UserWidget = WidgetTree ? Cast<UUserWidget>(WidgetTree->GetOuter()) : nullptr)
		{
			if (UUIComponentUserWidgetExtension* Extension = UserWidget->GetExtension<UUIComponentUserWidgetExtension>())
			{
				for (UUIComponent* Component : Extension->GetComponentsFor(this))
				{
					if (bNewlyCreated)
					{
						TSharedRef<SWidget> SafeComponentWidget = Component->RebuildWidgetWithContent(PublicWidget.ToSharedRef());
						// If the Widget is the same of null, we do nothing 
						if (SafeComponentWidget != PublicWidget)
						{
#if WITH_EDITOR
							if (IsDesignTime())
							{
								SafeComponentWidget->SetVisibility(EVisibility::Visible);
							}
#endif						
							ComponentWrapperWidget = SafeComponentWidget;
							PublicWidget = SafeComponentWidget;
						}
					}
					else if (ComponentWrapperWidget.IsValid())
					{
						PublicWidget = ComponentWrapperWidget.Pin();
					}
				}			
			}
		}
	}

#if WITH_EDITOR
	if (IsDesignTime())
	{
		if (bNewlyCreated)
		{
			TSharedRef<SWidget> SafeDesignWidget = RebuildDesignWidget(PublicWidget.ToSharedRef());
			if (SafeDesignWidget != PublicWidget)
			{
				DesignWrapperWidget = SafeDesignWidget;
				PublicWidget = SafeDesignWidget;
			}
		}
		else if (DesignWrapperWidget.IsValid())
		{
			PublicWidget = DesignWrapperWidget.Pin();
		}
	}
#endif

	if (bNewlyCreated)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		bRoutedSynchronizeProperties = false;
#endif

#if WIDGET_INCLUDE_RELFECTION_METADATA
		UObject* SourceAsset = GetSourceAssetOrClass();
		UClass* WidgetClass = GetClass();
		// We only need to do this once, when the slate widget is created.
		PublicWidget->AddMetadata<FReflectionMetaData>(MakeShared<FReflectionMetaData>(GetFName(), WidgetClass, this, SourceAsset));
#endif

		SynchronizeProperties();
		VerifySynchronizeProperties();
		OnWidgetRebuilt();
	}

	return PublicWidget.ToSharedRef();
}

TSharedPtr<SWidget> UWidget::GetCachedWidget() const
{	
	if (MyGCWidget.IsValid())
	{
		return MyGCWidget.Pin();
	}

	return MyWidget.Pin();
}

TSharedPtr<SWidget> UWidget::GetCachedWrappedWidget() const
{	
#if WITH_EDITOR
	if (DesignWrapperWidget.IsValid())
	{
		return DesignWrapperWidget.Pin();
	}
#endif //WITH_EDITOR
	
	if (ComponentWrapperWidget.IsValid())
	{
		return ComponentWrapperWidget.Pin();
	}
	return GetCachedWidget();
}

bool UWidget::IsConstructed() const
{
	const TSharedPtr<SWidget>& SafeWidget = GetCachedWidget();
	return SafeWidget.IsValid();
}

#if WITH_EDITOR

TSharedRef<SWidget> UWidget::RebuildDesignWidget(TSharedRef<SWidget> Content)
{
	return Content;
}

TSharedRef<SWidget> UWidget::CreateDesignerOutline(TSharedRef<SWidget> Content) const
{
	return SNew(SOverlay)
		.Visibility(EVisibility::Visible)
		.AddMetaData<FInvisibleToWidgetReflectorMetaData>(FInvisibleToWidgetReflectorMetaData())
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			Content
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SBorder)
			.AddMetaData<FInvisibleToWidgetReflectorMetaData>(FInvisibleToWidgetReflectorMetaData())
			.DesiredSizeScale(FVector2D(0.0f, 0.0f))
			.Visibility(HasAnyDesignerFlags(EWidgetDesignFlags::ShowOutline) ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
			.BorderImage(FUMGStyle::Get().GetBrush("MarchingAnts"))
		];
}

#endif

UGameInstance* UWidget::GetGameInstance() const
{
	if (UWorld* World = GetWorld())
	{
		return World->GetGameInstance();
	}

	return nullptr;
}

APlayerController* UWidget::GetOwningPlayer() const
{
	UWidgetTree* WidgetTree = Cast<UWidgetTree>(GetOuter());
	if (UUserWidget* UserWidget = WidgetTree ? Cast<UUserWidget>(WidgetTree->GetOuter()) : nullptr)
	{
		return UserWidget->GetOwningPlayer();
	}
	return nullptr;
}

ULocalPlayer* UWidget::GetOwningLocalPlayer() const
{
	UWidgetTree* WidgetTree = Cast<UWidgetTree>(GetOuter());
	if (UUserWidget* UserWidget = WidgetTree ? Cast<UUserWidget>(WidgetTree->GetOuter()) : nullptr)
	{
		return UserWidget->GetOwningLocalPlayer();
	}
	return nullptr;
}

#if WITH_EDITOR
#undef LOCTEXT_NAMESPACE
#define LOCTEXT_NAMESPACE "UMGEditor"

void UWidget::SetDesignerFlags(EWidgetDesignFlags NewFlags)
{
	DesignerFlags = static_cast<uint8>(GetDesignerFlags() | NewFlags);

	INamedSlotInterface* NamedSlotWidget = Cast<INamedSlotInterface>(this);
	if (NamedSlotWidget)
	{
		NamedSlotWidget->SetNamedSlotDesignerFlags(NewFlags);
	}
}

void UWidget::SetDisplayLabel(const FString& InDisplayLabel)
{
	DisplayLabel = InDisplayLabel;
}

const FString& UWidget::GetCategoryName() const
{
	return CategoryName;
}

void UWidget::SetCategoryName(const FString& InValue)
{
	CategoryName = InValue;
}

bool UWidget::IsGeneratedName() const
{
	if (!DisplayLabel.IsEmpty())
	{
		return false;
	}

	FString Name = GetName();

	if (Name == GetClass()->GetName() || Name.StartsWith(GetClass()->GetName() + TEXT("_")))
	{
		return true;
	}
	else if (GetClass()->ClassGeneratedBy != nullptr)
	{
		FString BaseNameForBP = GetClass()->GetName();
		BaseNameForBP.RemoveFromEnd(TEXT("_C"), ESearchCase::CaseSensitive);

		if (Name == BaseNameForBP || Name.StartsWith(BaseNameForBP + TEXT("_")))
		{
			return true;
		}
	}

	return false;
}

FString UWidget::GetLabelMetadata() const
{
	return TEXT("");
}

FText UWidget::GetLabelText() const
{
	return GetDisplayNameBase();
}

FText UWidget::GetLabelTextWithMetadata() const
{
	FText Label = GetDisplayNameBase();

	if (!bIsVariable || !GetLabelMetadata().IsEmpty())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BaseName"), Label);
		Args.Add(TEXT("Metadata"), FText::FromString(GetLabelMetadata()));
		Label = FText::Format(LOCTEXT("NonVariableLabelFormat", "[{BaseName}]{Metadata}"), Args);
	}

	return Label;
}

FText UWidget::GetDisplayNameBase() const
{
	const bool bHasDisplayLabel = !DisplayLabel.IsEmpty();
	if (IsGeneratedName() && !bIsVariable)
	{
		return GetClass()->GetDisplayNameText();
	}
	else
	{
		return FText::FromString(bHasDisplayLabel ? DisplayLabel : GetName());
	}
}

const FText UWidget::GetPaletteCategory()
{
	return LOCTEXT("Uncategorized", "Uncategorized");
}

void UWidget::CreatedFromPalette()
{
	// Allowing the variable creation if the setting allows it.
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	if (!UISettings->bAuthorizeAutomaticWidgetVariableCreation)
	{
		bIsVariable = false;
	}

	OnCreationFromPalette();
}

EVisibility UWidget::GetVisibilityInDesigner() const
{
	return bHiddenInDesigner ? EVisibility::Collapsed : EVisibility::Visible;
}

bool UWidget::IsEditorWidget() const
{
	if (UWidgetTree* WidgetTree = Cast<UWidgetTree>(GetOuter()))
	{
		//@TODO: DarenC - This is a bit dirty, can't find a cleaner alternative yet though.
		bool bIsEditorWidgetPreview = WidgetTree->RootWidget && WidgetTree->RootWidget->WidgetGeneratedBy.IsValid();
		UObject* WidgetBPObject = bIsEditorWidgetPreview ? WidgetTree->RootWidget->WidgetGeneratedBy.Get() : WidgetTree->GetOuter();

		if (UUserWidgetBlueprint* WidgetBP = Cast<UUserWidgetBlueprint>(WidgetBPObject))
		{
			return WidgetBP->AllowEditorWidget();
		}
		else if (UUserWidget* UserWidget = Cast<UUserWidget>(WidgetBPObject))
		{
			return UserWidget->IsEditorUtility();
		}
	}

	return false;
}

bool UWidget::IsVisibleInDesigner() const
{
	if (bHiddenInDesigner)
	{
		return false;
	}

	UWidget* Parent = GetParent();
	while (Parent != nullptr)
	{
		if (Parent->bHiddenInDesigner)
		{
			return false;
		}

		Parent = Parent->GetParent();
	}

	return true;
}

void UWidget::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if (SafeWidget.IsValid())
	{
		SynchronizeProperties();
	}
	else
	{
		SynchronizeAccessibleData();
	}
}

void UWidget::SelectByDesigner()
{
	OnSelectedByDesigner();

	UWidget* Parent = GetParent();
	while ( Parent != nullptr )
	{
		Parent->OnDescendantSelectedByDesigner(this);
		Parent = Parent->GetParent();
	}
}

void UWidget::DeselectByDesigner()
{
	OnDeselectedByDesigner();

	UWidget* Parent = GetParent();
	while ( Parent != nullptr )
	{
		Parent->OnDescendantDeselectedByDesigner(this);
		Parent = Parent->GetParent();
	}
}

#undef LOCTEXT_NAMESPACE
#define LOCTEXT_NAMESPACE "UMG"
#endif // WITH_EDITOR

void UWidget::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// This is a failsafe to make sure all the accessibility data is copied over in case
	// some rare instance isn't handled by SynchronizeProperties. It might not be necessary.
	SynchronizeAccessibleData();
}

#if WITH_EDITOR
bool UWidget::Modify(bool bAlwaysMarkDirty)
{
	bool Modified = Super::Modify(bAlwaysMarkDirty);

	if ( Slot )
	{
		Slot->SetFlags(RF_Transactional);
		Modified |= Slot->Modify(bAlwaysMarkDirty);
	}

	return Modified;
}
#endif

bool UWidget::IsChildOf(UWidget* PossibleParent)
{
	UPanelWidget* Parent = GetParent();
	if ( Parent == nullptr )
	{
		return false;
	}
	else if ( Parent == PossibleParent )
	{
		return true;
	}
	
	return Parent->IsChildOf(PossibleParent);
}

TSharedRef<SWidget> UWidget::RebuildWidget()
{
	ensureMsgf(false, TEXT("You must implement RebuildWidget() in your child class"));
	return SNew(SSpacer);
}

void UWidget::SynchronizeProperties()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bRoutedSynchronizeProperties = true;
#endif

	// Always sync accessible data even if the SWidget doesn't exist
	SynchronizeAccessibleData();

	// We want to apply the bindings to the cached widget, which could be the SWidget, or the SObjectWidget, 
	// in the case where it's a user widget.  We always want to prefer the SObjectWidget so that bindings to 
	// visibility and enabled status are not stomping values setup in the root widget in the User Widget.
	TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
	if ( !SafeWidget.IsValid() )
	{
		return;
	}

#if WITH_EDITOR
	TSharedPtr<SWidget> SafeContentWidget = MyGCWidget.IsValid() ? MyGCWidget.Pin() : MyWidget.Pin();
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITOR
	// Always use an enabled and visible state in the designer.
	if ( IsDesignTime() )
	{
		SafeWidget->SetEnabled(true);
		SafeWidget->SetVisibility(BIND_UOBJECT_ATTRIBUTE(EVisibility, GetVisibilityInDesigner));
	}
	else 
#endif
	{
		if ( bOverride_Cursor /*|| CursorDelegate.IsBound()*/ )
		{
			SafeWidget->SetCursor(Cursor);// PROPERTY_BINDING(EMouseCursor::Type, Cursor));
		}

		SafeWidget->SetEnabled(BITFIELD_PROPERTY_BINDING( bIsEnabled ));
		SafeWidget->SetVisibility(OPTIONAL_BINDING_CONVERT(ESlateVisibility, Visibility, EVisibility, ConvertVisibility));
	}

#if WITH_EDITOR
	// In the designer, we need to apply the clip to bounds flag to the real widget, not the designer outline.
	// because we may be changing a critical default set on the base that not actually set on the outline.
	// An example of this, would be changing the clipping bounds on a scrollbox.  The outline never clipped to bounds
	// so unless we tweak the -actual- value on the SScrollBox, the user won't see a difference in how the widget clips.
	SafeContentWidget->SetClipping(Clipping);
#else
	SafeWidget->SetClipping(Clipping);
#endif

	SafeWidget->SetPixelSnapping(PixelSnapping);
	SafeWidget->SetFlowDirectionPreference(FlowDirectionPreference);

	SafeWidget->ForceVolatile(bIsVolatile);

	SafeWidget->SetRenderOpacity(RenderOpacity);

	UpdateRenderTransform();
	SafeWidget->SetRenderTransformPivot(RenderTransformPivot);

	if ( ToolTipWidgetDelegate.IsBound() && !IsDesignTime() )
	{
		TSharedRef<FDelegateToolTip> ToolTip = MakeShareable(new FDelegateToolTip());
		ToolTip->ToolTipWidgetDelegate = ToolTipWidgetDelegate;
		SafeWidget->SetToolTip(ToolTip);
	}
	else if ( ToolTipWidget != nullptr )
	{
		TSharedRef<SToolTip> ToolTip = SNew(SToolTip)
			.TextMargin(FMargin(0))
			.BorderImage(nullptr)
			[
				ToolTipWidget->TakeWidget()
			];

		SafeWidget->SetToolTip(ToolTip);
	}
	else if ( !ToolTipText.IsEmpty() || ToolTipTextDelegate.IsBound() )
	{
		SafeWidget->SetToolTipText(PROPERTY_BINDING(FText, ToolTipText));
	}

#if WITH_ACCESSIBILITY
	if (AccessibleWidgetData)
	{
		TSharedPtr<SWidget> AccessibleWidget = GetAccessibleWidget();
		if (AccessibleWidget.IsValid())
		{
			AccessibleWidget->SetAccessibleBehavior((EAccessibleBehavior)AccessibleWidgetData->AccessibleBehavior, AccessibleWidgetData->CreateAccessibleTextAttribute(), EAccessibleType::Main);
			AccessibleWidget->SetAccessibleBehavior((EAccessibleBehavior)AccessibleWidgetData->AccessibleSummaryBehavior, AccessibleWidgetData->CreateAccessibleSummaryTextAttribute(), EAccessibleType::Summary);
			AccessibleWidget->SetCanChildrenBeAccessible(AccessibleWidgetData->bCanChildrenBeAccessible);
		}
	}
#endif
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


void UWidget::SynchronizeAccessibleData()
{
#if WITH_EDITORONLY_DATA
	if (bOverrideAccessibleDefaults)
	{
		if (!AccessibleWidgetData)
		{
			AccessibleWidgetData = NewObject<USlateAccessibleWidgetData>(this);
		}
		AccessibleWidgetData->bCanChildrenBeAccessible = bCanChildrenBeAccessible;
		AccessibleWidgetData->AccessibleBehavior = AccessibleBehavior;
		AccessibleWidgetData->AccessibleText = AccessibleText;
		AccessibleWidgetData->AccessibleTextDelegate = AccessibleTextDelegate;
		AccessibleWidgetData->AccessibleSummaryBehavior = AccessibleSummaryBehavior;
		AccessibleWidgetData->AccessibleSummaryText = AccessibleSummaryText;
		AccessibleWidgetData->AccessibleSummaryTextDelegate = AccessibleSummaryTextDelegate;
	}
	else if (AccessibleWidgetData)
	{
		AccessibleWidgetData = nullptr;
	}
#endif
}

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> UWidget::GetAccessibleWidget() const
{
	return GetCachedWidget();
}
#endif

FText UWidget::GetAccessibleText() const
{
#if WITH_ACCESSIBILITY
	TSharedPtr<SWidget> AccessibleWidget = GetAccessibleWidget();
	if (AccessibleWidget.IsValid())
	{
		return AccessibleWidget->GetAccessibleText();
	}
#endif
	return FText::GetEmpty();
}

FText UWidget::GetAccessibleSummaryText() const
{
#if WITH_ACCESSIBILITY
	TSharedPtr<SWidget> AccessibleWidget = GetAccessibleWidget();
	if (AccessibleWidget.IsValid())
	{
		return AccessibleWidget->GetAccessibleSummary();
	}
#endif
	return FText::GetEmpty();
}

UObject* UWidget::GetSourceAssetOrClass() const
{
	UObject* SourceAsset = nullptr;

#if WITH_EDITOR
	// In editor builds we add metadata to the widget so that once hit with the widget reflector it can report
	// where it comes from, what blueprint, what the name of the widget was...etc.
	SourceAsset = WidgetGeneratedBy.Get();
#else
	#if UE_HAS_WIDGET_GENERATED_BY_CLASS
		SourceAsset = WidgetGeneratedByClass.Get();
	#endif
#endif

	if (!SourceAsset)
	{
		if (UWidget* WidgetOuter = GetTypedOuter<UWidget>())
		{
			return WidgetOuter->GetSourceAssetOrClass();
		}
	}

	return SourceAsset;
}

void UWidget::BuildNavigation()
{
	if ( Navigation != nullptr )
	{
		TSharedPtr<SWidget> SafeWidget = GetCachedWidget();

		if ( SafeWidget.IsValid() )
		{
			TSharedPtr<FNavigationMetaData> MetaData = SafeWidget->GetMetaData<FNavigationMetaData>();
			if ( !MetaData.IsValid() )
			{
				MetaData = MakeShared<FNavigationMetaData>();
				SafeWidget->AddMetadata(MetaData.ToSharedRef());
			}

			Navigation->UpdateMetaData(MetaData.ToSharedRef());
		}
	}
}

UWorld* UWidget::GetWorld() const
{
	// UWidget's are given world scope by their owning user widget.  We can get that through the widget tree that should
	// be the outer of this widget.
	if ( UWidgetTree* OwningTree = Cast<UWidgetTree>(GetOuter()) )
	{
		return OwningTree->GetWorld();
	}

	return Super::GetWorld();
}

void UWidget::BeginDestroy()
{
	if (bIsManagedByGameViewportSubsystem)
	{
		if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
		{
			Subsystem->RemoveWidget(this);
		}
	}
	Super::BeginDestroy();
}

void UWidget::FinishDestroy()
{
	Super::FinishDestroy();
	DEC_DWORD_STAT(STAT_SlateUTotalWidgets);
}

EVisibility UWidget::ConvertSerializedVisibilityToRuntime(ESlateVisibility Input)
{
	switch ( Input )
	{
	case ESlateVisibility::Visible:
		return EVisibility::Visible;
	case ESlateVisibility::Collapsed:
		return EVisibility::Collapsed;
	case ESlateVisibility::Hidden:
		return EVisibility::Hidden;
	case ESlateVisibility::HitTestInvisible:
		return EVisibility::HitTestInvisible;
	case ESlateVisibility::SelfHitTestInvisible:
		return EVisibility::SelfHitTestInvisible;
	default:
		check(false);
		return EVisibility::Visible;
	}
}

ESlateVisibility UWidget::ConvertRuntimeToSerializedVisibility(const EVisibility& Input)
{
	if ( Input == EVisibility::Visible )
	{
		return ESlateVisibility::Visible;
	}
	else if ( Input == EVisibility::Collapsed )
	{
		return ESlateVisibility::Collapsed;
	}
	else if ( Input == EVisibility::Hidden )
	{
		return ESlateVisibility::Hidden;
	}
	else if ( Input == EVisibility::HitTestInvisible )
	{
		return ESlateVisibility::HitTestInvisible;
	}
	else if ( Input == EVisibility::SelfHitTestInvisible )
	{
		return ESlateVisibility::SelfHitTestInvisible;
	}
	else
	{
		check(false);
		return ESlateVisibility::Visible;
	}
}

FSizeParam UWidget::ConvertSerializedSizeParamToRuntime(const FSlateChildSize& Input)
{
	switch ( Input.SizeRule )
	{
	default:
	case ESlateSizeRule::Automatic:
		return FAuto();
	case ESlateSizeRule::Fill:
		return FStretch(Input.Value);
	}
}

UWidget* UWidget::FindChildContainingDescendant(UWidget* Root, UWidget* Descendant)
{
	if ( Root == nullptr )
	{
		return nullptr;
	}

	UWidget* Parent = Descendant->GetParent();

	while ( Parent != nullptr )
	{
		// If the Descendant's parent is the root, then the child containing the descendant is the descendant.
		if ( Parent == Root )
		{
			return Descendant;
		}

		Descendant = Parent;
		Parent = Parent->GetParent();
	}

	return nullptr;
}

// TODO: Clean this up to, move it to a user interface setting, don't use a config. 
FString UWidget::GetDefaultFontName()
{
	FString DefaultFontName = "/Engine/EngineFonts/Roboto";
	GConfig->GetString(TEXT("SlateStyle"), TEXT("DefaultFontName"), DefaultFontName, GEngineIni);

	return DefaultFontName;
}

//bool UWidget::BindProperty(const FName& DestinationProperty, UObject* SourceObject, const FName& SourceProperty)
//{
//	FDelegateProperty* DelegateProperty = FindFProperty<FDelegateProperty>(GetClass(), FName(*( DestinationProperty.ToString() + TEXT("Delegate") )));
//
//	if ( DelegateProperty )
//	{
//		FDynamicPropertyPath BindingPath(SourceProperty.ToString());
//		return AddBinding(DelegateProperty, SourceObject, BindingPath);
//	}
//
//	return false;
//}

TSubclassOf<UPropertyBinding> UWidget::FindBinderClassForDestination(FProperty* Property)
{
	if (BinderClasses.IsEmpty())
	{
		TArray<UClass*> PropertyBindingClasses;
		GetDerivedClasses(UPropertyBinding::StaticClass(), PropertyBindingClasses);
		BinderClasses.Reserve(PropertyBindingClasses.Num());
		for (UClass* PropertyBindingClass : PropertyBindingClasses)
		{
			BinderClasses.Emplace(PropertyBindingClass);
		}
	}

	for (TSubclassOf<UPropertyBinding>& BinderClass : BinderClasses)
	{
		if (BinderClass.GetDefaultObject()->IsSupportedDestination(Property))
		{
			return BinderClass;
		}
	}

	return nullptr;
}

static UPropertyBinding* GenerateBinder(FDelegateProperty* DelegateProperty, UObject* Container, UObject* SourceObject, const FDynamicPropertyPath& BindingPath)
{
	FScriptDelegate* ScriptDelegate = DelegateProperty->GetPropertyValuePtr_InContainer(Container);
	if ( ScriptDelegate )
	{
		// Only delegates that take no parameters have native binders.
		UFunction* SignatureFunction = DelegateProperty->SignatureFunction;
		if ( SignatureFunction->NumParms == 1 )
		{
			if ( FProperty* ReturnProperty = SignatureFunction->GetReturnProperty() )
			{
				TSubclassOf<UPropertyBinding> BinderClass = UWidget::FindBinderClassForDestination(ReturnProperty);
				if ( BinderClass != nullptr )
				{
					UPropertyBinding* Binder = NewObject<UPropertyBinding>(Container, BinderClass);
					Binder->SourceObject = SourceObject;
					Binder->SourcePath = BindingPath;
					Binder->Bind(ReturnProperty, ScriptDelegate);

					return Binder;
				}
			}
		}
	}

	return nullptr;
}

bool UWidget::AddBinding(FDelegateProperty* DelegateProperty, UObject* SourceObject, const FDynamicPropertyPath& BindingPath)
{
	if ( UPropertyBinding* Binder = GenerateBinder(DelegateProperty, this, SourceObject, BindingPath) )
	{
		// Remove any existing binding object for this property.
		for ( int32 BindingIndex = 0; BindingIndex < NativeBindings.Num(); BindingIndex++ )
		{
			if ( NativeBindings[BindingIndex]->DestinationProperty == DelegateProperty->GetFName() )
			{
				NativeBindings.RemoveAt(BindingIndex);
				break;
			}
		}

		NativeBindings.Add(Binder);

		// Only notify the bindings have changed if we've already create the underlying slate widget.
		if ( MyWidget.IsValid() )
		{
			OnBindingChanged(DelegateProperty->GetFName());
		}

		return true;
	}

	return false;
}

FDelegateHandle UWidget::RegisterPostStateListener(const FOnWidgetStateBroadcast::FDelegate& ListenerDelegate, bool bBroadcastCurrentState)
{
	if (!bWidgetStateInitialized)
	{
		MyWidgetStateBitfield = UWidgetStateSettings::Get()->GetInitialRegistrationBitfield(this);
		bWidgetStateInitialized = true;
	}

	if (bBroadcastCurrentState)
	{
		ListenerDelegate.ExecuteIfBound(this, MyWidgetStateBitfield);
	}

	return PostWidgetStateChanged.Add(ListenerDelegate);
}

void UWidget::UnregisterPostStateListener(const FDelegateHandle& ListenerDelegate)
{
	PostWidgetStateChanged.Remove(ListenerDelegate);
}

void UWidget::OnBindingChanged(const FName& Property)
{

}

void UWidget::BroadcastBinaryPostStateChange(const FWidgetStateBitfield& StateChange, bool bInValue)
{
	if (bShouldBroadcastState && bWidgetStateInitialized)
	{
		MyWidgetStateBitfield.SetBinaryState(StateChange, bInValue);
		PostWidgetStateChanged.Broadcast(this, MyWidgetStateBitfield);
	}
}

void UWidget::BroadcastEnumPostStateChange(const FWidgetStateBitfield& StateChange)
{
}

namespace UE::UMG::Private
{
	UWidgetFieldNotificationExtension* FindOrAddWidgetNotifyExtension(UWidget* Widget)
	{
		if (UUserWidget* UserWidget = Cast<UUserWidget>(Widget))
		{
			if (UWidgetFieldNotificationExtension* Extension = UserWidget->GetExtension<UWidgetFieldNotificationExtension>())
			{
				return Extension;
			}
			return UserWidget->AddExtension<UWidgetFieldNotificationExtension>();
		}
		else if (UWidgetTree* WidgetTree = Cast<UWidgetTree>(Widget->GetOuter()))
		{
			if (UUserWidget* InnerUserWidget = Cast<UUserWidget>(WidgetTree->GetOuter()))
			{
				if (UWidgetFieldNotificationExtension* Extension = InnerUserWidget->GetExtension<UWidgetFieldNotificationExtension>())
				{
					return Extension;
				}
				return InnerUserWidget->AddExtension<UWidgetFieldNotificationExtension>();
			}
		}
		return nullptr;
	}
	UWidgetFieldNotificationExtension* FindWidgetNotifyExtension(const UWidget* Widget)
	{
		if (const UUserWidget* UserWidget = Cast<const UUserWidget>(Widget))
		{
			if (UWidgetFieldNotificationExtension* Extension = UserWidget->GetExtension<UWidgetFieldNotificationExtension>())
			{
				return Extension;
			}
		}
		else if (const UWidgetTree* WidgetTree = Cast<const UWidgetTree>(Widget->GetOuter()))
		{
			if (const UUserWidget* InnerUserWidget = Cast<const UUserWidget>(WidgetTree->GetOuter()))
			{
				if (UWidgetFieldNotificationExtension* Extension = InnerUserWidget->GetExtension<UWidgetFieldNotificationExtension>())
				{
					return Extension;
				}
			}
		}
		return nullptr;
	}
}

FDelegateHandle UWidget::AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate)
{
	FDelegateHandle Result;
	if (InFieldId.IsValid())
	{
		if (UWidgetFieldNotificationExtension* Extension = UE::UMG::Private::FindOrAddWidgetNotifyExtension(this))
		{
			Result = Extension->AddFieldValueChangedDelegate(this, InFieldId, MoveTemp(InNewDelegate));
			if (Result.IsValid())
			{
				EnabledFieldNotifications.PadToNum(InFieldId.GetIndex() + 1, false);
				EnabledFieldNotifications[InFieldId.GetIndex()] = true;
			}
		}
	}
	return Result;
}

void UWidget::K2_AddFieldValueChangedDelegate(FFieldNotificationId InFieldId, FFieldValueChangedDynamicDelegate InDelegate)
{
	if (InFieldId.IsValid())
	{
		const UE::FieldNotification::FFieldId FieldId = GetFieldNotificationDescriptor().GetField(GetClass(), InFieldId.FieldName);
		if (ensureMsgf(FieldId.IsValid(), TEXT("The field should be compiled correctly.")))
		{
			if (UWidgetFieldNotificationExtension* Extension = UE::UMG::Private::FindOrAddWidgetNotifyExtension(this))
			{
				if (Extension->AddFieldValueChangedDelegate(this, FieldId, InDelegate).IsValid())
				{
					EnabledFieldNotifications.PadToNum(FieldId.GetIndex() + 1, false);
					EnabledFieldNotifications[FieldId.GetIndex()] = true;
				}
			}
		}
	}
}

bool UWidget::RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle)
{
	bool bResult = false;
	if (InFieldId.IsValid() && InHandle.IsValid() && EnabledFieldNotifications.IsValidIndex(InFieldId.GetIndex()) && EnabledFieldNotifications[InFieldId.GetIndex()])
	{
		UWidgetFieldNotificationExtension* Extension = UE::UMG::Private::FindWidgetNotifyExtension(this);
		checkf(Extension, TEXT("If the EnabledFieldNotifications is valid, then the Extension must also be valid."));
		UWidgetFieldNotificationExtension::FRemoveFromResult RemoveResult = Extension->RemoveFieldValueChangedDelegate(this, InFieldId, InHandle);
		bResult = RemoveResult.bRemoved;
		EnabledFieldNotifications[InFieldId.GetIndex()] = RemoveResult.bHasOtherBoundDelegates;
	}
	return bResult;
}

void UWidget::K2_RemoveFieldValueChangedDelegate(FFieldNotificationId InFieldId, FFieldValueChangedDynamicDelegate InDelegate)
{
	if (InFieldId.IsValid())
	{
		const UE::FieldNotification::FFieldId FieldId = GetFieldNotificationDescriptor().GetField(GetClass(), InFieldId.FieldName);
		if (ensureMsgf(FieldId.IsValid(), TEXT("The field should be compiled correctly.")))
		{
			if (EnabledFieldNotifications.IsValidIndex(FieldId.GetIndex()) && EnabledFieldNotifications[FieldId.GetIndex()])
			{
				UWidgetFieldNotificationExtension* Extension = UE::UMG::Private::FindWidgetNotifyExtension(this);
				checkf(Extension, TEXT("If the EnabledFieldNotifications is valid, then the Extension must also be valid."));
				UWidgetFieldNotificationExtension::FRemoveFromResult RemoveResult = Extension->RemoveFieldValueChangedDelegate(this, FieldId, InDelegate);
				EnabledFieldNotifications[FieldId.GetIndex()] = RemoveResult.bHasOtherBoundDelegates;
			}
		}
	}
}

int32 UWidget::RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject)
{
	int32 bResult = 0;
	if (InUserObject)
	{
		if (UWidgetFieldNotificationExtension* Extension = UE::UMG::Private::FindWidgetNotifyExtension(this))
		{
			UWidgetFieldNotificationExtension::FRemoveAllResult RemoveResult = Extension->RemoveAllFieldValueChangedDelegates(this, InUserObject);
			bResult = RemoveResult.RemoveCount;
			EnabledFieldNotifications = RemoveResult.HasFields;
		}
	}
	return bResult;
}

int32 UWidget::RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject)
{
	int32 bResult = 0;
	if (InUserObject)
	{
		if (UWidgetFieldNotificationExtension* Extension = UE::UMG::Private::FindWidgetNotifyExtension(this))
		{
			UWidgetFieldNotificationExtension::FRemoveAllResult RemoveResult = Extension->RemoveAllFieldValueChangedDelegates(this, InFieldId, InUserObject);
			bResult = RemoveResult.RemoveCount;
			EnabledFieldNotifications = RemoveResult.HasFields;
		}
	}
	return bResult;
}

void UWidget::BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId)
{
	if (InFieldId.IsValid() && EnabledFieldNotifications.IsValidIndex(InFieldId.GetIndex()) && EnabledFieldNotifications[InFieldId.GetIndex()])
	{
		UWidgetFieldNotificationExtension* Extension = UE::UMG::Private::FindWidgetNotifyExtension(this);
		checkf(Extension, TEXT("If the EnabledFieldNotifications is valid, then the Extension must also be valid."));
		Extension->BroadcastFieldValueChanged(this, InFieldId);
	}
}

void UWidget::K2_BroadcastFieldValueChanged(FFieldNotificationId InFieldId)
{
	if (InFieldId.IsValid())
	{
		const UE::FieldNotification::FFieldId FieldId = GetFieldNotificationDescriptor().GetField(GetClass(), InFieldId.FieldName);
		if (ensureMsgf(FieldId.IsValid(), TEXT("The field should be compiled correctly.")))
		{
			BroadcastFieldValueChanged(FieldId);
		}
	}
}

#undef LOCTEXT_NAMESPACE

