// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WidgetComponent.h"
#include "Engine/GameInstance.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "LocalVertexFactory.h"
#include "Materials/Material.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineGlobals.h"
#include "MaterialShared.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Widgets/SWindow.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Input/HittestGrid.h"
#include "SceneManagement.h"
#include "DynamicMeshBuilder.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "Rendering/SlateDrawBuffer.h"
#include "Slate/SGameLayerManager.h"
#include "Slate/WidgetRenderer.h"
#include "Slate/SWorldWidgetScreenLayer.h"
#include "UObject/EditorObjectVersion.h"
#include "Widgets/SViewport.h"
#include "Widgets/SVirtualWindow.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetComponent)

DECLARE_CYCLE_STAT(TEXT("3DHitTesting"), STAT_Slate3DHitTesting, STATGROUP_Slate);

static int32 MaximumRenderTargetWidth = 3840;
static FAutoConsoleVariableRef CVarMaximumRenderTargetWidth
(
	TEXT("WidgetComponent.MaximumRenderTargetWidth"),
	MaximumRenderTargetWidth,
	TEXT("Sets the maximum width of the render target used by a Widget Component.")
);

static int32 MaximumRenderTargetHeight = 2160;
static FAutoConsoleVariableRef CVarMaximumRenderTargetHeight
(
	TEXT("WidgetComponent.MaximumRenderTargetHeight"),
	MaximumRenderTargetHeight,
	TEXT("Sets the maximum height of the render target used by a Widget Component.")
);

static bool bUseAutomaticTickModeByDefault = false;
static FAutoConsoleVariableRef CVarbUseAutomaticTickModeByDefault
(
	TEXT("WidgetComponent.UseAutomaticTickModeByDefault"),
	bUseAutomaticTickModeByDefault,
	TEXT("Sets to true to Disable Tick by default on Widget Components when set to false, the tick will enabled by default.")
);

class FWorldWidgetScreenLayer : public IGameLayer
{
public:
	FWorldWidgetScreenLayer(const FLocalPlayerContext& PlayerContext)
	{
		OwningPlayer = PlayerContext;
	}

	virtual ~FWorldWidgetScreenLayer()
	{
		// empty virtual destructor to help clang warning
	}

	void AddComponent(UWidgetComponent* Component)
	{
		if (Component)
		{
			Components.AddUnique(Component);

			if (TSharedPtr<SWorldWidgetScreenLayer> ScreenLayer = ScreenLayerPtr.Pin())
			{
				if (UUserWidget* UserWidget = Component->GetUserWidgetObject())
				{
					ScreenLayer->AddComponent(Component, UserWidget->TakeWidget());
				}
				else if (Component->GetSlateWidget().IsValid())
				{
					ScreenLayer->AddComponent(Component, Component->GetSlateWidget().ToSharedRef());
				}
			}
		}
	}

	void RemoveComponent(UWidgetComponent* Component)
	{
		if (Component)
		{
			Components.RemoveSwap(Component);

			if (TSharedPtr<SWorldWidgetScreenLayer> ScreenLayer = ScreenLayerPtr.Pin())
			{
				ScreenLayer->RemoveComponent(Component);
			}
		}
	}

	virtual TSharedRef<SWidget> AsWidget() override
	{
		if (TSharedPtr<SWorldWidgetScreenLayer> ScreenLayer = ScreenLayerPtr.Pin())
		{
			return ScreenLayer.ToSharedRef();
		}

		TSharedRef<SWorldWidgetScreenLayer> NewScreenLayer = SNew(SWorldWidgetScreenLayer, OwningPlayer);
		ScreenLayerPtr = NewScreenLayer;

		// Add all the pending user widgets to the surface
		for ( TWeakObjectPtr<UWidgetComponent>& WeakComponent : Components )
		{
			if ( UWidgetComponent* Component = WeakComponent.Get() )
			{
				if ( UUserWidget* UserWidget = Component->GetUserWidgetObject() )
				{
					NewScreenLayer->AddComponent(Component, UserWidget->TakeWidget());
				}
				else if (Component->GetSlateWidget().IsValid())
				{
					NewScreenLayer->AddComponent(Component, Component->GetSlateWidget().ToSharedRef()); 
				}
			}
		}

		return NewScreenLayer;
	}

private:
	FLocalPlayerContext OwningPlayer;
	TWeakPtr<SWorldWidgetScreenLayer> ScreenLayerPtr;
	TArray<TWeakObjectPtr<UWidgetComponent>> Components;
};




/**
* The hit tester used by all Widget Component objects.
*/
class FWidget3DHitTester : public ICustomHitTestPath
{
public:
	FWidget3DHitTester( UWorld* InWorld )
		: World( InWorld )
		, CachedFrame(-1)
	{}

	// ICustomHitTestPath implementation
	virtual TArray<FWidgetAndPointer> GetBubblePathAndVirtualCursors(const FGeometry& InGeometry, FVector2D DesktopSpaceCoordinate, bool bIgnoreEnabledStatus) const override
	{
		SCOPE_CYCLE_COUNTER(STAT_Slate3DHitTesting);

		if( World.IsValid() /*&& ensure( World->IsGameWorld() )*/ )
		{
			UWorld* SafeWorld = World.Get();
			if ( SafeWorld )
			{
				ULocalPlayer* const TargetPlayer = GEngine->GetLocalPlayerFromControllerId(SafeWorld, 0);

				if( TargetPlayer && TargetPlayer->PlayerController )
				{
					FVector2D LocalMouseCoordinate = InGeometry.AbsoluteToLocal(DesktopSpaceCoordinate) * InGeometry.Scale;

					if ( UPrimitiveComponent* HitComponent = GetHitResultAtScreenPositionAndCache(TargetPlayer->PlayerController, LocalMouseCoordinate) )
					{
						if ( UWidgetComponent* WidgetComponent = Cast<UWidgetComponent>(HitComponent) )
						{
							if ( WidgetComponent->GetReceiveHardwareInput() )
							{
								if ( WidgetComponent->GetCurrentDrawSize().X != 0 && WidgetComponent->GetCurrentDrawSize().Y != 0 )
								{
									// Get the "forward" vector based on the current rotation system.
									const FVector ForwardVector = WidgetComponent->GetForwardVector();

									// Make sure the player is interacting with the front of the widget
									if ( FVector::DotProduct(ForwardVector, CachedHitResult.ImpactPoint - CachedHitResult.TraceStart) < 0.f )
									{
										return WidgetComponent->GetHitWidgetPath(CachedHitResult.Location, bIgnoreEnabledStatus);
									}
								}
							}
						}
					}
				}
			}
		}

		return TArray<FWidgetAndPointer>();
	}

	virtual void ArrangeCustomHitTestChildren( FArrangedChildren& ArrangedChildren ) const override
	{
		for( TWeakObjectPtr<UWidgetComponent> Component : RegisteredComponents )
		{
			UWidgetComponent* WidgetComponent = Component.Get();
			// Check if visible;
			if ( WidgetComponent && WidgetComponent->GetSlateWindow().IsValid() )
			{
				FGeometry WidgetGeom;

				ArrangedChildren.AddWidget( FArrangedWidget( WidgetComponent->GetSlateWindow().ToSharedRef(), WidgetGeom.MakeChild( WidgetComponent->GetCurrentDrawSize(), FSlateLayoutTransform() ) ) );
			}
		}
	}

	virtual TOptional<FVirtualPointerPosition> TranslateMouseCoordinateForCustomHitTestChild(const SWidget& ChildWidget, const FGeometry& ViewportGeometry, const FVector2D ScreenSpaceMouseCoordinate, const FVector2D LastScreenSpaceMouseCoordinate) const override
	{
		if ( World.IsValid() && ensure(World->IsGameWorld()) )
		{
			ULocalPlayer* const TargetPlayer = GEngine->GetLocalPlayerFromControllerId(World.Get(), 0);
			if ( TargetPlayer && TargetPlayer->PlayerController )
			{
				FVector2D LocalMouseCoordinate = ViewportGeometry.AbsoluteToLocal(ScreenSpaceMouseCoordinate) * ViewportGeometry.Scale;

				// Check for a hit against any widget components in the world
				for ( TWeakObjectPtr<UWidgetComponent> Component : RegisteredComponents )
				{
					UWidgetComponent* WidgetComponent = Component.Get();
					// Check if visible;
					if ( WidgetComponent && WidgetComponent->GetSlateWindow().Get() == &ChildWidget )
					{
						if ( UPrimitiveComponent* HitComponent = GetHitResultAtScreenPositionAndCache(TargetPlayer->PlayerController, LocalMouseCoordinate) )
						{
							if ( WidgetComponent->GetReceiveHardwareInput() )
							{
								if ( WidgetComponent->GetCurrentDrawSize().X != 0 && WidgetComponent->GetCurrentDrawSize().Y != 0 )
								{
									if ( WidgetComponent == HitComponent )
									{
										FVector2D LocalHitLocation;
										WidgetComponent->GetLocalHitLocation(CachedHitResult.Location, LocalHitLocation);
										return FVirtualPointerPosition(LocalHitLocation, LocalHitLocation);
									}
								}
							}
						}
					}
				}
			}
		}

		return TOptional<FVirtualPointerPosition>();
	}
	// End ICustomHitTestPath

	UPrimitiveComponent* GetHitResultAtScreenPositionAndCache(APlayerController* PlayerController, FVector2D ScreenPosition) const
	{
		UPrimitiveComponent* HitComponent = nullptr;

		if ( GFrameNumber != CachedFrame || CachedScreenPosition != ScreenPosition )
		{
			CachedFrame = GFrameNumber;
			CachedScreenPosition = ScreenPosition;

			if ( PlayerController )
			{
				if ( PlayerController->GetHitResultAtScreenPosition(ScreenPosition, ECC_Visibility, true, CachedHitResult) )
				{
					return CachedHitResult.Component.Get();
				}
			}
		}
		else
		{
			return CachedHitResult.Component.Get();
		}

		return nullptr;
	}

	void RegisterWidgetComponent( UWidgetComponent* InComponent )
	{
		RegisteredComponents.AddUnique( InComponent );
	}

	void UnregisterWidgetComponent( UWidgetComponent* InComponent )
	{
		RegisteredComponents.RemoveSingleSwap( InComponent );
	}

	uint32 GetNumRegisteredComponents() const { return RegisteredComponents.Num(); }
	
	UWorld* GetWorld() const { return World.Get(); }

private:
	TArray< TWeakObjectPtr<UWidgetComponent> > RegisteredComponents;
	TWeakObjectPtr<UWorld> World;

	mutable int64 CachedFrame;
	mutable FVector2D CachedScreenPosition;
	mutable FHitResult CachedHitResult;
};



/** Represents a billboard sprite to the scene manager. */
class FWidget3DSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	/** Initialization constructor. */
	FWidget3DSceneProxy( UWidgetComponent* InComponent, ISlate3DRenderer& InRenderer )
		: FPrimitiveSceneProxy( InComponent )
		, ArcAngle(FMath::DegreesToRadians(InComponent->GetCylinderArcAngle()))
		, Pivot( InComponent->GetPivot() )
		, Renderer( InRenderer )
		, RenderTarget( InComponent->GetRenderTarget() )
		, MaterialInstance( InComponent->GetMaterialInstance() )
		, BlendMode( InComponent->GetBlendMode() )
		, GeometryMode(InComponent->GetGeometryMode())
		, BodySetup(InComponent->GetBodySetup())
	{
		bWillEverBeLit = false;

		if (MaterialInstance)
		{
			MaterialRelevance = MaterialInstance->GetRelevance_Concurrent(GetScene().GetShaderPlatform());
		}
	}

	// FPrimitiveSceneProxy interface.
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
#if WITH_EDITOR
		const bool bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;

		auto WireframeMaterialInstance = new FColoredMaterialRenderProxy(
			GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : nullptr,
			FLinearColor(0, 0.5f, 1.f)
			);

		Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);

		FMaterialRenderProxy* ParentMaterialProxy = nullptr;
		if ( bWireframe )
		{
			ParentMaterialProxy = WireframeMaterialInstance;
		}
		else if (MaterialInstance != nullptr)
		{
			ParentMaterialProxy = MaterialInstance->GetRenderProxy();
		}
#else
		FMaterialRenderProxy* ParentMaterialProxy = MaterialInstance ? MaterialInstance->GetRenderProxy() : nullptr;
#endif

		//FSpriteTextureOverrideRenderProxy* TextureOverrideMaterialProxy = new FSpriteTextureOverrideRenderProxy(ParentMaterialProxy,

		const FMatrix& ViewportLocalToWorld = GetLocalToWorld();

		FMatrix PreviousLocalToWorld;

		if (!GetScene().GetPreviousLocalToWorld(GetPrimitiveSceneInfo(), PreviousLocalToWorld))
		{
			PreviousLocalToWorld = GetLocalToWorld();
		}

		if( RenderTarget )
		{
			FTextureResource* TextureResource = RenderTarget->GetResource();
			if ( TextureResource )
			{
				if (GeometryMode == EWidgetGeometryMode::Plane)
				{
					float U = -RenderTarget->SizeX * static_cast<float>(Pivot.X);
					float V = -RenderTarget->SizeY * static_cast<float>(Pivot.Y);
					float UL = RenderTarget->SizeX * (1.0f - static_cast<float>(Pivot.X));
					float VL = RenderTarget->SizeY * (1.0f - static_cast<float>(Pivot.Y));

					int32 VertexIndices[4];

					for ( int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++ )
					{
						FDynamicMeshBuilder MeshBuilder(Views[ViewIndex]->GetFeatureLevel());

						if ( VisibilityMap & ( 1 << ViewIndex ) )
						{
							VertexIndices[0] = MeshBuilder.AddVertex(-FVector3f(0, U, V ),  FVector2f(0, 0), FVector3f(0, -1, 0), FVector3f(0, 0, -1), FVector3f(1, 0, 0), FColor::White);
							VertexIndices[1] = MeshBuilder.AddVertex(-FVector3f(0, U, VL),  FVector2f(0, 1), FVector3f(0, -1, 0), FVector3f(0, 0, -1), FVector3f(1, 0, 0), FColor::White);
							VertexIndices[2] = MeshBuilder.AddVertex(-FVector3f(0, UL, VL), FVector2f(1, 1), FVector3f(0, -1, 0), FVector3f(0, 0, -1), FVector3f(1, 0, 0), FColor::White);
							VertexIndices[3] = MeshBuilder.AddVertex(-FVector3f(0, UL, V),  FVector2f(1, 0), FVector3f(0, -1, 0), FVector3f(0, 0, -1), FVector3f(1, 0, 0), FColor::White);

							MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[1], VertexIndices[2]);
							MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[2], VertexIndices[3]);

							FDynamicMeshBuilderSettings Settings;
							Settings.bDisableBackfaceCulling = false;
							Settings.bReceivesDecals = true;
							Settings.bUseSelectionOutline = true;
							Settings.bIsFirstPerson = IsFirstPerson();
							MeshBuilder.GetMesh(ViewportLocalToWorld, PreviousLocalToWorld, ParentMaterialProxy, SDPG_World, Settings, nullptr, ViewIndex, Collector, FHitProxyId());
						}
					}
				}
				else
				{
					ensure(GeometryMode == EWidgetGeometryMode::Cylinder);

					const int32 NumSegments = FMath::Lerp(4, 32, ArcAngle/PI);


					const double Radius = RenderTarget->SizeX / ArcAngle;
					const double Apothem = Radius * FMath::Cos(0.5*ArcAngle);
					const double ChordLength = 2.0f * Radius * FMath::Sin(0.5*ArcAngle);
					
					const double PivotOffsetX = ChordLength * (0.5-Pivot.X);
					const double V = -RenderTarget->SizeY * Pivot.Y;
					const double VL = RenderTarget->SizeY * (1.0 - Pivot.Y);

					int32 VertexIndices[4];

					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						FDynamicMeshBuilder MeshBuilder(Views[ViewIndex]->GetFeatureLevel());

						if (VisibilityMap & (1 << ViewIndex))
						{
							const double RadiansPerStep = ArcAngle / NumSegments;

							FVector LastTangentX;
							FVector LastTangentY;
							FVector LastTangentZ;

							for (int32 Segment = 0; Segment < NumSegments; Segment++ )
							{
								const double Angle = -ArcAngle / 2 + Segment * RadiansPerStep;
								const double NextAngle = Angle + RadiansPerStep;
								
								// Polar to Cartesian
								const double X0 = Radius * FMath::Cos(Angle) - Apothem;
								const double Y0 = Radius * FMath::Sin(Angle);
								const double X1 = Radius * FMath::Cos(NextAngle) - Apothem;
								const double Y1 = Radius * FMath::Sin(NextAngle);

								const float U0 = static_cast<float>(Segment) / NumSegments;
								const float U1 = static_cast<float>(Segment+1) / NumSegments;

								const FVector Vertex0 = -FVector(X0, PivotOffsetX + Y0, V);
								const FVector Vertex1 = -FVector(X0, PivotOffsetX + Y0, VL);
								const FVector Vertex2 = -FVector(X1, PivotOffsetX + Y1, VL);
								const FVector Vertex3 = -FVector(X1, PivotOffsetX + Y1, V);

								FVector TangentX = Vertex3 - Vertex0;
								TangentX.Normalize();
								FVector TangentY = Vertex1 - Vertex0;
								TangentY.Normalize();
								FVector TangentZ = FVector::CrossProduct(TangentX, TangentY);

								if (Segment == 0)
								{
									LastTangentX = TangentX;
									LastTangentY = TangentY;
									LastTangentZ = TangentZ;
								}

								VertexIndices[0] = MeshBuilder.AddVertex((FVector3f)Vertex0, FVector2f(U0, 0), (FVector3f)LastTangentX, (FVector3f)LastTangentY, (FVector3f)LastTangentZ, FColor::White);
								VertexIndices[1] = MeshBuilder.AddVertex((FVector3f)Vertex1, FVector2f(U0, 1), (FVector3f)LastTangentX, (FVector3f)LastTangentY, (FVector3f)LastTangentZ, FColor::White);
								VertexIndices[2] = MeshBuilder.AddVertex((FVector3f)Vertex2, FVector2f(U1, 1), (FVector3f)TangentX, (FVector3f)TangentY, (FVector3f)TangentZ, FColor::White);
								VertexIndices[3] = MeshBuilder.AddVertex((FVector3f)Vertex3, FVector2f(U1, 0), (FVector3f)TangentX, (FVector3f)TangentY, (FVector3f)TangentZ, FColor::White);

								MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[1], VertexIndices[2]);
								MeshBuilder.AddTriangle(VertexIndices[0], VertexIndices[2], VertexIndices[3]);

								LastTangentX = TangentX;
								LastTangentY = TangentY;
								LastTangentZ = TangentZ;
							}

							FDynamicMeshBuilderSettings Settings;
							Settings.bDisableBackfaceCulling = false;
							Settings.bReceivesDecals = true;
							Settings.bUseSelectionOutline = true;
							Settings.bIsFirstPerson = IsFirstPerson();
							MeshBuilder.GetMesh(ViewportLocalToWorld, PreviousLocalToWorld, ParentMaterialProxy, SDPG_World, Settings, nullptr, ViewIndex, Collector, FHitProxyId());
						}
					}
				}
			}
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		for ( int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++ )
		{
			if ( VisibilityMap & ( 1 << ViewIndex ) )
			{
				RenderCollision(BodySetup, Collector, ViewIndex, ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
				RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
			}
		}
#endif
	}

	void RenderCollision(UBodySetup* InBodySetup, FMeshElementCollector& Collector, int32 ViewIndex, const FEngineShowFlags& EngineShowFlags, const FBoxSphereBounds& InBounds, bool bRenderInEditor) const
	{
		if ( InBodySetup )
		{
			bool bDrawCollision = EngineShowFlags.Collision && IsCollisionEnabled();

			if ( bDrawCollision && AllowDebugViewmodes() )
			{
				// Draw simple collision as wireframe if 'show collision', collision is enabled, and we are not using the complex as the simple
				const bool bDrawSimpleWireframeCollision = InBodySetup->CollisionTraceFlag != ECollisionTraceFlag::CTF_UseComplexAsSimple;

				if ( FMath::Abs(GetLocalToWorld().Determinant()) < SMALL_NUMBER )
				{
					// Catch this here or otherwise GeomTransform below will assert
					// This spams so commented out
					//UE_LOGF(LogStaticMesh, Log, "Zero scaling not supported (%ls)", *StaticMesh->GetPathName());
				}
				else
				{
					const bool bDrawSolid = !bDrawSimpleWireframeCollision;
					const bool bProxyIsSelected = IsSelected();

					if ( bDrawSolid )
					{
						// Make a material for drawing solid collision stuff
						auto SolidMaterialInstance = new FColoredMaterialRenderProxy(
							GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
							GetWireframeColor()
							);

						Collector.RegisterOneFrameMaterialProxy(SolidMaterialInstance);

						FTransform GeomTransform(GetLocalToWorld());
						InBodySetup->AggGeom.GetAggGeom(GeomTransform, GetWireframeColor().ToFColor(true), SolidMaterialInstance, false, true, AlwaysHasVelocity(), ViewIndex, Collector);
					}
					// wireframe
					else
					{
						FColor CollisionColor = FColor(157, 149, 223, 255);
						FTransform GeomTransform(GetLocalToWorld());
						InBodySetup->AggGeom.GetAggGeom(GeomTransform, GetSelectionColor(CollisionColor, bProxyIsSelected, IsHovered()).ToFColor(true), nullptr, false, false, AlwaysHasVelocity(), ViewIndex, Collector);
					}
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		bool bVisible = true;

		FPrimitiveViewRelevance Result;

		MaterialRelevance.SetPrimitiveViewRelevance(Result);

		Result.bDrawRelevance = IsShown(View) && bVisible && View->Family->EngineShowFlags.WidgetComponents;
		Result.bDynamicRelevance = true;
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		Result.bEditorPrimitiveRelevance = false;
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

		return Result;
	}

	virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override
	{
		bDynamic = false;
		bRelevant = false;
		bLightMapped = false;
		bShadowMapped = false;
	}

	virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override
	{
		Origin = GetLocalToWorld().GetOrigin();
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }

	SIZE_T GetAllocatedSize(void) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

private:
	FVector Origin;
	double ArcAngle;
	FVector2D Pivot;
	ISlate3DRenderer& Renderer;
	UTextureRenderTarget2D* RenderTarget;
	UMaterialInstanceDynamic* MaterialInstance;
	FMaterialRelevance MaterialRelevance;
	EWidgetBlendMode BlendMode;
	EWidgetGeometryMode GeometryMode;
	UBodySetup* BodySetup;
};






UWidgetComponent::UWidgetComponent( const FObjectInitializer& PCIP )
	: Super( PCIP )
	, DrawSize( FIntPoint( 500, 500 ) )
	, bManuallyRedraw(false)
	, bRedrawRequested(true)
	, RedrawTime(0.0f)
	, LastWidgetRenderTime(0.0)
	, CurrentDrawSize(FIntPoint(0, 0))
	, bUseInvalidationInWorldSpace(false)
	, bReceiveHardwareInput(false)
	, bWindowFocusable(true)
	, WindowVisibility(EWindowVisibility::SelfHitTestInvisible)
	, bApplyGammaCorrection(false)
	, BackgroundColor( FLinearColor::Transparent )
	, TintColorAndOpacity( FLinearColor::White )
	, OpacityFromTexture( 1.0f )
	, BlendMode( EWidgetBlendMode::Masked )
	, bOverrideRenderTargetFormat( false )
	, RenderTargetFormatOverride( RTF_RGBA8 )
	, bIsTwoSided( false )
	, TickWhenOffscreen( false )
	, SharedLayerName(TEXT("WidgetComponentScreenLayer"))
	, LayerZOrder(-100)
	, GeometryMode(EWidgetGeometryMode::Plane)
	, CylinderArcAngle(180.0f)
	, TickMode(bUseAutomaticTickModeByDefault ? ETickMode::Automatic : ETickMode::Enabled)
    , bRenderCleared(false)
	, bOnWidgetVisibilityChangedRegistered(false)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true;

	SetRelativeRotation(FRotator::ZeroRotator);

	BodyInstance.SetCollisionProfileName(FName("UI"));

	// Translucent material instances
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> TranslucentMaterial_Finder( TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Translucent") );
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> TranslucentMaterial_OneSided_Finder(TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Translucent_OneSided"));
	TranslucentMaterial = TranslucentMaterial_Finder.Object;
	TranslucentMaterial_OneSided = TranslucentMaterial_OneSided_Finder.Object;

	// Opaque material instances
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> OpaqueMaterial_Finder( TEXT( "/Engine/EngineMaterials/Widget3DPassThrough_Opaque" ) );
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> OpaqueMaterial_OneSided_Finder(TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Opaque_OneSided"));
	OpaqueMaterial = OpaqueMaterial_Finder.Object;
	OpaqueMaterial_OneSided = OpaqueMaterial_OneSided_Finder.Object;

	// Masked material instances
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> MaskedMaterial_Finder(TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Masked"));
	static ConstructorHelpers::FObjectFinder<UMaterialInterface> MaskedMaterial_OneSided_Finder(TEXT("/Engine/EngineMaterials/Widget3DPassThrough_Masked_OneSided"));
	MaskedMaterial = MaskedMaterial_Finder.Object;
	MaskedMaterial_OneSided = MaskedMaterial_OneSided_Finder.Object;

	LastLocalHitLocation = FVector2D::ZeroVector;
	//SetGenerateOverlapEvents(false);
	bUseEditorCompositing = false;

	Space = EWidgetSpace::World;
	TimingPolicy = EWidgetTimingPolicy::RealTime;
	Pivot = FVector2D(0.5f, 0.5f);

	bAddedToScreen = false;
}

void UWidgetComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::ChangedWidgetComponentWindowVisibilityDefault)
	{
		// Reset the default value for visibility
		WindowVisibility = EWindowVisibility::Visible;
	}
}

bool UWidgetComponent::CanBeInCluster() const
{
	return false;
}

void UWidgetComponent::PostLoad()
{
	Super::PostLoad();

	PrecachePSOs();
}

void UWidgetComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (MaterialInstance)
	{
		FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
		ComponentParams.Priority = EPSOPrecachePriority::High;
		ComponentParams.MaterialInterface = MaterialInstance;
		ComponentParams.PSOPrecacheParams = BasePrecachePSOParams;

		if (RHISupportsManualVertexFetch(GMaxRHIShaderPlatform))
		{
			ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType));
		}
		else
		{
			FVertexDeclarationElementList VtxElements;
			VtxElements.Add(FVertexElement(0, 0, VET_Float3, 0, sizeof(FVector3f)));
			VtxElements.Add(FVertexElement(1, 0, VET_PackedNormal, 1, sizeof(FPackedNormal) * 2));
			VtxElements.Add(FVertexElement(1, sizeof(FPackedNormal), VET_PackedNormal, 2, sizeof(FPackedNormal) * 2));
			VtxElements.Add(FVertexElement(2, 0, VET_Color, 3, sizeof(FColor)));
			VtxElements.Add(FVertexElement(3, 0, VET_Float2, 4, sizeof(FVector2f)));
			VtxElements.Add(FVertexElement(3, 0, VET_Float2, 5, sizeof(FVector2f)));
			VtxElements.Add(FVertexElement(3, 0, VET_Float2, 6, sizeof(FVector2f)));
			VtxElements.Add(FVertexElement(3, 0, VET_Float2, 7, sizeof(FVector2f)));
			VtxElements.Add(FVertexElement(3, 0, VET_Float2, 15, sizeof(FVector2f)));
			if (IsGPUSkinPassThroughSupported(GMaxRHIShaderPlatform))
			{
				VtxElements.Add(FVertexElement(4, 0, VET_Float4, 14, 0));
				VtxElements.Add(FVertexElement(5, 0, VET_UInt, 13, 0, true));
			}
			else
			{
				VtxElements.Add(FVertexElement(4, 0, VET_UInt, 13, 0, true));
			}
			ComponentParams.VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLocalVertexFactory::StaticType, VtxElements));
		}
	}
}

void UWidgetComponent::BeginPlay()
{
	SetComponentTickEnabled(TickMode != ETickMode::Disabled);
	InitWidget();
	Super::BeginPlay();

	CurrentDrawSize = DrawSize;
	UpdateBodySetup(true);
	RecreatePhysicsState();
}

void UWidgetComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	ReleaseResources();
}

void UWidgetComponent::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	// If the InLevel is null, it's a signal that the entire world is about to disappear, so
	// go ahead and remove this widget from the viewport, it could be holding onto too many
	// dangerous actor references that won't carry over into the next world.
	if (InLevel == nullptr && InWorld == GetWorld())
	{
		ReleaseResources();
	}
}


void UWidgetComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	Super::SetMaterial(ElementIndex, Material);

	UpdateMaterialInstance();
}

void UWidgetComponent::UpdateMaterialInstance()
{
	// Always clear the material instance in case we're going from 3D to 2D.
	MaterialInstance = nullptr;

	if (Space == EWidgetSpace::Screen)
	{
		return;
	}

	UMaterialInterface* BaseMaterial = GetMaterial(0);
	MaterialInstance = UMaterialInstanceDynamic::Create(BaseMaterial, this);
	if (MaterialInstance)
	{
		MaterialInstance->AddToCluster(this);
	}
	UpdateMaterialInstanceParameters();

	PrecachePSOs();

	OnMaterialInstanceUpdated.Broadcast();

	MarkRenderStateDirty();
}

void UWidgetComponent::OnHiddenInGameChanged()
{
	Super::OnHiddenInGameChanged();
	
	// If the component is changed from hidden to shown in game, we must start a tick to render the widget Component
	if (bHiddenInGame == false)
	{
		if (ShouldReenableComponentTickWhenWidgetBecomesVisible())
		{
			SetComponentTickEnabled(true);
		}
	}
}

FPrimitiveSceneProxy* UWidgetComponent::CreateSceneProxy()
{
	if (Space == EWidgetSpace::Screen)
	{
		return nullptr;
	}

	if (WidgetRenderer && CurrentSlateWidget.IsValid())
	{
		if (ISlate3DRenderer* SlateRenderer = WidgetRenderer->GetSlateRenderer())
		{
			RequestRenderUpdate();
			LastWidgetRenderTime = 0;

			return new FWidget3DSceneProxy(this, *SlateRenderer);
		}
	}

#if WITH_EDITOR
	// make something so we can see this component in the editor
	class FWidgetBoxProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FWidgetBoxProxy(const UWidgetComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
			, BoxExtents(1.f, InComponent->GetCurrentDrawSize().X / 2.0f, InComponent->GetCurrentDrawSize().Y / 2.0f)
		{
			bWillEverBeLit = false;
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_BoxSceneProxy_GetDynamicMeshElements);

			const FMatrix& LocalToWorld = GetLocalToWorld();

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					const FLinearColor DrawColor = GetViewSelectionColor(FColor::White, *View, IsSelected(), IsHovered(), false, IsIndividuallySelected());

					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
					DrawOrientedWireBox(PDI, LocalToWorld.GetOrigin(), LocalToWorld.GetScaledAxis(EAxis::X), LocalToWorld.GetScaledAxis(EAxis::Y), LocalToWorld.GetScaledAxis(EAxis::Z), BoxExtents, DrawColor, SDPG_World);
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			if (!View->bIsGameView)
			{
				// Should we draw this because collision drawing is enabled, and we have collision
				const bool bShowForCollision = View->Family->EngineShowFlags.Collision && IsCollisionEnabled();
				Result.bDrawRelevance = IsShown(View) || bShowForCollision;
				Result.bDynamicRelevance = true;
				Result.bShadowRelevance = IsShadowCast(View);
				Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			}
			return Result;
		}
		virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
		SIZE_T GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }

	private:
		const FVector	BoxExtents;
	};

	return new FWidgetBoxProxy(this);
#else
	return nullptr;
#endif
}

FBoxSphereBounds UWidgetComponent::CalcBounds(const FTransform & LocalToWorld) const
{
	if ( Space != EWidgetSpace::Screen )
	{
		const FVector Origin = FVector(.5f,
			-( CurrentDrawSize.X * 0.5f ) + (CurrentDrawSize.X * Pivot.X ),
			-( CurrentDrawSize.Y * 0.5f ) + ( CurrentDrawSize.Y * Pivot.Y ));

		const FVector BoxExtent = FVector(1.f, CurrentDrawSize.X / 2.0f, CurrentDrawSize.Y / 2.0f);

		FBoxSphereBounds NewBounds(Origin, BoxExtent, CurrentDrawSize.Size() / 2.0f);
		NewBounds = NewBounds.TransformBy(LocalToWorld);

		NewBounds.BoxExtent *= BoundsScale;
		NewBounds.SphereRadius *= BoundsScale;

		return NewBounds;
	}
	else
	{
		return FBoxSphereBounds(ForceInit).TransformBy(LocalToWorld);
	}
}

UBodySetup* UWidgetComponent::GetBodySetup() 
{
	UpdateBodySetup();
	return BodySetup;
}

FCollisionShape UWidgetComponent::GetCollisionShape(float Inflation) const
{
	if ( Space != EWidgetSpace::Screen )
	{
		const FVector BoxHalfExtent = ( FVector(0.01f, CurrentDrawSize.X * 0.5f, CurrentDrawSize.Y * 0.5f) * GetComponentTransform().GetScale3D() );
		return FCollisionShape::MakeBox(BoxHalfExtent).Inflate(Inflation);
	}
	else
	{
		return FCollisionShape::MakeBox(FVector::ZeroVector);
	}
}

void UWidgetComponent::OnRegister()
{
	// Set this prior to registering the scene component so that bounds are calculated correctly.
	CurrentDrawSize = DrawSize;

	Super::OnRegister();

#if !UE_SERVER
	FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &ThisClass::OnLevelRemovedFromWorld);

	if ( !IsRunningDedicatedServer() )
	{
		const bool bIsGameWorld = GetWorld()->IsGameWorld();
		if ( Space != EWidgetSpace::Screen )
		{
			if ( CanReceiveHardwareInput() && bIsGameWorld )
			{
				TSharedPtr<SViewport> GameViewportWidget = GEngine->GetGameViewportWidget();
				RegisterHitTesterWithViewport(GameViewportWidget);
			}

			if ( !WidgetRenderer && !GUsingNullRHI )
			{
				WidgetRenderer = new FWidgetRenderer(bApplyGammaCorrection);
			}
		}

		BodySetup = nullptr;

#if WITH_EDITOR
		if (!bIsGameWorld)
		{
			InitWidget();
		}
#endif
	}
#endif // !UE_SERVER
}

void UWidgetComponent::SetWindowFocusable(bool bInWindowFocusable)
{
	bWindowFocusable = bInWindowFocusable;
 	if (SlateWindow.IsValid())
 	{
 		SlateWindow->SetIsFocusable(bWindowFocusable);
 	}
};

EVisibility UWidgetComponent::ConvertWindowVisibilityToVisibility(EWindowVisibility visibility)
{
	switch (visibility)
	{
	case EWindowVisibility::Visible:
		return EVisibility::Visible;
	case EWindowVisibility::SelfHitTestInvisible:
		return EVisibility::SelfHitTestInvisible;
	default:
		checkNoEntry();
		return EVisibility::SelfHitTestInvisible;
	}	
}

void UWidgetComponent::OnWidgetVisibilityChanged(ESlateVisibility InVisibility)
{
	ensure(TickMode != ETickMode::Enabled);
	ensure(Widget);
	ensure(bOnWidgetVisibilityChangedRegistered);

	if (InVisibility != ESlateVisibility::Collapsed && InVisibility != ESlateVisibility::Hidden)
	{
		if (ShouldReenableComponentTickWhenWidgetBecomesVisible())
		{
			SetComponentTickEnabled(true);
		}

		if (bOnWidgetVisibilityChangedRegistered)
		{
			if (Widget)
			{
				Widget->OnNativeVisibilityChanged.RemoveAll(this);
			}
			bOnWidgetVisibilityChangedRegistered = false;
		}
	}
}

void UWidgetComponent::SetWindowVisibility(EWindowVisibility InVisibility)
{
	ensure(Widget);

	WindowVisibility = InVisibility;
	if (SlateWindow.IsValid())
	{
		SlateWindow->SetVisibility(ConvertWindowVisibilityToVisibility(WindowVisibility));
		if (bUseInvalidationInWorldSpace)
		{
			SlateWindow->InvalidateRootLayout();
		}
	}

	if (IsWidgetVisible())
	{
		if (ShouldReenableComponentTickWhenWidgetBecomesVisible())
		{
			SetComponentTickEnabled(true);
		}

		if (bOnWidgetVisibilityChangedRegistered)
		{
			if (Widget)
			{
				Widget->OnNativeVisibilityChanged.RemoveAll(this);
			}
			bOnWidgetVisibilityChangedRegistered = false;
		}
	}
}

void UWidgetComponent::SetTickMode(ETickMode InTickMode)
{
	TickMode = InTickMode;
	SetComponentTickEnabled(InTickMode != ETickMode::Disabled);
}

bool UWidgetComponent::IsWidgetVisible() const
{
	//  If we are in World Space, if the component or the SlateWindow is not visible the Widget is not visible.
	if (Space == EWidgetSpace::World && (!IsVisible() || !SlateWindow.IsValid() || !SlateWindow->GetVisibility().IsVisible()))
	{
		return false;
	}	
	
	// If we have a UUserWidget check its visibility
	if (Widget)
	{
		return Widget->IsVisible();
	}

	// If we use a SlateWidget check its visibility
	return SlateWidget.IsValid() && SlateWidget->GetVisibility().IsVisible();
}

bool UWidgetComponent::CanReceiveHardwareInput() const
{
	return bReceiveHardwareInput && GeometryMode == EWidgetGeometryMode::Plane;
}

void UWidgetComponent::RegisterHitTesterWithViewport(TSharedPtr<SViewport> ViewportWidget)
{
#if !UE_SERVER
	if ( ViewportWidget.IsValid() )
	{
		TSharedPtr<ICustomHitTestPath> CustomHitTestPath = ViewportWidget->GetCustomHitTestPath();
		if ( !CustomHitTestPath.IsValid() )
		{
			CustomHitTestPath = MakeShareable(new FWidget3DHitTester(GetWorld()));
			ViewportWidget->SetCustomHitTestPath(CustomHitTestPath);
		}

		TSharedPtr<FWidget3DHitTester> Widget3DHitTester = StaticCastSharedPtr<FWidget3DHitTester>(CustomHitTestPath);
		if ( Widget3DHitTester->GetWorld() == GetWorld() )
		{
			Widget3DHitTester->RegisterWidgetComponent(this);
		}
	}
#endif

}

void UWidgetComponent::UnregisterHitTesterWithViewport(TSharedPtr<SViewport> ViewportWidget)
{
#if !UE_SERVER
	if ( CanReceiveHardwareInput() )
	{
		TSharedPtr<ICustomHitTestPath> CustomHitTestPath = ViewportWidget->GetCustomHitTestPath();
		if ( CustomHitTestPath.IsValid() )
		{
			TSharedPtr<FWidget3DHitTester> WidgetHitTestPath = StaticCastSharedPtr<FWidget3DHitTester>(CustomHitTestPath);

			WidgetHitTestPath->UnregisterWidgetComponent(this);

			if ( WidgetHitTestPath->GetNumRegisteredComponents() == 0 )
			{
				ViewportWidget->SetCustomHitTestPath(nullptr);
			}
		}
	}
#endif
}

void UWidgetComponent::OnUnregister()
{
#if !UE_SERVER
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

	if ( GetWorld()->IsGameWorld() )
	{
		TSharedPtr<SViewport> GameViewportWidget = GEngine->GetGameViewportWidget();
		if ( GameViewportWidget.IsValid() )
		{
			UnregisterHitTesterWithViewport(GameViewportWidget);
		}
	}
#endif

#if WITH_EDITOR
	if (!GetWorld()->IsGameWorld())
	{
		ReleaseResources();
	}
#endif

	Super::OnUnregister();
}

void UWidgetComponent::DestroyComponent(bool bPromoteChildren/*= false*/)
{
	Super::DestroyComponent(bPromoteChildren);

	ReleaseResources();
}

void UWidgetComponent::ReleaseResources()
{
	if (Widget)
	{
		RemoveWidgetFromScreen();
		if (bOnWidgetVisibilityChangedRegistered)
		{
			Widget->OnNativeVisibilityChanged.RemoveAll(this);
			bOnWidgetVisibilityChangedRegistered = false;
		}
		Widget = nullptr;
	}

	if (SlateWidget.IsValid())
	{
		RemoveWidgetFromScreen();
		SlateWidget.Reset();
	}
	
	if (WidgetRenderer)
	{
		BeginCleanup(WidgetRenderer);
		WidgetRenderer = nullptr;
	}

	UnregisterWindow();
}

void UWidgetComponent::RegisterWindow()
{
	if ( SlateWindow.IsValid() )
	{
		if (!CanReceiveHardwareInput() && FSlateApplication::IsInitialized() )
		{
			FSlateApplication::Get().RegisterVirtualWindow(SlateWindow.ToSharedRef());
		}

		if (Widget && !Widget->IsDesignTime())
		{
			if (UWorld* LocalWorld = GetWorld())
			{
				if (LocalWorld->IsGameWorld())
				{
					if (UGameInstance* GameInstance = LocalWorld->GetGameInstance())
					{
						if (UGameViewportClient* GameViewportClient = GameInstance->GetGameViewportClient())
						{
							SlateWindow->AssignParentWidget(GameViewportClient->GetGameViewportWidget());
						}
					}
				}
			}
		}
	}
}

void UWidgetComponent::UnregisterWindow()
{
	if ( SlateWindow.IsValid() )
	{
		if ( !CanReceiveHardwareInput() && FSlateApplication::IsInitialized() )
		{
			FSlateApplication::Get().UnregisterVirtualWindow(SlateWindow.ToSharedRef());
		}

		SlateWindow.Reset();
	}
}

void UWidgetComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (IsRunningDedicatedServer())
	{
		SetTickMode(ETickMode::Disabled);
		return;
	}


#if !UE_SERVER
	if (!IsRunningDedicatedServer())
	{
		UpdateWidget();

		// There is no Widget set and we already rendered an empty widget. No need to continue.
		if (Widget == nullptr && !SlateWidget.IsValid() && bRenderCleared)
		{
			return;	
		}

		// We have a Widget, it's invisible and we are in automatic or disabled TickMode, we disable ticking and register a callback to know if visibility changes.
		if (Widget && TickMode != ETickMode::Enabled && !IsWidgetVisible())
		{
			SetComponentTickEnabled(false);
			if (!bOnWidgetVisibilityChangedRegistered)
			{
				Widget->OnNativeVisibilityChanged.AddUObject(this, &UWidgetComponent::OnWidgetVisibilityChanged);
				bOnWidgetVisibilityChangedRegistered = true;
			}
		}

		// Tick Mode is Disabled, we stop here and Disable the Component Tick
		if (TickMode == ETickMode::Disabled && !bRedrawRequested)
		{
			SetComponentTickEnabled(false);
			return;
		}

	    if ( Space == EWidgetSpace::World)
	    {
			if ( ShouldDrawWidget() )
		    {
				// Calculate the actual delta time since we last drew, this handles the case where we're ticking when
				// the world is paused, this also takes care of the case where the widget component is rendering at
				// a different rate than the rest of the world.
				const float DeltaTimeFromLastDraw = static_cast<float>(LastWidgetRenderTime == 0 ? 0 : (GetCurrentTime() - LastWidgetRenderTime));
				DrawWidgetToRenderTarget(DeltaTimeFromLastDraw);

				// We draw an empty widget.
				if (Widget == nullptr && !SlateWidget.IsValid())
				{
					bRenderCleared = true;
				}
		    }
	    }
	    
	}
#endif // !UE_SERVER
}

bool UWidgetComponent::ShouldReenableComponentTickWhenWidgetBecomesVisible() const
{
	return (TickMode != ETickMode::Disabled) || bRedrawRequested;
}

void UWidgetComponent::UpdateWidgetOnScreen()
{
	if ((GetUserWidgetObject() || GetSlateWidget().IsValid()) && Space == EWidgetSpace::Screen)
	{
		UWorld* ThisWorld = GetWorld();
		if (ThisWorld && ThisWorld->IsGameWorld())
		{
			ULocalPlayer* TargetPlayer = GetOwnerPlayer();
			APlayerController* PlayerController = TargetPlayer ? ToRawPtr(TargetPlayer->PlayerController) : nullptr;
			if (TargetPlayer && PlayerController && IsVisible() && !(GetOwner()->IsHidden()))
			{
				if (!bAddedToScreen)
				{
					AddWidgetToScreen(TargetPlayer);
				}
				return;
			}
		}
	}
	if (bAddedToScreen)
	{
		RemoveWidgetFromScreen();
	}
}

void UWidgetComponent::AddWidgetToScreen(ULocalPlayer* TargetPlayer)
{
	if (GetUserWidgetObject() || GetSlateWidget().IsValid())
	{
		UWorld* ThisWorld = GetWorld();
		if (ThisWorld && ThisWorld->IsGameWorld())
		{
			if (UGameViewportClient* ViewportClient = ThisWorld->GetGameViewport())
			{
				TSharedPtr<IGameLayerManager> LayerManager = ViewportClient->GetGameLayerManager();
				if (LayerManager.IsValid())
				{
					TSharedPtr<FWorldWidgetScreenLayer> ScreenLayer;

					FLocalPlayerContext PlayerContext(TargetPlayer, ThisWorld);

					TSharedPtr<IGameLayer> Layer = LayerManager->FindLayerForPlayer(TargetPlayer, SharedLayerName);
					if (!Layer.IsValid())
					{
						TSharedRef<FWorldWidgetScreenLayer> NewScreenLayer = MakeShareable(new FWorldWidgetScreenLayer(PlayerContext));
						LayerManager->AddLayerForPlayer(TargetPlayer, SharedLayerName, NewScreenLayer, LayerZOrder);
						ScreenLayer = NewScreenLayer;
					}
					else
					{
						ScreenLayer = StaticCastSharedPtr<FWorldWidgetScreenLayer>(Layer);
					}

					bAddedToScreen = true;
					ScreenLayer->AddComponent(this);
				}
			}
		}
	}
}

bool UWidgetComponent::ShouldDrawWidget() const
{
	const float RenderTimeThreshold = .5f;
	if (IsVisible())
	{
		// If we don't tick when off-screen, don't bother ticking if it hasn't been rendered recently
		if (TickWhenOffscreen || WasRecentlyRendered(RenderTimeThreshold) || LastWidgetRenderTime == 0.0)
		{
			if ( ( GetCurrentTime() - LastWidgetRenderTime) >= RedrawTime )
			{
				return bManuallyRedraw ? bRedrawRequested : true;
			}
		}
	}

	return false;
}

void UWidgetComponent::DrawWidgetToRenderTarget(float DeltaTime)
{
	if ( GUsingNullRHI )
	{
		return;
	}

	if ( !SlateWindow.IsValid() )
	{
		return;
	}

	if ( !WidgetRenderer )
	{
		return;
	}

	const int32 MaxAllowedDrawSize = GetMax2DTextureDimension();
	if ( DrawSize.X <= 0 || DrawSize.Y <= 0 || DrawSize.X > MaxAllowedDrawSize || DrawSize.Y > MaxAllowedDrawSize )
	{
		return;
	}

	const FIntPoint PreviousDrawSize = CurrentDrawSize;
	CurrentDrawSize = DrawSize;

	const float DrawScale = 1.0f;

	if (bUseInvalidationInWorldSpace)
	{
		SlateWindow->ProcessWindowInvalidation();
		SlateWindow->SlatePrepass(DrawScale);
		WidgetRenderer->SetIsPrepassNeeded(false);
	}
	else if (bDrawAtDesiredSize)
	{
		SlateWindow->SlatePrepass(DrawScale);
		WidgetRenderer->SetIsPrepassNeeded(false);
	}
	else
	{
		WidgetRenderer->SetIsPrepassNeeded(true);
	}

	bool bHasValidSize = true;
	if ( bDrawAtDesiredSize )
	{
		FVector2D DesiredSize = SlateWindow->GetDesiredSize();
		DesiredSize.X = FMath::RoundToDouble(DesiredSize.X);
		DesiredSize.Y = FMath::RoundToDouble(DesiredSize.Y);
		CurrentDrawSize = DesiredSize.IntPoint();

		if (DesiredSize.X <= 0 || DesiredSize.Y <= 0)
		{
			bHasValidSize = false;
		}
	}

	if ( CurrentDrawSize != PreviousDrawSize )
	{
		if (bHasValidSize)
		{
			UpdateBodySetup(true);
			RecreatePhysicsState();
		}
		else
		{
			DestroyPhysicsState();
		}
	}

	UpdateRenderTarget(CurrentDrawSize);

	// The render target could be null if the current draw size is zero
	if(RenderTarget)
	{
		bRedrawRequested = false;

		if (bUseInvalidationInWorldSpace)
		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.DeferRetainedRenderingRenderThread"));
			ensure(CVar);
			bool bDeferRetainedRenderingRenderThread = CVar ? CVar->GetInt() != 0 : false;

			FPaintArgs PaintArgs(nullptr, SlateWindow->GetHittestGrid(), FVector2D::ZeroVector, FApp::GetCurrentTime(), DeltaTime);
			TSharedRef<SVirtualWindow> WindowToDraw = SlateWindow.ToSharedRef();

			WidgetRenderer->DrawInvalidationRoot(WindowToDraw, RenderTarget, PaintArgs, DrawScale, CurrentDrawSize, bDeferRetainedRenderingRenderThread);
		}
		else
		{
			WidgetRenderer->DrawWindow(
				RenderTarget,
				SlateWindow->GetHittestGrid(),
				SlateWindow.ToSharedRef(),
				DrawScale,
				CurrentDrawSize,
				DeltaTime);
		}

		LastWidgetRenderTime = GetCurrentTime();

		if (TickMode == ETickMode::Disabled && IsComponentTickEnabled())
		{
			SetComponentTickEnabled(false);
		}
	}
}

double UWidgetComponent::ComputeComponentWidth() const
{
	switch (GeometryMode)
	{
		default:
		case EWidgetGeometryMode::Plane:
			return CurrentDrawSize.X;

		case EWidgetGeometryMode::Cylinder:
			const double ArcAngleRadians = FMath::DegreesToRadians(GetCylinderArcAngle());
			const double Radius = CurrentDrawSize.X / ArcAngleRadians;
			// Chord length is 2*R*Sin(Theta/2)
			return 2.0f * Radius * FMath::Sin(0.5f*ArcAngleRadians);
	}
}

double UWidgetComponent::GetCurrentTime() const
{
	return (TimingPolicy == EWidgetTimingPolicy::RealTime) ? FApp::GetCurrentTime() : static_cast<double>(GetWorld()->GetTimeSeconds());
}

void UWidgetComponent::OnWorldUserEntered(int32 SlateUserIndex)
{
	UsersEnteredWorldWidget.Add(SlateUserIndex);
}

void UWidgetComponent::ResetWorldUserFocusOnWidget()
{
	// we want to go through all the users which could've focused
	// on our widget to release any focus paths/strongs refs they might have to our widget.
	for(int32 UserIndex : UsersEnteredWorldWidget)
	{
		TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(UserIndex);
		if (!SlateUser)
		{
			continue;
		}

		TSharedPtr<SWidget> CurrentWidgetInPath = SlateUser->GetFocusedWidget();
		while(CurrentWidgetInPath != nullptr)
		{
			if(CurrentWidgetInPath == SlateWindow)
			{
				FSlateApplication::Get().ClearUserFocus(UserIndex, EFocusCause::SetDirectly);

				break;
			}
			
			CurrentWidgetInPath = CurrentWidgetInPath->GetParentWidget();
		}
	}
	UsersEnteredWorldWidget.Reset();
}

void UWidgetComponent::RemoveWidgetFromScreen()
{
#if !UE_SERVER
	if (!IsRunningDedicatedServer())
	{
		if(SlateWindow != nullptr)
		{
			ResetWorldUserFocusOnWidget();
			
			if(GetWidgetSpace() == EWidgetSpace::World)
			{
				SlateWindow->SetContent(SNullWidget::NullWidget);
			}
			
			SlateWindow.Reset();
		}
		
		bAddedToScreen = false;

		if (UWorld* World = GetWorld())
		{
			if (UGameViewportClient* ViewportClient = World->GetGameViewport())
			{
				if (ULocalPlayer* TargetPlayer = GetOwnerPlayer())
				{
					TSharedPtr<IGameLayerManager> LayerManager = ViewportClient->GetGameLayerManager();
					if (LayerManager.IsValid())
					{
						TSharedPtr<IGameLayer> Layer = LayerManager->FindLayerForPlayer(TargetPlayer, SharedLayerName);
						if (Layer.IsValid())
						{
							TSharedPtr<FWorldWidgetScreenLayer> ScreenLayer = StaticCastSharedPtr<FWorldWidgetScreenLayer>(Layer);
							ScreenLayer->RemoveComponent(this);
						}
					}
				}
			}
		}
	}
#endif // !UE_SERVER
}

TStructOnScope<FActorComponentInstanceData> UWidgetComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FWidgetComponentInstanceData>(this);
}

void UWidgetComponent::ApplyComponentInstanceData(FWidgetComponentInstanceData* WidgetInstanceData)
{
	check(WidgetInstanceData);

	// Note: ApplyComponentInstanceData is called while the component is registered so the rendering thread is already using this component
	// That means all component state that is modified here must be mirrored on the scene proxy, which will be recreated to receive the changes later due to MarkRenderStateDirty.

	if (GetWidgetClass() != WidgetInstanceData->WidgetClass)
	{
		return;
	}

	RenderTarget = WidgetInstanceData->RenderTarget;
	if ( MaterialInstance && RenderTarget )
	{
		MaterialInstance->SetTextureParameterValue("SlateUI", RenderTarget);
	}

	MarkRenderStateDirty();
}

void UWidgetComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	if (MaterialInstance)
	{
		OutMaterials.AddUnique(MaterialInstance);
	}
}

#if WITH_EDITOR

bool UWidgetComponent::CanEditChange(const FProperty* InProperty) const
{
	if ( InProperty )
	{
		FString PropertyName = InProperty->GetName();

		if ( PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, GeometryMode) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, TimingPolicy) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, bWindowFocusable) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, WindowVisibility) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, bManuallyRedraw) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, RedrawTime) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, BackgroundColor) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, TintColorAndOpacity) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, OpacityFromTexture) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, BlendMode) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, bIsTwoSided) ||
			 PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, TickWhenOffscreen) )
		{
			return Space != EWidgetSpace::Screen;
		}

		if ( PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, bReceiveHardwareInput) )
		{
			return Space != EWidgetSpace::Screen && GeometryMode == EWidgetGeometryMode::Plane;
		}

		if ( PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UWidgetComponent, CylinderArcAngle) )
		{
			return GeometryMode == EWidgetGeometryMode::Cylinder;
		}
	}

	return Super::CanEditChange(InProperty);
}

void UWidgetComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* Property = PropertyChangedEvent.MemberProperty;

	if( Property && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive )
	{
		static FName DrawSizeName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, DrawSize);
		static FName PivotName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, Pivot);
		static FName WidgetClassName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, WidgetClass);
		static FName IsTwoSidedName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, bIsTwoSided);
		static FName BackgroundColorName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, BackgroundColor);
		static FName TintColorAndOpacityName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, TintColorAndOpacity);
		static FName OpacityFromTextureName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, OpacityFromTexture);
		static FName BlendModeName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, BlendMode);
		static FName GeometryModeName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, GeometryMode);
		static FName CylinderArcAngleName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, CylinderArcAngle);
		static FName bWindowFocusableName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, bWindowFocusable);
		static FName WindowVisibilityName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, WindowVisibility);
		static FName UseInvalidationName = GET_MEMBER_NAME_CHECKED(UWidgetComponent, bUseInvalidationInWorldSpace);

		auto PropertyName = Property->GetFName();

		if (PropertyName == DrawSizeName)
		{
			if (DrawSize.X > MaximumRenderTargetWidth)
			{
				DrawSize.X = MaximumRenderTargetWidth;
			}
			if (DrawSize.Y > MaximumRenderTargetHeight)
			{
				DrawSize.Y = MaximumRenderTargetHeight;
			}
		}

		if (PropertyName == WidgetClassName)
		{
			Widget = nullptr;

			UpdateWidget();
			MarkRenderStateDirty();
		}
		else if (PropertyName == UseInvalidationName)
		{
			if (SlateWindow)
			{
				SlateWindow->SetAllowFastUpdate(bUseInvalidationInWorldSpace);
				SlateWindow->InvalidateRootLayout();
			}
		}
		else if (PropertyName == DrawSizeName
			|| PropertyName == PivotName
			|| PropertyName == GeometryModeName
			|| PropertyName == CylinderArcAngleName)
		{
			MarkRenderStateDirty();
			UpdateBodySetup(true);
			RecreatePhysicsState();
		}
		else if (PropertyName == IsTwoSidedName
			|| PropertyName == BlendModeName
			|| PropertyName == BackgroundColorName
			|| PropertyName == TintColorAndOpacityName
			|| PropertyName == OpacityFromTextureName)
		{
			MarkRenderStateDirty();
		}
		else if (PropertyName == bWindowFocusableName)
		{
			SetWindowFocusable(bWindowFocusable);
		}
		else if (PropertyName == WindowVisibilityName)
		{
			SetWindowVisibility(WindowVisibility);
		}

	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif

void UWidgetComponent::InitWidget()
{
	if (IsRunningDedicatedServer())
	{
		SetTickMode(ETickMode::Disabled);
		return;
	}

	// Don't do any work if Slate is not initialized
	if ( FSlateApplication::IsInitialized() )
	{
		if (UWorld* World = GetWorld())
		{
			if (WidgetClass && Widget == nullptr && !World->bIsTearingDown)
			{
				Widget = CreateWidget(World, WidgetClass);
				SetTickMode(TickMode);
			}

#if WITH_EDITOR
			if (Widget && !World->IsGameWorld() && !bEditTimeUsable)
			{
				if (!GEnableVREditorHacks)
				{
					// Prevent native ticking of editor component previews
					Widget->SetDesignerFlags(EWidgetDesignFlags::Designing);
				}
			}
#endif
		}
	}
}

void UWidgetComponent::SetOwnerPlayer(ULocalPlayer* LocalPlayer)
{
	if ( OwnerPlayer != LocalPlayer )
	{
		RemoveWidgetFromScreen();
		OwnerPlayer = LocalPlayer;
	}
}

void UWidgetComponent::SetManuallyRedraw(bool bUseManualRedraw)
{
	bManuallyRedraw = bUseManualRedraw;
}

ULocalPlayer* UWidgetComponent::GetOwnerPlayer() const
{
	if (OwnerPlayer)
	{
		return OwnerPlayer;
	}

	if (APawn* OwningPawn = Cast<APawn>(GetOwner()))
	{
		if (APlayerController* OwningController = Cast<APlayerController>(OwningPawn->GetController()))
		{
			if (ULocalPlayer* OwningLocalPlayer = OwningController->GetLocalPlayer())
			{
				return OwningLocalPlayer;
			}
		}
	}
	
	if (const UWorld* LocalWorld = GetWorld())
	{
		if (const UGameInstance* GameInstance = LocalWorld->GetGameInstance())
		{
			return GameInstance->GetFirstGamePlayer();
		}
	}

	return nullptr;
}

UUserWidget* UWidgetComponent::GetWidget() const
{
	return Widget;
}

void UWidgetComponent::SetWidget(UUserWidget* InWidget)
{
	if (InWidget != nullptr)
	{
		SetSlateWidget(nullptr);
	}

	if (Widget)
	{
		RemoveWidgetFromScreen();
	}

	Widget = InWidget;

	UpdateWidget();
}

void UWidgetComponent::SetSlateWidget(const TSharedPtr<SWidget>& InSlateWidget)
{
	if (Widget != nullptr)
	{
		SetWidget(nullptr);
	}

	if (SlateWidget.IsValid())
	{
		RemoveWidgetFromScreen();
		SlateWidget.Reset();
	}

	SlateWidget = InSlateWidget;

	UpdateWidget();
}

void UWidgetComponent::UpdateWidget()
{
	// Don't do any work if Slate is not initialized
	if (FSlateApplication::IsInitialized() && IsValidChecked(this))
	{
		if (Space == EWidgetSpace::World)
		{
			// Look for a UMG widget set
			TSharedPtr<SWidget> NewSlateWidget;
			if (Widget)
			{
				NewSlateWidget = Widget->TakeWidget();
			}

			// Create the SlateWindow if it doesn't exists
			bool bNeededNewWindow = false;
			if (!SlateWindow.IsValid())
			{
				UpdateMaterialInstance();

				SlateWindow = SNew(SVirtualWindow).Size(CurrentDrawSize);
				SlateWindow->SetIsFocusable(bWindowFocusable);
				SlateWindow->SetAllowFastUpdate(bUseInvalidationInWorldSpace);
				SlateWindow->SetVisibility(ConvertWindowVisibilityToVisibility(WindowVisibility));
				RegisterWindow();

				bNeededNewWindow = true;
			}

			SlateWindow->Resize(CurrentDrawSize);

			// Add the UMG or SlateWidget to the Component
			bool bWidgetChanged = false;

			// We Get here if we have a UMG Widget
			if (NewSlateWidget.IsValid())
			{
				if (NewSlateWidget != CurrentSlateWidget || bNeededNewWindow)
				{
					CurrentSlateWidget = NewSlateWidget;
					SlateWindow->SetContent(NewSlateWidget.ToSharedRef());
					bRenderCleared = false;
					bWidgetChanged = true;
				}
			}
			// If we don't have one, we look for a Slate Widget
			else if (SlateWidget.IsValid())
			{
				if (SlateWidget != CurrentSlateWidget || bNeededNewWindow)
				{
					CurrentSlateWidget = SlateWidget;
					SlateWindow->SetContent(SlateWidget.ToSharedRef());
					bRenderCleared = false;
					bWidgetChanged = true;
				}
			}
			else
			{
				if (CurrentSlateWidget != SNullWidget::NullWidget)
				{
					CurrentSlateWidget = SNullWidget::NullWidget;
					bRenderCleared = false;
					bWidgetChanged = true;
				}
				SlateWindow->SetContent(SNullWidget::NullWidget);
			}
		
			if (bNeededNewWindow || bWidgetChanged)
			{
				MarkRenderStateDirty();
				SetComponentTickEnabled(true);
			}
		}

		UpdateWidgetOnScreen();		
	}
}

void UWidgetComponent::UpdateRenderTarget(FIntPoint DesiredRenderTargetSize)
{
	bool bWidgetRenderStateDirty = false;
	bool bClearColorChanged = false;

	FLinearColor ActualBackgroundColor = BackgroundColor;
	switch ( BlendMode )
	{
	case EWidgetBlendMode::Opaque:
		ActualBackgroundColor.A = 1.0f;
		break;
	case EWidgetBlendMode::Masked:
		ActualBackgroundColor.A = 0.0f;
		break;
	}

	if ( DesiredRenderTargetSize.X != 0 && DesiredRenderTargetSize.Y != 0 )
	{
		const EPixelFormat requestedFormat = bOverrideRenderTargetFormat
			? GetPixelFormatFromRenderTargetFormat(RenderTargetFormatOverride)
			: FSlateApplication::Get().GetRenderer()->GetSlateRecommendedColorFormat();

		if ( RenderTarget == nullptr )
		{
			RenderTarget = NewObject<UTextureRenderTarget2D>(this);
			RenderTarget->ClearColor = ActualBackgroundColor;

			bClearColorChanged = bWidgetRenderStateDirty = true;

			RenderTarget->InitCustomFormat(DesiredRenderTargetSize.X, DesiredRenderTargetSize.Y, requestedFormat, false);

			if ( MaterialInstance )
			{
				MaterialInstance->SetTextureParameterValue("SlateUI", RenderTarget);
			}
		}
		else
		{
			bClearColorChanged = (RenderTarget->ClearColor != ActualBackgroundColor);

			// Update the clear color or format
			if ( bClearColorChanged || RenderTarget->SizeX != DesiredRenderTargetSize.X || RenderTarget->SizeY != DesiredRenderTargetSize.Y )
			{
				RenderTarget->ClearColor = ActualBackgroundColor;
				RenderTarget->InitCustomFormat(DesiredRenderTargetSize.X, DesiredRenderTargetSize.Y, requestedFormat, false);
				RenderTarget->UpdateResourceImmediate();
				bWidgetRenderStateDirty = true;
			}
		}
	}

	if ( RenderTarget )
	{
		// If the clear color of the render target changed, update the BackColor of the material to match
		if ( bClearColorChanged && MaterialInstance )
		{
			MaterialInstance->SetVectorParameterValue("BackColor", RenderTarget->ClearColor);
		}

		if ( bWidgetRenderStateDirty )
		{
			MarkRenderStateDirty();
		}
	}
}

void UWidgetComponent::UpdateBodySetup( bool bDrawSizeChanged )
{
	if (Space == EWidgetSpace::Screen)
	{
		// We do not have a body setup in screen space
		BodySetup = nullptr;
	}
	else if ( !BodySetup || bDrawSizeChanged )
	{
		BodySetup = NewObject<UBodySetup>(this);
		BodySetup->CollisionTraceFlag = CTF_UseSimpleAsComplex;
		BodySetup->AggGeom.BoxElems.Add(FKBoxElem());

		FKBoxElem* BoxElem = BodySetup->AggGeom.BoxElems.GetData();

		const double Width = ComputeComponentWidth();
		const double Height = CurrentDrawSize.Y;
		const FVector Origin = FVector(.5f,
			-( Width * 0.5f ) + ( Width * Pivot.X ),
			-( Height * 0.5f ) + ( Height * Pivot.Y ));
			
		BoxElem->X = 0.01f;
		BoxElem->Y = static_cast<float>(Width);
		BoxElem->Z = static_cast<float>(Height);

		BoxElem->SetTransform(FTransform::Identity);
		BoxElem->Center = Origin;
	}
}

void UWidgetComponent::GetLocalHitLocation(FVector WorldHitLocation, FVector2D& OutLocalWidgetHitLocation) const
{
	ensureMsgf(GeometryMode == EWidgetGeometryMode::Plane, TEXT("Method does not support non-planar widgets."));

	// Find the hit location on the component
	FVector ComponentHitLocation = GetComponentTransform().InverseTransformPosition(WorldHitLocation);

	// Convert the 3D position of component space, into the 2D equivalent
	OutLocalWidgetHitLocation = FVector2D(-ComponentHitLocation.Y, -ComponentHitLocation.Z);

	// Offset the position by the pivot to get the position in widget space.
	OutLocalWidgetHitLocation.X += CurrentDrawSize.X * Pivot.X;
	OutLocalWidgetHitLocation.Y += CurrentDrawSize.Y * Pivot.Y;

	// Apply the parabola distortion
	FVector2D NormalizedLocation = OutLocalWidgetHitLocation / CurrentDrawSize;

	OutLocalWidgetHitLocation.Y = CurrentDrawSize.Y * NormalizedLocation.Y;
}


TOptional<double> FindLineSphereIntersection(const FVector& Start, const FVector& Dir, float Radius)
{
	// Solution exist at two possible locations:
	// (Start + Dir * t) (dot) (Start + Dir * t) = Radius^2
	// Dir(dot)Dir*t^2 + 2*Start(dot)Dir + Start(dot)Start - Radius^2 = 0
	//
	// Recognize quadratic form with:
	const double a = FVector::DotProduct(Dir,Dir);
	const double b = 2 * FVector::DotProduct(Start,Dir);
	const double c = FVector::DotProduct(Start,Start) - Radius*Radius;

	const double Discriminant = b*b - 4 * a * c;
	
	if (Discriminant >= 0)
	{
		const double SqrtDiscr = FMath::Sqrt(Discriminant);
		const double Soln1 = (-b + SqrtDiscr) / (2 * a);

		return Soln1;
	}
	else
	{
		return TOptional<double>();
	}
}

TTuple<FVector, FVector2D> UWidgetComponent::GetCylinderHitLocation(FVector WorldHitLocation, FVector WorldHitDirection) const
{
	// Turn this on to see a visualiztion of cylindrical collision testing.
	static const bool bDrawCollisionDebug = false;

	ensure(GeometryMode == EWidgetGeometryMode::Cylinder);
		

	FTransform ToWorld = GetComponentToWorld();

	const FVector HitLocation_ComponentSpace = GetComponentTransform().InverseTransformPosition(WorldHitLocation);
	const FVector HitDirection_ComponentSpace = GetComponentTransform().InverseTransformVector(WorldHitDirection);


	const double ArcAngleRadians = FMath::DegreesToRadians(GetCylinderArcAngle());
	const double Radius = CurrentDrawSize.X / ArcAngleRadians;
	const double Apothem = Radius * FMath::Cos(0.5f*ArcAngleRadians);
	const double ChordLength = 2.0f * Radius * FMath::Sin(0.5f*ArcAngleRadians);

	const double PivotOffsetX = ChordLength * (0.5-Pivot.X);

	if (bDrawCollisionDebug)
	{
		// Draw component-space axes
		UKismetSystemLibrary::DrawDebugArrow((UWidgetComponent*)(this), ToWorld.TransformPosition(FVector::ZeroVector), ToWorld.TransformPosition(FVector(50.f, 0, 0)), 2.0f, FLinearColor::Red);
		UKismetSystemLibrary::DrawDebugArrow((UWidgetComponent*)(this), ToWorld.TransformPosition(FVector::ZeroVector), ToWorld.TransformPosition(FVector(0, 50.f, 0)), 2.0f, FLinearColor::Green);
		UKismetSystemLibrary::DrawDebugArrow((UWidgetComponent*)(this), ToWorld.TransformPosition(FVector::ZeroVector), ToWorld.TransformPosition(FVector(0, 0, 50.f)), 2.0f, FLinearColor::Blue);

		// Draw the imaginary circle which we use to describe the cylinder.
		// Note that we transform all the hit locations into a space where the circle's origin is at (0,0).
		UKismetSystemLibrary::DrawDebugCircle((UWidgetComponent*)(this), ToWorld.TransformPosition(FVector::ZeroVector), static_cast<float>(ToWorld.GetScale3D().X*Radius), 64, FLinearColor::Green,
			0, 1.0f, FVector(0, 1, 0), FVector(1, 0, 0));
		UKismetSystemLibrary::DrawDebugLine((UWidgetComponent*)(this), ToWorld.TransformPosition(FVector(-Apothem, -Radius, 0.0f)), ToWorld.TransformPosition(FVector(-Apothem, +Radius, 0.0f)), FLinearColor::Green);
	}

	const FVector HitLocation_CircleSpace( -Apothem, HitLocation_ComponentSpace.Y + PivotOffsetX, 0.0f );
	const FVector HitDirection_CircleSpace( HitDirection_ComponentSpace.X, HitDirection_ComponentSpace.Y, 0.0f );

	// DRAW HIT DIRECTION
	if (bDrawCollisionDebug)
	{
		UKismetSystemLibrary::DrawDebugCircle((UWidgetComponent*)(this), ToWorld.TransformPosition(FVector(HitLocation_CircleSpace.X, HitLocation_CircleSpace.Y,0)), 2.0f);
		FVector HitDirection_CircleSpace_Normalized = HitDirection_CircleSpace;
		HitDirection_CircleSpace_Normalized.Normalize();
		HitDirection_CircleSpace_Normalized *= 40;
		UKismetSystemLibrary::DrawDebugLine(
			(UWidgetComponent*)(this),
			ToWorld.TransformPosition(FVector(HitLocation_CircleSpace.X, HitLocation_CircleSpace.Y, 0.0f)),
			ToWorld.TransformPosition(FVector(HitLocation_CircleSpace.X + HitDirection_CircleSpace_Normalized.X, HitLocation_CircleSpace.Y + HitDirection_CircleSpace_Normalized.Y, 0.0f)),
			FLinearColor::White, 0, 0.1f);
	}

	// Perform a ray vs. circle intersection test (effectively in 2D because Z coordinate is always 0)
	const TOptional<double> Solution = FindLineSphereIntersection(HitLocation_CircleSpace, HitDirection_CircleSpace, static_cast<float>(Radius));
	if (Solution.IsSet())
	{
		const double Time = Solution.GetValue();

		const FVector TrueHitLocation_CircleSpace = HitLocation_CircleSpace + HitDirection_CircleSpace * Time;
		if (bDrawCollisionDebug)
		{
			UKismetSystemLibrary::DrawDebugLine((UWidgetComponent*)(this),
				ToWorld.TransformPosition(FVector(HitLocation_CircleSpace.X, HitLocation_CircleSpace.Y, 0.0f)),
				ToWorld.TransformPosition(FVector(TrueHitLocation_CircleSpace.X, TrueHitLocation_CircleSpace.Y, 0.0f)),
				FLinearColor(1, 0, 1, 1), 0, 0.5f);
		 }
			
		// Determine the widget-space X hit coordinate.
		const double Endpoint1 = FMath::Fmod(FMath::Atan2(-0.5f*ChordLength, -Apothem) + 2*PI, 2*PI);
		const double Endpoint2 = FMath::Fmod(FMath::Atan2(+0.5f*ChordLength, -Apothem) + 2*PI, 2*PI);
		const double HitAngleRads = FMath::Fmod((float)FMath::Atan2(TrueHitLocation_CircleSpace.Y, TrueHitLocation_CircleSpace.X) + 2*PI, 2*PI);
		const double HitAngleZeroToOne = (HitAngleRads - FMath::Min(Endpoint1, Endpoint2)) / FMath::Abs(Endpoint2 - Endpoint1);


		// Determine the widget-space Y hit coordinate
		const FVector CylinderHitLocation_ComponentSpace = HitLocation_ComponentSpace + HitDirection_ComponentSpace*Time;
		const double YHitLocation = (-CylinderHitLocation_ComponentSpace.Z + CurrentDrawSize.Y*Pivot.Y);

		const FVector2D WidgetSpaceHitCoord = FVector2D(HitAngleZeroToOne * CurrentDrawSize.X, YHitLocation);
			
		return MakeTuple(GetComponentTransform().TransformPosition(CylinderHitLocation_ComponentSpace), WidgetSpaceHitCoord);
	}
	else
	{
		return MakeTuple(FVector::ZeroVector, FVector2D::ZeroVector);
	}
}

UUserWidget* UWidgetComponent::GetUserWidgetObject() const
{
	return Widget;
}

UTextureRenderTarget2D* UWidgetComponent::GetRenderTarget() const
{
	return RenderTarget;
}

UMaterialInstanceDynamic* UWidgetComponent::GetMaterialInstance() const
{
	return MaterialInstance;
}

const TSharedPtr<SWidget>& UWidgetComponent::GetSlateWidget() const
{
	return SlateWidget;
}

TArray<FWidgetAndPointer> UWidgetComponent::GetHitWidgetPath(FVector WorldHitLocation, bool bIgnoreEnabledStatus, float CursorRadius)
{
	ensure(GeometryMode == EWidgetGeometryMode::Plane);

	FVector2D LocalHitLocation;
	GetLocalHitLocation(WorldHitLocation, LocalHitLocation);

	return GetHitWidgetPath(LocalHitLocation, bIgnoreEnabledStatus, CursorRadius);
}


TArray<FWidgetAndPointer> UWidgetComponent::GetHitWidgetPath(FVector2D WidgetSpaceHitCoordinate, bool bIgnoreEnabledStatus, float CursorRadius /*= 0.0f*/)
{
	const FVector2D& LocalHitLocation = WidgetSpaceHitCoordinate;
	const FVirtualPointerPosition VirtualMouseCoordinate(LocalHitLocation, LastLocalHitLocation);

	// Cache the location of the hit
	LastLocalHitLocation = LocalHitLocation;

	TArray<FWidgetAndPointer> ArrangedWidgets;
	if ( SlateWindow.IsValid() )
	{
		// @todo slate - widget components would need to be associated with a user for this to be anthing valid
		const int32 UserIndex = INDEX_NONE;
		ArrangedWidgets = SlateWindow->GetHittestGrid().GetBubblePath( LocalHitLocation, CursorRadius, bIgnoreEnabledStatus, UserIndex);

		for( FWidgetAndPointer& ArrangedWidget : ArrangedWidgets )
		{
			ArrangedWidget.SetPointerPosition(VirtualMouseCoordinate);
		}
	}

	return ArrangedWidgets;
}

TSharedPtr<SWindow> UWidgetComponent::GetSlateWindow() const
{
	return SlateWindow;
}

FVector2D UWidgetComponent::GetDrawSize() const
{
	return DrawSize;
}

FVector2D UWidgetComponent::GetCurrentDrawSize() const
{
	return CurrentDrawSize;
}

void UWidgetComponent::SetDrawSize(FVector2D Size)
{
	FIntPoint NewDrawSize((int32)Size.X, (int32)Size.Y);

	if ( NewDrawSize != DrawSize )
	{
		DrawSize = NewDrawSize;
		MarkRenderStateDirty();
	}
}

void UWidgetComponent::RequestRedraw()
{
	bRedrawRequested = true;
}

void UWidgetComponent::RequestRenderUpdate()
{
	bRedrawRequested = true;
	if (TickMode == ETickMode::Disabled)
	{
		SetComponentTickEnabled(true);
	}
}

void UWidgetComponent::SetBlendMode( const EWidgetBlendMode NewBlendMode )
{
	if( NewBlendMode != this->BlendMode )
	{
		this->BlendMode = NewBlendMode;
		if( IsRegistered() )
		{
			MarkRenderStateDirty();
		}
	}
}

void UWidgetComponent::SetTwoSided( const bool bWantTwoSided )
{
	if( bWantTwoSided != this->bIsTwoSided )
	{
		this->bIsTwoSided = bWantTwoSided;
		if( IsRegistered() )
		{
			MarkRenderStateDirty();
		}
	}
}

void UWidgetComponent::SetBackgroundColor( const FLinearColor NewBackgroundColor )
{
	if( NewBackgroundColor != this->BackgroundColor)
	{
		this->BackgroundColor = NewBackgroundColor;
		MarkRenderStateDirty();
	}
}

void UWidgetComponent::SetTintColorAndOpacity( const FLinearColor NewTintColorAndOpacity )
{
	if( NewTintColorAndOpacity != this->TintColorAndOpacity )
	{
		this->TintColorAndOpacity = NewTintColorAndOpacity;
		UpdateMaterialInstanceParameters();
	}
}

void UWidgetComponent::SetOpacityFromTexture( const float NewOpacityFromTexture )
{
	if( NewOpacityFromTexture != this->OpacityFromTexture )
	{
		this->OpacityFromTexture = NewOpacityFromTexture;
		UpdateMaterialInstanceParameters();
	}
}

TSharedPtr< SWindow > UWidgetComponent::GetVirtualWindow() const
{
	return StaticCastSharedPtr<SWindow>(SlateWindow);
}

UMaterialInterface* UWidgetComponent::GetMaterial(int32 MaterialIndex) const
{
	if ( OverrideMaterials.IsValidIndex(MaterialIndex) && ( OverrideMaterials[MaterialIndex] != nullptr ) )
	{
		return OverrideMaterials[MaterialIndex];
	}
	else
	{
		switch ( BlendMode )
		{
		case EWidgetBlendMode::Opaque:
			return bIsTwoSided ? OpaqueMaterial : OpaqueMaterial_OneSided;
		case EWidgetBlendMode::Masked:
			return bIsTwoSided ? MaskedMaterial : MaskedMaterial_OneSided;
		case EWidgetBlendMode::Transparent:
			return bIsTwoSided ? TranslucentMaterial : TranslucentMaterial_OneSided;
		}
	}

	return nullptr;
}

int32 UWidgetComponent::GetNumMaterials() const
{
	return FMath::Max<int32>(OverrideMaterials.Num(), 1);
}

void UWidgetComponent::UpdateMaterialInstanceParameters()
{
	if ( MaterialInstance )
	{
		MaterialInstance->SetTextureParameterValue("SlateUI", RenderTarget);
		MaterialInstance->SetVectorParameterValue("TintColorAndOpacity", TintColorAndOpacity);
		MaterialInstance->SetScalarParameterValue("OpacityFromTexture", OpacityFromTexture);
	}
}

void UWidgetComponent::SetWidgetClass(TSubclassOf<UUserWidget> InWidgetClass)
{
	if (WidgetClass != InWidgetClass)
	{
		WidgetClass = InWidgetClass;

		if (FSlateApplication::IsInitialized())
		{
			if (HasBegunPlay() && !GetWorld()->bIsTearingDown)
			{
				if (WidgetClass)
				{
					UUserWidget* NewWidget = CreateWidget(GetWorld(), WidgetClass);
					SetWidget(NewWidget);
				}
				else
				{
					SetWidget(nullptr);
				}
			}
		}
	}
}

