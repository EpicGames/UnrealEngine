// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WidgetInteractionComponent.h"
#include "UMGPrivate.h"
#include "CollisionQueryParams.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/GameViewportClient.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Components/ArrowComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Kismet/GameplayStatics.h"
#include "DrawDebugHelpers.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

#include "Components/WidgetComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetInteractionComponent)


#define LOCTEXT_NAMESPACE "WidgetInteraction"

UWidgetInteractionComponent::UWidgetInteractionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VirtualUserIndex(0)
	, PointerIndex(0)
	, InteractionDistance(500)
	, InteractionSource(EWidgetInteractionSource::World)
	, bEnableHitTesting(true)
	, bShowDebug(false)
	, DebugSphereLineThickness(2.f)
	, DebugLineThickness(1.f)
	, DebugColor(FLinearColor::Red)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = true;
	TraceChannel = ECC_Visibility;
	bAutoActivate = true;

#if WITH_EDITORONLY_DATA
	ArrowComponent = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UArrowComponent>(this, TEXT("ArrowComponent0"));

	if ( ArrowComponent && !IsTemplate() )
	{
		ArrowComponent->ArrowColor = DebugColor.ToFColor(true);
		ArrowComponent->AttachToComponent(this, FAttachmentTransformRules(EAttachmentRule::KeepRelative, false));
	}
#endif
	
#if WITH_EDITOR
	FWorldDelegates::OnPIEEnded.AddUObject(this, &UWidgetInteractionComponent::OnPIEEnded);
#endif
}

void UWidgetInteractionComponent::OnComponentCreated()
{
	Super::OnComponentCreated();
	
#if WITH_EDITORONLY_DATA
	if ( ArrowComponent )
	{
		ArrowComponent->ArrowColor = DebugColor.ToFColor(true);
		ArrowComponent->SetVisibility(bEnableHitTesting);
	}
#endif
}

void UWidgetInteractionComponent::Activate(bool bReset)
{
	Super::Activate(bReset);

	// Only create another user in a real world. FindOrCreateVirtualUser changes focus
	if ( FSlateApplication::IsInitialized() && !GetWorld()->IsPreviewWorld())
	{
		if ( !VirtualUser.IsValid() )
		{
			VirtualUser = FSlateApplication::Get().FindOrCreateVirtualUser(VirtualUserIndex);
		}
	}
}

void UWidgetInteractionComponent::Deactivate()
{
	Super::Deactivate();

	if ( FSlateApplication::IsInitialized() )
	{
		if ( VirtualUser.IsValid() )
		{
			FSlateApplication::Get().UnregisterUser(VirtualUser->GetUserIndex());
			VirtualUser.Reset();
		}
	}
}

void UWidgetInteractionComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	SimulatePointerMovement();
}

bool UWidgetInteractionComponent::CanSendInput()
{
	return FSlateApplication::IsInitialized() && VirtualUser.IsValid();
}

void UWidgetInteractionComponent::SetCustomHitResult(const FHitResult& HitResult)
{
	CustomHitResult = HitResult;
}

void UWidgetInteractionComponent::SetFocus(UWidget* FocusWidget)
{
	if (VirtualUser.IsValid())
	{
		FSlateApplication::Get().SetUserFocus(VirtualUser->GetUserIndex(), FocusWidget->GetCachedWidget(), EFocusCause::SetDirectly);
	}
}

#if WITH_EDITOR
void UWidgetInteractionComponent::OnPIEEnded(UGameInstance*)
{
	if(LastWidgetPath.IsValid() && VirtualUser.IsValid())
	{
		TSharedPtr<SWidget> LastVirtualUserWidget = LastWidgetPath.GetLastWidget().Pin();
		int32 UserIndex = VirtualUser->GetUserIndex();
		
		if (LastVirtualUserWidget != nullptr && LastVirtualUserWidget->HasUserFocus(UserIndex))
		{
			FSlateApplication::Get().ClearUserFocus(UserIndex, EFocusCause::SetDirectly);
		}
	}
}
#endif

FWidgetPath UWidgetInteractionComponent::FindHoveredWidgetPath(const FWidgetTraceResult& TraceResult) const
{
	if (TraceResult.HitWidgetComponent)
	{
		return FWidgetPath(TraceResult.HitWidgetComponent->GetHitWidgetPath(TraceResult.LocalHitLocation, /*bIgnoreEnabledStatus*/ false));
	}
	else
	{
		return FWidgetPath();
	}
}

UWidgetInteractionComponent::FWidgetTraceResult UWidgetInteractionComponent::PerformTrace() const
{
	FWidgetTraceResult TraceResult;

	TArray<FHitResult> MultiHits;

	FVector WorldDirection = FVector::ZeroVector;

	switch( InteractionSource )
	{
		case EWidgetInteractionSource::World:
		{
			const FVector WorldLocation = GetComponentLocation();
			const FTransform WorldTransform = GetComponentTransform();
			WorldDirection = WorldTransform.GetUnitAxis(EAxis::X);

			TArray<UPrimitiveComponent*> PrimitiveChildren;
			GetRelatedComponentsToIgnoreInAutomaticHitTesting(PrimitiveChildren);

			FCollisionQueryParams Params(SCENE_QUERY_STAT(WidgetInteractionComponentTrace));
			Params.AddIgnoredComponents(PrimitiveChildren);

			TraceResult.LineStartLocation = WorldLocation;
			TraceResult.LineEndLocation = WorldLocation + (WorldDirection * InteractionDistance);

			GetWorld()->LineTraceMultiByChannel(MultiHits, TraceResult.LineStartLocation, TraceResult.LineEndLocation, TraceChannel, Params);
			break;
		}
		case EWidgetInteractionSource::Mouse:
		case EWidgetInteractionSource::CenterScreen:
		{
			TArray<UPrimitiveComponent*> PrimitiveChildren;
			GetRelatedComponentsToIgnoreInAutomaticHitTesting(PrimitiveChildren);

			FCollisionQueryParams Params(SCENE_QUERY_STAT(WidgetInteractionComponentTrace));
			Params.AddIgnoredComponents(PrimitiveChildren);

			const UWorld* World = GetWorld();
			APlayerController* PlayerController = World ? World->GetFirstPlayerController():nullptr;
			if (!PlayerController)
			{
				UE_LOGF(LogUMG, Warning, "Widget Interaction Component cannot perform trace without a valid PlayerController.");
				return FWidgetTraceResult();
			}
			ULocalPlayer* LocalPlayer = PlayerController->GetLocalPlayer();
			
			if ( LocalPlayer && LocalPlayer->ViewportClient )
			{
				if ( InteractionSource == EWidgetInteractionSource::Mouse )
				{
					FVector2D MousePosition;
					if ( LocalPlayer->ViewportClient->GetMousePosition(MousePosition) )
					{
						FVector WorldOrigin;
						if ( UGameplayStatics::DeprojectScreenToWorld(PlayerController, MousePosition, WorldOrigin, WorldDirection) == true )
						{
							TraceResult.LineStartLocation = WorldOrigin;
							TraceResult.LineEndLocation = WorldOrigin + WorldDirection * InteractionDistance;

							GetWorld()->LineTraceMultiByChannel(MultiHits, TraceResult.LineStartLocation, TraceResult.LineEndLocation, TraceChannel, Params);
						}
					}
				}
				else if ( InteractionSource == EWidgetInteractionSource::CenterScreen )
				{
					FVector2D ViewportSize;
					LocalPlayer->ViewportClient->GetViewportSize(ViewportSize);

					FVector WorldOrigin;
					if ( UGameplayStatics::DeprojectScreenToWorld(PlayerController, ViewportSize * 0.5f, WorldOrigin, WorldDirection) == true )
					{
						TraceResult.LineStartLocation = WorldOrigin;
						TraceResult.LineEndLocation = WorldOrigin + WorldDirection * InteractionDistance;

						GetWorld()->LineTraceMultiByChannel(MultiHits, WorldOrigin, WorldOrigin + WorldDirection * InteractionDistance, TraceChannel, Params);
					}
				}
			}
			break;
		}
		case EWidgetInteractionSource::Custom:
		{
			WorldDirection = (CustomHitResult.TraceEnd - CustomHitResult.TraceStart).GetSafeNormal();
			TraceResult.HitResult = CustomHitResult;
			TraceResult.bWasHit = CustomHitResult.bBlockingHit;
			TraceResult.LineStartLocation = CustomHitResult.TraceStart;
			TraceResult.LineEndLocation = CustomHitResult.TraceEnd;
			break;
		}
	}

	// If it's not a custom interaction, we do some custom filtering to ignore invisible widgets.
	if ( InteractionSource != EWidgetInteractionSource::Custom )
	{
		for ( const FHitResult& HitResult : MultiHits )
		{
			if ( UWidgetComponent* HitWidgetComponent = Cast<UWidgetComponent>(HitResult.GetComponent()) )
			{
				if ( HitWidgetComponent->IsVisible() )
				{
					TraceResult.bWasHit = true;
					TraceResult.HitResult = HitResult;
					break;
				}
			}
			else if (HitResult.bBlockingHit)
			{
				// If we hit something that wasn't a widget component, we're done.
				break;
			}
		}
	}

	// Resolve trace to location on widget.
	if (TraceResult.bWasHit)
	{
		TraceResult.HitWidgetComponent = Cast<UWidgetComponent>(TraceResult.HitResult.GetComponent());
		if (TraceResult.HitWidgetComponent)
		{
			// @todo WASTED WORK: GetLocalHitLocation() gets called in GetHitWidgetPath();

			if (TraceResult.HitWidgetComponent->GetGeometryMode() == EWidgetGeometryMode::Cylinder)
			{				
				TTuple<FVector, FVector2D> CylinderHitLocation = TraceResult.HitWidgetComponent->GetCylinderHitLocation(TraceResult.HitResult.ImpactPoint, WorldDirection);
				TraceResult.HitResult.ImpactPoint = CylinderHitLocation.Get<0>();
				TraceResult.LocalHitLocation = CylinderHitLocation.Get<1>();
			}
			else
			{
				ensure(TraceResult.HitWidgetComponent->GetGeometryMode() == EWidgetGeometryMode::Plane);
				TraceResult.HitWidgetComponent->GetLocalHitLocation(TraceResult.HitResult.ImpactPoint, TraceResult.LocalHitLocation);
			}
			TraceResult.HitWidgetPath = FindHoveredWidgetPath(TraceResult);
		}
	}

	return TraceResult;
}

void UWidgetInteractionComponent::GetRelatedComponentsToIgnoreInAutomaticHitTesting(TArray<UPrimitiveComponent*>& IgnorePrimitives) const
{
	TArray<USceneComponent*> SceneChildren;
	if ( AActor* Owner = GetOwner() )
	{
		if ( USceneComponent* Root = Owner->GetRootComponent() )
		{
			Root = Root->GetAttachmentRoot();
			Root->GetChildrenComponents(true, SceneChildren);
			SceneChildren.Add(Root);
		}
	}

	for ( USceneComponent* SceneComponent : SceneChildren )
	{
		if ( UPrimitiveComponent* PrimtiveComponet = Cast<UPrimitiveComponent>(SceneComponent) )
		{
			// Don't ignore widget components that are siblings.
			if ( SceneComponent->IsA<UWidgetComponent>() )
			{
				continue;
			}

			IgnorePrimitives.Add(PrimtiveComponet);
		}
	}
}

bool UWidgetInteractionComponent::CanInteractWithComponent(UWidgetComponent* Component) const
{
	bool bCanInteract = false;

	if (Component)
	{
		bCanInteract = !GetWorld()->IsPaused() || Component->PrimaryComponentTick.bTickEvenWhenPaused;
	}

	return bCanInteract;
}

FWidgetPath UWidgetInteractionComponent::DetermineWidgetUnderPointer()
{
	FWidgetPath WidgetPathUnderPointer;

	bIsHoveredWidgetInteractable = false;
	bIsHoveredWidgetFocusable = false;
	bIsHoveredWidgetHitTestVisible = false;

	UWidgetComponent* OldHoveredWidgetComponent = GetHoveredWidgetComponent();
	WeakHoveredWidgetComponent.Reset();

	FWidgetTraceResult TraceResult = PerformTrace();
	LastHitResult = TraceResult.HitResult;
	WeakHoveredWidgetComponent = TraceResult.HitWidgetComponent;
	LastLocalHitLocation = LocalHitLocation;
	LocalHitLocation = TraceResult.bWasHit
		? TraceResult.LocalHitLocation
		: LastLocalHitLocation;
	WidgetPathUnderPointer = TraceResult.HitWidgetPath;

	UWidgetComponent* NewHoveredWidgetComponent = GetHoveredWidgetComponent();

#if ENABLE_DRAW_DEBUG
	if ( bShowDebug )
	{
		if ( NewHoveredWidgetComponent )
		{
			UKismetSystemLibrary::DrawDebugSphere(this, LastHitResult.ImpactPoint, 2.5f, 12, DebugColor, 0, DebugSphereLineThickness);
		}

		if ( InteractionSource == EWidgetInteractionSource::World || InteractionSource == EWidgetInteractionSource::Custom )
		{
			if ( NewHoveredWidgetComponent )
			{
				UKismetSystemLibrary::DrawDebugLine(this, LastHitResult.TraceStart, LastHitResult.ImpactPoint, DebugColor, 0, DebugLineThickness);
			}
			else
			{
				UKismetSystemLibrary::DrawDebugLine(this, TraceResult.LineStartLocation, TraceResult.LineEndLocation, DebugColor, 0, DebugLineThickness);
			}
		}
	}
#endif // ENABLE_DRAW_DEBUG

	if ( NewHoveredWidgetComponent )
	{
		NewHoveredWidgetComponent->RequestRedraw();
	}

	if ( WidgetPathUnderPointer.IsValid() )
	{
		const FArrangedChildren::FArrangedWidgetArray& AllArrangedWidgets = WidgetPathUnderPointer.Widgets.GetInternalArray();
		for ( const FArrangedWidget& ArrangedWidget : AllArrangedWidgets )
		{
			const TSharedRef<SWidget>& Widget = ArrangedWidget.Widget;
			if ( Widget->IsEnabled() )
			{
				if ( Widget->IsInteractable() )
				{
					bIsHoveredWidgetInteractable = true;
				}

				if ( Widget->SupportsKeyboardFocus() )
				{
					bIsHoveredWidgetFocusable = true;
				}
			}

			if ( Widget->GetVisibility().IsHitTestVisible() )
			{
				bIsHoveredWidgetHitTestVisible = true;
			}
		}
	}

	if ( NewHoveredWidgetComponent != OldHoveredWidgetComponent )
	{
		if ( OldHoveredWidgetComponent )
		{
			OldHoveredWidgetComponent->RequestRedraw();
		}
		
		if ( NewHoveredWidgetComponent )
		{
			if(VirtualUser.IsValid())
			{
				NewHoveredWidgetComponent->OnWorldUserEntered(VirtualUser->GetUserIndex());
			}
		}

		OnHoveredWidgetChanged.Broadcast( NewHoveredWidgetComponent, OldHoveredWidgetComponent );
	}

	return WidgetPathUnderPointer;
}

void UWidgetInteractionComponent::SimulatePointerMovement()
{
	if ( !bEnableHitTesting )
	{
		return;
	}

	if ( !CanSendInput() )
	{
		return;
	}
	
	FWidgetPath WidgetPathUnderFinger = DetermineWidgetUnderPointer();

	ensure(PointerIndex >= 0);
	FPointerEvent PointerEvent(
		VirtualUser->GetUserIndex(),
		(uint32)PointerIndex,
		LocalHitLocation,
		LastLocalHitLocation,
		PressedKeys,
		FKey(),
		0.0f,
		ModifierKeys);
	
	if (WidgetPathUnderFinger.IsValid())
	{
		check(WeakHoveredWidgetComponent.IsValid());
		LastWidgetPath = WidgetPathUnderFinger;
		
		FSlateApplication::Get().RoutePointerMoveEvent(WidgetPathUnderFinger, PointerEvent, false);
	}
	else
	{
		FWidgetPath EmptyWidgetPath;
		FSlateApplication::Get().RoutePointerMoveEvent(EmptyWidgetPath, PointerEvent, false);

		LastWidgetPath = FWeakWidgetPath();
	}
}

void UWidgetInteractionComponent::PressPointerKey(FKey Key)
{
	if ( !CanSendInput() )
	{
		return;
	}

	if (PressedKeys.Contains(Key))
	{
		return;
	}
	
	PressedKeys.Add(Key);
	
	if ( !LastWidgetPath.IsValid() )
	{
		// If the cached widget path isn't valid, attempt to find a valid widget since we might have received a touch input
		LastWidgetPath = DetermineWidgetUnderPointer();
	}

	FWidgetPath WidgetPathUnderFinger = LastWidgetPath.ToWidgetPath();

	ensure(PointerIndex >= 0);

	FPointerEvent PointerEvent;

	// Find the primary input device for this Slate User
	FInputDeviceId InputDeviceId = INPUTDEVICEID_NONE;
	if (TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(VirtualUser->GetUserIndex()))
	{
		FPlatformUserId PlatUser = SlateUser->GetPlatformUserId();
		InputDeviceId = IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(PlatUser);
	}

	// Just in case there was no input device assigned to this virtual user, get the default platform
	// input device
	if (!InputDeviceId.IsValid())
	{
		InputDeviceId = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
	}
	
	if (Key.IsTouch())
	{
		PointerEvent = FPointerEvent(
			InputDeviceId,
			(uint32)PointerIndex,
			LocalHitLocation,
			LastLocalHitLocation,
			1.0f,
			false,
			false,
			false,
			FModifierKeysState(),
			TOptional(VirtualUser->GetUserIndex()));
	}
	else
	{
		PointerEvent = FPointerEvent(
			InputDeviceId,
			(uint32)PointerIndex,
			LocalHitLocation,
			LastLocalHitLocation,
			PressedKeys,
			Key,
			0.0f,
			ModifierKeys,
			VirtualUser->GetUserIndex());
	}
	
		
	FReply Reply = FSlateApplication::Get().RoutePointerDownEvent(WidgetPathUnderFinger, PointerEvent);
	
	// @TODO Something about double click, expose directly, or automatically do it if key press happens within
	// the double click timeframe?
	//Reply = FSlateApplication::Get().RoutePointerDoubleClickEvent( WidgetPathUnderFinger, PointerEvent );
}

void UWidgetInteractionComponent::ReleasePointerKey(FKey Key)
{
	if ( !CanSendInput() )
	{
		return;
	}

	if (!PressedKeys.Contains(Key))
	{
		return;
	}
	
	PressedKeys.Remove(Key);
	
	FWidgetPath WidgetPathUnderFinger = LastWidgetPath.ToWidgetPath();
	// Need to clear the widget path for cases where the component isn't ticking/clearing itself.
	LastWidgetPath = FWeakWidgetPath();

	ensure(PointerIndex >= 0);
	FPointerEvent PointerEvent;

	// Find the primary input device for this Slate User
	FInputDeviceId InputDeviceId = INPUTDEVICEID_NONE;
	if (TSharedPtr<FSlateUser> SlateUser = FSlateApplication::Get().GetUser(VirtualUser->GetUserIndex()))
	{
		FPlatformUserId PlatUser = SlateUser->GetPlatformUserId();
		InputDeviceId = IPlatformInputDeviceMapper::Get().GetPrimaryInputDeviceForUser(PlatUser);
	}

	// Just in case there was no input device assigned to this virtual user, get the default platform
	// input device
	if (!InputDeviceId.IsValid())
	{
		InputDeviceId = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
	}

	if (Key.IsTouch())
	{
		PointerEvent = FPointerEvent(
			InputDeviceId,
			(uint32)PointerIndex,
			LocalHitLocation,
			LastLocalHitLocation,
			0.0f,
			false,
			false,
			false,
			FModifierKeysState(),
			TOptional(VirtualUser->GetUserIndex()));
	}
	else
	{
		PointerEvent = FPointerEvent(
			InputDeviceId,
			(uint32)PointerIndex,
			LocalHitLocation,
			LastLocalHitLocation,
			PressedKeys,
			Key,
			0.0f,
			ModifierKeys,
			VirtualUser->GetUserIndex());
	}
		
	FReply Reply = FSlateApplication::Get().RoutePointerUpEvent(WidgetPathUnderFinger, PointerEvent);
}

bool UWidgetInteractionComponent::PressKey(FKey Key, bool bRepeat)
{
	if ( !CanSendInput() )
	{
		return false;
	}

	bool bHasKeyCode, bHasCharCode;
	uint32 KeyCode, CharCode;
	GetKeyAndCharCodes(Key, bHasKeyCode, KeyCode, bHasCharCode, CharCode);

	FKeyEvent KeyEvent(Key, ModifierKeys, VirtualUser->GetUserIndex(), bRepeat, CharCode, KeyCode);
	bool bDownResult = FSlateApplication::Get().ProcessKeyDownEvent(KeyEvent);

	bool bKeyCharResult = false;
	if (bHasCharCode)
	{
		if (CharCode <= 0xD7FF || (CharCode >= 0xE000 && CharCode <= 0xFFFF)) // This is a valid UTF16 char from Basic Multilangual Plane
		{
			FCharacterEvent CharacterEvent(static_cast<const TCHAR>(CharCode), ModifierKeys, VirtualUser->GetUserIndex(), bRepeat);
			bKeyCharResult = FSlateApplication::Get().ProcessKeyCharEvent(CharacterEvent);
		}
	}

	return bDownResult || bKeyCharResult;
}

bool UWidgetInteractionComponent::ReleaseKey(FKey Key)
{
	if ( !CanSendInput() )
	{
		return false;
	}

	bool bHasKeyCode, bHasCharCode;
	uint32 KeyCode, CharCode;
	GetKeyAndCharCodes(Key, bHasKeyCode, KeyCode, bHasCharCode, CharCode);

	FKeyEvent KeyEvent(Key, ModifierKeys, VirtualUser->GetUserIndex(), false, CharCode, KeyCode);
	return FSlateApplication::Get().ProcessKeyUpEvent(KeyEvent);
}

void UWidgetInteractionComponent::GetKeyAndCharCodes(const FKey& Key, bool& bHasKeyCode, uint32& KeyCode, bool& bHasCharCode, uint32& CharCode)
{
	const uint32* KeyCodePtr;
	const uint32* CharCodePtr;
	FInputKeyManager::Get().GetCodesFromKey(Key, KeyCodePtr, CharCodePtr);

	bHasKeyCode = KeyCodePtr ? true : false;
	bHasCharCode = CharCodePtr ? true : false;

	KeyCode = KeyCodePtr ? *KeyCodePtr : 0;
	CharCode = CharCodePtr ? *CharCodePtr : 0;

	// These special keys are not handled by the platform layer, and while not printable
	// have character mappings that several widgets look for, since the hardware sends them.
	if (CharCodePtr == nullptr)
	{
		if (Key == EKeys::Tab)
		{
			CharCode = '\t';
			bHasCharCode = true;
		}
		else if (Key == EKeys::BackSpace)
		{
			CharCode = '\b';
			bHasCharCode = true;
		}
		else if (Key == EKeys::Enter)
		{
			CharCode = '\n';
			bHasCharCode = true;
		}
	}
}

bool UWidgetInteractionComponent::PressAndReleaseKey(FKey Key)
{
	const bool PressResult = PressKey(Key, false);
	const bool ReleaseResult = ReleaseKey(Key);

	return PressResult || ReleaseResult;
}

bool UWidgetInteractionComponent::SendKeyChar(FString Characters, bool bRepeat)
{
	if ( !CanSendInput() )
	{
		return false;
	}

	bool bProcessResult = false;

	for ( int32 CharIndex = 0; CharIndex < Characters.Len(); CharIndex++ )
	{
		TCHAR CharKey = Characters[CharIndex];

		FCharacterEvent CharacterEvent(CharKey, ModifierKeys, VirtualUser->GetUserIndex(), bRepeat);
		bProcessResult |= FSlateApplication::Get().ProcessKeyCharEvent(CharacterEvent);
	}

	return bProcessResult;
}

void UWidgetInteractionComponent::ScrollWheel(float ScrollDelta)
{
	if ( !CanSendInput() )
	{
		return;
	}

	FWidgetPath WidgetPathUnderFinger = LastWidgetPath.ToWidgetPath();

	ensure(PointerIndex >= 0);
	FPointerEvent MouseWheelEvent(
		VirtualUser->GetUserIndex(),
		(uint32)PointerIndex,
		LocalHitLocation,
		LastLocalHitLocation,
		PressedKeys,
		EKeys::MouseWheelAxis,
		ScrollDelta,
		ModifierKeys);

	FSlateApplication::Get().RouteMouseWheelOrGestureEvent(WidgetPathUnderFinger, MouseWheelEvent, nullptr);
}

UWidgetComponent* UWidgetInteractionComponent::GetHoveredWidgetComponent() const
{
	return WeakHoveredWidgetComponent.Get();
}

bool UWidgetInteractionComponent::IsOverInteractableWidget() const
{
	return bIsHoveredWidgetInteractable;
}

bool UWidgetInteractionComponent::IsOverFocusableWidget() const
{
	return bIsHoveredWidgetFocusable;
}

bool UWidgetInteractionComponent::IsOverHitTestVisibleWidget() const
{
	return bIsHoveredWidgetHitTestVisible;
}

const FWeakWidgetPath& UWidgetInteractionComponent::GetHoveredWidgetPath() const
{
	return LastWidgetPath;
}

const FHitResult& UWidgetInteractionComponent::GetLastHitResult() const
{
	return LastHitResult;
}

FVector2D UWidgetInteractionComponent::Get2DHitLocation() const
{
	return LocalHitLocation;
}

#undef LOCTEXT_NAMESPACE

