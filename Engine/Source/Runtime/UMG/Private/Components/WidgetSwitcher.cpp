// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WidgetSwitcher.h"
#include "SlateFwd.h"
#include "Components/WidgetSwitcherSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetSwitcher)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UWidgetSwitcher

UWidgetSwitcher::UWidgetSwitcher(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = true;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
}

void UWidgetSwitcher::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyWidgetSwitcher.Reset();
}

int32 UWidgetSwitcher::GetNumWidgets() const
{
	if ( MyWidgetSwitcher.IsValid() )
	{
		return MyWidgetSwitcher->GetNumWidgets();
	}

	return Slots.Num();
}

int32 UWidgetSwitcher::GetActiveWidgetIndex() const
{
	if ( MyWidgetSwitcher.IsValid() )
	{
		return MyWidgetSwitcher->GetActiveWidgetIndex();
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ActiveWidgetIndex;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UWidgetSwitcher::SetActiveWidgetIndex(int32 Index)
{
	if (ActiveWidgetIndex != Index)
	{
		ActiveWidgetIndex = Index;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::ActiveWidgetIndex);
		SetActiveWidgetIndexForSlateWidget();
	}
}


void UWidgetSwitcher::SetActiveWidget(UWidget* Widget)
{
	int32 NewIndex = GetChildIndex(Widget);
	if (ActiveWidgetIndex != NewIndex)
	{
		ActiveWidgetIndex = NewIndex;
		BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::ActiveWidgetIndex);
		SetActiveWidgetIndexForSlateWidget();
	}
}

void UWidgetSwitcher::SetActiveWidgetIndexForSlateWidget()
{
	if (MyWidgetSwitcher.IsValid())
	{
		// Ensure the index is clamped to a valid range.
		int32 SafeIndex = FMath::Clamp(ActiveWidgetIndex, 0, FMath::Max(0, Slots.Num() - 1));
		MyWidgetSwitcher->SetActiveWidgetIndex(SafeIndex);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UWidget* UWidgetSwitcher::GetWidgetAtIndex( int32 Index ) const
{
	if ( Slots.IsValidIndex( Index ) )
	{
		return Slots[ Index ]->Content;
	}

	return nullptr;
}

UWidget* UWidgetSwitcher::GetActiveWidget()const
{
	return GetWidgetAtIndex(GetActiveWidgetIndex());
}

UClass* UWidgetSwitcher::GetSlotClass() const
{
	return UWidgetSwitcherSlot::StaticClass();
}

void UWidgetSwitcher::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyWidgetSwitcher.IsValid() )
	{
		CastChecked<UWidgetSwitcherSlot>(InSlot)->BuildSlot(MyWidgetSwitcher.ToSharedRef());

		// Adding the slot might have changed the active widget index.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ActiveWidgetIndex = MyWidgetSwitcher->GetActiveWidgetIndex();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UWidgetSwitcher::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyWidgetSwitcher.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyWidgetSwitcher->RemoveSlot(Widget.ToSharedRef());

			// Removing the slot might have changed the active widget index.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ActiveWidgetIndex = MyWidgetSwitcher->GetActiveWidgetIndex();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

TSharedRef<SWidget> UWidgetSwitcher::RebuildWidget()
{
	MyWidgetSwitcher = SNew(SWidgetSwitcher);

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UWidgetSwitcherSlot* TypedSlot = Cast<UWidgetSwitcherSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyWidgetSwitcher.ToSharedRef());
		}
	}

	return MyWidgetSwitcher.ToSharedRef();
}

void UWidgetSwitcher::SynchronizeProperties()
{
	Super::SynchronizeProperties();
	SetActiveWidgetIndexForSlateWidget();
}

#if WITH_EDITOR

const FText UWidgetSwitcher::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

void UWidgetSwitcher::OnDescendantSelectedByDesigner(UWidget* DescendantWidget)
{
	// Temporarily sets the active child to the selected child to make
	// dragging and dropping easier in the editor.
	UWidget* SelectedChild = UWidget::FindChildContainingDescendant(this, DescendantWidget);
	if ( SelectedChild )
	{
		int32 OverrideIndex = GetChildIndex(SelectedChild);
		if ( OverrideIndex != -1 && MyWidgetSwitcher.IsValid() )
		{
			MyWidgetSwitcher->SetActiveWidgetIndex(OverrideIndex);
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UWidgetSwitcher::OnDescendantDeselectedByDesigner(UWidget* DescendantWidget)
{
	SetActiveWidgetIndex(ActiveWidgetIndex);
}

void UWidgetSwitcher::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	ActiveWidgetIndex = FMath::Clamp(ActiveWidgetIndex, 0, FMath::Max(0, Slots.Num() - 1));

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

