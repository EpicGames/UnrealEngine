// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WidgetSwitcherSlot.h"
#include "SlateFwd.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetSwitcherSlot)

/////////////////////////////////////////////////////
// UWidgetSwitcherSlot

UWidgetSwitcherSlot::UWidgetSwitcherSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWidgetSwitcherSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UWidgetSwitcherSlot::BuildSlot(TSharedRef<SWidgetSwitcher> WidgetSwitcher)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WidgetSwitcher->AddSlot()
		.Expose(Slot)
		.Padding(Padding)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWidgetSwitcherSlot::SetContent(UWidget* NewContent)
{
	Content = NewContent;
	if (Slot)
	{
		Slot->AttachWidget(NewContent ? NewContent->TakeWidget() : SNullWidget::NullWidget);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UWidgetSwitcherSlot::GetPadding() const
{
	return Padding;
}

void UWidgetSwitcherSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Slot )
	{
		Slot->SetPadding(InPadding);
	}
}

EHorizontalAlignment UWidgetSwitcherSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UWidgetSwitcherSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UWidgetSwitcherSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UWidgetSwitcherSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Slot )
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UWidgetSwitcherSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetPadding(Padding);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

