// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WindowTitleBarArea.h"
#include "EngineGlobals.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SWindowTitleBarArea.h"
#include "Components/WindowTitleBarAreaSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WindowTitleBarArea)

namespace UE::UMG::Private
{
	static TAutoConsoleVariable<bool> CVarForceWindowButtonsHidden(
		TEXT("WindowTitleBar.ForceWindowButtonsHidden"),
		false,
		TEXT("If true, force the window title bar buttons to be hidden."),
		ECVF_Default);
}

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UWindowTitleBarArea

UWindowTitleBarArea::UWindowTitleBarArea(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::Visible);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bDoubleClickTogglesFullscreen = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWindowTitleBarArea::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyWindowTitleBarArea.Reset();

	if (WindowActionNotificationHandle.IsValid())
	{
		FSlateApplication::Get().UnregisterOnWindowActionNotification(WindowActionNotificationHandle);
		WindowActionNotificationHandle.Reset();
	}
}

TSharedRef<SWidget> UWindowTitleBarArea::RebuildWidget()
{
	MyWindowTitleBarArea = SNew(SWindowTitleBarArea);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bDoubleClickTogglesFullscreen)
	{
		WindowActionNotificationHandle = FSlateApplication::Get().RegisterOnWindowActionNotification(BIND_UOBJECT_DELEGATE(FOnWindowAction, HandleWindowAction));
	}
	else if (WindowActionNotificationHandle.IsValid())
	{
		FSlateApplication::Get().UnregisterOnWindowActionNotification(WindowActionNotificationHandle);
		WindowActionNotificationHandle.Reset();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	MyWindowTitleBarArea->SetRequestToggleFullscreenCallback(BIND_UOBJECT_DELEGATE(FSimpleDelegate, RequestToggleFullscreen));

	if (GetChildrenCount() > 0)
	{
		Cast<UWindowTitleBarAreaSlot>(GetContentSlot())->BuildSlot(MyWindowTitleBarArea.ToSharedRef());
	}

	if (GEngine && GEngine->GameViewport && GEngine->GameViewport->GetWindow().IsValid())
	{
		MyWindowTitleBarArea->SetGameWindow(GEngine->GameViewport->GetWindow());
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyWindowTitleBarArea->SetWindowButtonsVisibility(UE::UMG::Private::CVarForceWindowButtonsHidden.GetValueOnAnyThread() ? false : bWindowButtonsEnabled);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return MyWindowTitleBarArea.ToSharedRef();
}

UClass* UWindowTitleBarArea::GetSlotClass() const
{
	return UWindowTitleBarAreaSlot::StaticClass();
}

void UWindowTitleBarArea::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live slot if it already exists
	if (MyWindowTitleBarArea.IsValid())
	{
		// Construct the underlying slot.
		UWindowTitleBarAreaSlot* WindowTitleBarAreaSlot = CastChecked<UWindowTitleBarAreaSlot>(InSlot);
		WindowTitleBarAreaSlot->BuildSlot(MyWindowTitleBarArea.ToSharedRef());
	}
}

void UWindowTitleBarArea::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if (MyWindowTitleBarArea.IsValid())
	{
		MyWindowTitleBarArea->SetContent(SNullWidget::NullWidget);
	}
}

void UWindowTitleBarArea::SetPadding(FMargin InPadding)
{
	if (MyWindowTitleBarArea.IsValid())
	{
		MyWindowTitleBarArea->SetPadding(InPadding);
	}
}

void UWindowTitleBarArea::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	if (MyWindowTitleBarArea.IsValid())
	{
		MyWindowTitleBarArea->SetHAlign(InHorizontalAlignment);
	}
}

void UWindowTitleBarArea::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	if (MyWindowTitleBarArea.IsValid())
	{
		MyWindowTitleBarArea->SetVAlign(InVerticalAlignment);
	}
}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UWindowTitleBarArea::SetWindowButtonsEnabled(bool InWindowButtonsEnabled)
{
	bWindowButtonsEnabled = InWindowButtonsEnabled;
	
	if (MyWindowTitleBarArea.IsValid())
	{
		MyWindowTitleBarArea->SetWindowButtonsVisibility(bWindowButtonsEnabled);
	}
}

bool UWindowTitleBarArea::IsWindowButtonsEnabled() const
{
	return bWindowButtonsEnabled;
}

void UWindowTitleBarArea::SetDoubleClickTogglesFullscreen(bool InDoubleClickTogglesFullscreen)
{
	bDoubleClickTogglesFullscreen = InDoubleClickTogglesFullscreen;

	if (bDoubleClickTogglesFullscreen)
	{
		WindowActionNotificationHandle = FSlateApplication::Get().RegisterOnWindowActionNotification(BIND_UOBJECT_DELEGATE(FOnWindowAction, HandleWindowAction));
	}
	else if (WindowActionNotificationHandle.IsValid())
	{
		FSlateApplication::Get().UnregisterOnWindowActionNotification(WindowActionNotificationHandle);
		WindowActionNotificationHandle.Reset();
	}
}

bool UWindowTitleBarArea::IsDoubleClickTogglesFullscreen() const
{
	return bDoubleClickTogglesFullscreen;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UWindowTitleBarArea::PostLoad()
{
	Super::PostLoad();

	if (GetChildrenCount() > 0)
	{
		//TODO UMG Pre-Release Upgrade, now have slots of their own.  Convert existing slot to new slot.
		if (UPanelSlot* PanelSlot = GetContentSlot())
		{
			UWindowTitleBarAreaSlot* WindowTitleBarAreaSlot = Cast<UWindowTitleBarAreaSlot>(PanelSlot);
			if (WindowTitleBarAreaSlot == NULL)
			{
				WindowTitleBarAreaSlot = NewObject<UWindowTitleBarAreaSlot>(this);
				WindowTitleBarAreaSlot->Content = GetContentSlot()->Content;
				WindowTitleBarAreaSlot->Content->Slot = WindowTitleBarAreaSlot;
				Slots[0] = WindowTitleBarAreaSlot;
			}
		}
	}
}

#if WITH_EDITOR
const FText UWindowTitleBarArea::GetPaletteCategory()
{
	return LOCTEXT("Advanced", "Advanced");
}
#endif

bool UWindowTitleBarArea::HandleWindowAction(const TSharedRef<FGenericWindow>& PlatformWindow, EWindowAction::Type WindowAction)
{
	if (GEngine && (WindowAction == EWindowAction::Maximize || WindowAction == EWindowAction::Restore))
	{
		GEngine->DeferredCommands.Add(TEXT("TOGGLE_FULLSCREEN"));
		return true;
	}
	return false;
}

void UWindowTitleBarArea::RequestToggleFullscreen()
{
	// This is only called in fullscreen mode when user double clicks the title bar or clicks the restore button.
	if (GEngine)
	{
		GEngine->DeferredCommands.Add(TEXT("TOGGLE_FULLSCREEN"));
	}
}


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

