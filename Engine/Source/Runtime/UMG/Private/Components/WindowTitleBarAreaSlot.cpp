// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WindowTitleBarAreaSlot.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Layout/SWindowTitleBarArea.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WindowTitleBarAreaSlot)

/////////////////////////////////////////////////////
// UWindowTitleBarAreaSlot

UWindowTitleBarAreaSlot::UWindowTitleBarAreaSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Padding = FMargin(0.f);

	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWindowTitleBarAreaSlot::BuildSlot(TSharedRef<SWindowTitleBarArea> InWindowTitleBarArea)
{
	WindowTitleBarArea = InWindowTitleBarArea;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WindowTitleBarArea->SetPadding(Padding);
	WindowTitleBarArea->SetHAlign(HorizontalAlignment);
	WindowTitleBarArea->SetVAlign(VerticalAlignment);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	WindowTitleBarArea->SetContent(Content ? Content->TakeWidget() : SNullWidget::NullWidget);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UWindowTitleBarAreaSlot::GetPadding() const
{
	return Padding;
}

void UWindowTitleBarAreaSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	CastChecked<UWindowTitleBarArea>(Parent)->SetPadding(InPadding);
}

EHorizontalAlignment UWindowTitleBarAreaSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UWindowTitleBarAreaSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	CastChecked<UWindowTitleBarArea>(Parent)->SetHorizontalAlignment(InHorizontalAlignment);
}

EVerticalAlignment UWindowTitleBarAreaSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UWindowTitleBarAreaSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	CastChecked<UWindowTitleBarArea>(Parent)->SetVerticalAlignment(InVerticalAlignment);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UWindowTitleBarAreaSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (WindowTitleBarArea.IsValid())
	{
		SetPadding(Padding);
		SetHorizontalAlignment(HorizontalAlignment);
		SetVerticalAlignment(VerticalAlignment);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWindowTitleBarAreaSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	WindowTitleBarArea.Reset();
}

