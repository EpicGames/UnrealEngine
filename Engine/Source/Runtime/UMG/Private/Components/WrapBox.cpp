// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WrapBox.h"
#include "Components/WrapBoxSlot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WrapBox)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UWrapBox

UWrapBox::UWrapBox(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bIsVariable = false;
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);
	WrapSize = 500;
	bExplicitWrapSize = false;
	HorizontalAlignment = HAlign_Left;
	Orientation = EOrientation::Orient_Horizontal;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWrapBox::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyWrapBox.Reset();
}

UClass* UWrapBox::GetSlotClass() const
{
	return UWrapBoxSlot::StaticClass();
}

void UWrapBox::OnSlotAdded(UPanelSlot* InSlot)
{
	// Add the child to the live canvas if it already exists
	if ( MyWrapBox.IsValid() )
	{
		CastChecked<UWrapBoxSlot>(InSlot)->BuildSlot(MyWrapBox.ToSharedRef());
	}
}

void UWrapBox::OnSlotRemoved(UPanelSlot* InSlot)
{
	// Remove the widget from the live slot if it exists.
	if ( MyWrapBox.IsValid() && InSlot->Content)
	{
		TSharedPtr<SWidget> Widget = InSlot->Content->GetCachedWidget();
		if ( Widget.IsValid() )
		{
			MyWrapBox->RemoveSlot(Widget.ToSharedRef());
		}
	}
}

UWrapBoxSlot* UWrapBox::AddChildToWrapBox(UWidget* Content)
{
	return Cast<UWrapBoxSlot>(Super::AddChild(Content));
}

TSharedRef<SWidget> UWrapBox::RebuildWidget()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MyWrapBox = SNew(SWrapBox)
		.UseAllottedSize(!bExplicitWrapSize)
		.PreferredSize(WrapSize)
		.HAlign(HorizontalAlignment)
		.Orientation(Orientation);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for ( UPanelSlot* PanelSlot : Slots )
	{
		if ( UWrapBoxSlot* TypedSlot = Cast<UWrapBoxSlot>(PanelSlot) )
		{
			TypedSlot->Parent = this;
			TypedSlot->BuildSlot(MyWrapBox.ToSharedRef());
		}
	}

	return MyWrapBox.ToSharedRef();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UWrapBox::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyWrapBox.IsValid())
	{
		return;
	}

	MyWrapBox->SetInnerSlotPadding(InnerSlotPadding);
	MyWrapBox->SetUseAllottedSize(!bExplicitWrapSize);
	MyWrapBox->SetWrapSize(WrapSize);
	MyWrapBox->SetHorizontalAlignment(HorizontalAlignment);
	MyWrapBox->SetOrientation(Orientation);
}

FVector2D UWrapBox::GetInnerSlotPadding() const
{
	return InnerSlotPadding;
}

void UWrapBox::SetInnerSlotPadding(FVector2D InPadding)
{
	InnerSlotPadding = InPadding;
	if (MyWrapBox.IsValid())
	{
		MyWrapBox->SetInnerSlotPadding(InPadding);
	}
}

float UWrapBox::GetWrapSize() const
{
	return WrapSize;
}

void UWrapBox::SetWrapSize(float InWrapSize)
{
	WrapSize = InWrapSize;
	if (MyWrapBox.IsValid())
	{
		MyWrapBox->SetWrapSize(InWrapSize);
	}
}

bool UWrapBox::UseExplicitWrapSize() const
{
	return bExplicitWrapSize;
}

void UWrapBox::SetExplicitWrapSize(bool bInExplicitWrapSize)
{
	bExplicitWrapSize = bInExplicitWrapSize;
	if (MyWrapBox.IsValid())
	{
		MyWrapBox->SetUseAllottedSize(!bExplicitWrapSize);
	}
}

EHorizontalAlignment UWrapBox::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UWrapBox::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if (MyWrapBox)
	{
		MyWrapBox->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EOrientation UWrapBox::GetOrientation() const
{
	return Orientation;
}

void UWrapBox::SetOrientation(EOrientation InOrientation)
{
	Orientation = InOrientation;
	if (MyWrapBox)
	{
		MyWrapBox->SetOrientation(InOrientation);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR

const FText UWrapBox::GetPaletteCategory()
{
	return LOCTEXT("Panel", "Panel");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

