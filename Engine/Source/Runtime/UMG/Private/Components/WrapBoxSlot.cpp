// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/WrapBoxSlot.h"
#include "Components/Widget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WrapBoxSlot)

/////////////////////////////////////////////////////
// UWrapBoxSlot

UWrapBoxSlot::UWrapBoxSlot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Slot(nullptr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	HorizontalAlignment = HAlign_Fill;
	VerticalAlignment = VAlign_Fill;

	bFillEmptySpace = false;
	FillSpanWhenLessThan = 0;
	bForceNewLine = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UWrapBoxSlot::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	Slot = nullptr;
}

void UWrapBoxSlot::BuildSlot(TSharedRef<SWrapBox> WrapBox)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	WrapBox->AddSlot()
		.Padding(Padding)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		.FillEmptySpace(bFillEmptySpace)
		.FillLineWhenSizeLessThan(FillSpanWhenLessThan == 0 ? TOptional<float>() : TOptional<float>(FillSpanWhenLessThan))
		.ForceNewLine(bForceNewLine)
		.Expose(Slot)
		[
			Content == nullptr ? SNullWidget::NullWidget : Content->TakeWidget()
		];
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FMargin UWrapBoxSlot::GetPadding() const
{
	return Padding;
}

void UWrapBoxSlot::SetPadding(FMargin InPadding)
{
	Padding = InPadding;
	if ( Slot )
	{
		Slot->SetPadding(InPadding);
	}
}

bool UWrapBoxSlot::DoesFillEmptySpace() const
{
	return bFillEmptySpace;
}

void UWrapBoxSlot::SetFillEmptySpace(bool InbFillEmptySpace)
{
	bFillEmptySpace = InbFillEmptySpace;
	if ( Slot )
	{
		Slot->SetFillEmptySpace(InbFillEmptySpace);
	}
}

float UWrapBoxSlot::GetFillSpanWhenLessThan() const
{
	return FillSpanWhenLessThan;
}

void UWrapBoxSlot::SetFillSpanWhenLessThan(float InFillSpanWhenLessThan)
{
	FillSpanWhenLessThan = InFillSpanWhenLessThan;
	if ( Slot )
	{
		Slot->SetFillLineWhenSizeLessThan(InFillSpanWhenLessThan == 0 ? TOptional<float>() : TOptional<float>(InFillSpanWhenLessThan));
	}
}

EHorizontalAlignment UWrapBoxSlot::GetHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void UWrapBoxSlot::SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment)
{
	HorizontalAlignment = InHorizontalAlignment;
	if ( Slot )
	{
		Slot->SetHorizontalAlignment(InHorizontalAlignment);
	}
}

EVerticalAlignment UWrapBoxSlot::GetVerticalAlignment() const
{
	return VerticalAlignment;
}

void UWrapBoxSlot::SetVerticalAlignment(EVerticalAlignment InVerticalAlignment)
{
	VerticalAlignment = InVerticalAlignment;
	if ( Slot )
	{
		Slot->SetVerticalAlignment(InVerticalAlignment);
	}
}

bool UWrapBoxSlot::DoesForceNewLine() const
{
	return bForceNewLine;
}

void UWrapBoxSlot::SetNewLine(bool InForceNewLine)
{
	bForceNewLine = InForceNewLine;
	if (Slot)
	{
		Slot->SetForceNewLine(InForceNewLine);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UWrapBoxSlot::SynchronizeProperties()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetPadding(Padding);
	SetFillEmptySpace(bFillEmptySpace);
	SetFillSpanWhenLessThan(FillSpanWhenLessThan);
	SetHorizontalAlignment(HorizontalAlignment);
	SetVerticalAlignment(VerticalAlignment);
	SetNewLine(bForceNewLine);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

