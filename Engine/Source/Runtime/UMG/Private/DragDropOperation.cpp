// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/DragDropOperation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DragDropOperation)

/////////////////////////////////////////////////////
// UDragDropOperation

UDragDropOperation::UDragDropOperation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Pivot = EDragPivot::CenterCenter;
}

EUMGItemDropZone UDragDropOperation::ConvertSlateDropZoneToUMG(TOptional<EItemDropZone> DropZone)
{
	if (DropZone != NullOpt)
	{
		switch (DropZone.GetValue())
		{
		case EItemDropZone::AboveItem:
			return EUMGItemDropZone::AboveItem;
		case EItemDropZone::OntoItem:
			return EUMGItemDropZone::OntoItem;
		case EItemDropZone::BelowItem:
			return EUMGItemDropZone::BelowItem;
		default:
			return EUMGItemDropZone::None;
		}
	}
	else
	{
		return EUMGItemDropZone::None;
	}
}

/// @cond DOXYGEN_WARNINGS

void UDragDropOperation::Drop_Implementation(const FPointerEvent& PointerEvent)
{
	OnDrop.Broadcast(this);
}

void UDragDropOperation::DragCancelled_Implementation(const FPointerEvent& PointerEvent)
{
	OnDragCancelled.Broadcast(this);
}

void UDragDropOperation::Dragged_Implementation(const FPointerEvent& PointerEvent)
{
	OnDragged.Broadcast(this);
}

/// @endcond

