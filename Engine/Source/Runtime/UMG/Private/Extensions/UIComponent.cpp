// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/UIComponent.h"

#include "CoreGlobals.h"
#include "Components/Widget.h"
#include "Blueprint/UserWidget.h"
#include "Engine/BlueprintGeneratedClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIComponent)

void UUIComponent::Initialize(UWidget* Target)
{
	ensure(!bInitialized);
	ensure(!Target->IsConstructed());
	ensure(Target);
	ensure(Owner.IsExplicitlyNull());
	Owner = Target;
	Owner->bWrappedByComponent = true;
	OnInitialize();
	bInitialized = true;
}

void UUIComponent::PreConstruct(bool bIsDesignTime)
{
	ensure(bInitialized);
	OnPreConstruct(bIsDesignTime);
}


void UUIComponent::Construct()
{
	OnConstruct();
}

void UUIComponent::Destruct()
{
	OnDestruct();
}

void UUIComponent::GraphRenamed(UEdGraph* Graph, const FName& OldName, const FName& NewName)
{
	OnGraphRenamed(Graph, OldName, NewName);
}

TWeakObjectPtr<UWidget> UUIComponent::GetOwner() const
{
	return Owner;
}

void UUIComponent::OnInitialize()
{

}

void UUIComponent::OnPreConstruct(bool bIsDesignTime)
{

}

void UUIComponent::OnConstruct()
{

}

void UUIComponent::OnDestruct()
{

}

void UUIComponent::OnGraphRenamed(UEdGraph* Graph, const FName& OldName, const FName& NewName)
{

}

TSharedRef<SWidget> UUIComponent::RebuildWidgetWithContent(TSharedRef<SWidget> Content)
{
	return Content;
}

void UUIComponent::FFieldNotificationClassDescriptor::ForEachField(const UClass* Class, TFunctionRef<bool(::UE::FieldNotification::FFieldId FieldId)> Callback) const
{
	if (const UBlueprintGeneratedClass* BPClass = Cast<const UBlueprintGeneratedClass>(Class))
	{
		BPClass->ForEachFieldNotify(Callback, true);
	}
}

FDelegateHandle UUIComponent::AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate)
{
	return (InFieldId.IsValid()) ? Delegates.Add(this, InFieldId, MoveTemp(InNewDelegate)) : FDelegateHandle();
}

bool UUIComponent::RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle)
{
	if (InFieldId.IsValid() && InHandle.IsValid())
	{
		return Delegates.RemoveFrom(this, InFieldId, InHandle).bRemoved;
	}
	return false;
}

int32 UUIComponent::RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject)
{
	if (InUserObject)
	{
		return Delegates.RemoveAll(this, InUserObject).RemoveCount;
	}
	return false;
}

int32 UUIComponent::RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject)
{
	if (InFieldId.IsValid() && InUserObject)
	{
		return Delegates.RemoveAll(this, InFieldId, InUserObject).RemoveCount;
	}
	return false;
}

void UUIComponent::BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId)
{
	if (InFieldId.IsValid())
	{
		Delegates.Broadcast(this, InFieldId);
	}
}

const UE::FieldNotification::IClassDescriptor& UUIComponent::GetFieldNotificationDescriptor() const
{
	static FFieldNotificationClassDescriptor Local;
	return Local;
}

#if WITH_EDITOR
EDataValidationResult UUIComponent::IsDataValidForOwner(const UWidget* OwnerWidget, FDataValidationContext& Context) const
{
	return IsDataValid(Context);
}
#endif // WITH_EDITOR
