// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/UIComponentContainer.h"

#include "Blueprint/WidgetTree.h"
#include "CoreMinimal.h"
#include "Components/Widget.h"
#include "Extensions/UIComponent.h"
#include "Slate/SObjectWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIComponentContainer)

FUIComponentTarget::FUIComponentTarget()
{}

FUIComponentTarget::FUIComponentTarget(UUIComponent* InComponent, FName InChildName)
{
	TargetName = InChildName;
	Component = InComponent;
}

UWidget* FUIComponentTarget::Resolve(const UWidgetTree* WidgetTree)
{
	if (!TargetName.IsNone() && WidgetTree)
	{
		return WidgetTree->FindWidget(TargetName);
	}

	return nullptr;
}

void FUIComponentTarget::SetTargetName(FName NewName)
{
	TargetName = NewName;
}

void UUIComponentContainer::AddComponent(FName TargetName, UUIComponent* Component)
{	
	if (ensure(!TargetName.IsNone() && Component != nullptr))
	{
		if(!GetComponent(Component->GetClass(), TargetName))
		{
			Modify();
			Components.Emplace(FUIComponentTarget(Component, TargetName));
		}
	}
}

void UUIComponentContainer::RemoveComponent(FName TargetName, UUIComponent* Component)
{
	bool bModified = false;
	auto UpdateModify = [&bModified, Self = this]()
	{
		if (!bModified)
		{
			bModified = true;
			Self->Modify();
		}
	};
	
	if(ensure(!TargetName.IsNone() && Component != nullptr))
	{
		for (int i = 0; i < Components.Num(); i++)
		{
			FUIComponentTarget& Target = Components[i];
			if (Target.GetComponent() == Component)
			{
				ensure(Target.GetTargetName() == TargetName);
				UpdateModify();
				Components.RemoveAt(i);
				return;
			}
		}
	}
}

void UUIComponentContainer::RemoveAllComponentsOfType(const UClass* ComponentClass, FName TargetName)
{
	ensure(!TargetName.IsNone());

	bool bModified = false;
	auto UpdateModify = [&bModified, Self = this]()
	{
		if (!bModified)
		{
			bModified = true;
			Self->Modify();
		}
	};
	
	for (int32 Index = Components.Num() - 1; Index >= 0; --Index)
	{
		FUIComponentTarget& Target = Components[Index];
		if (Target.GetComponent()->GetClass() == ComponentClass && Target.GetTargetName() == TargetName)
		{
			UpdateModify();
			Components.RemoveAt(Index);
		}
	}
}

void UUIComponentContainer::RemoveAllComponentsFor(FName TargetName)
{
	bool bModified = false;
	auto UpdateModify = [&bModified, Self = this]()
	{
		if (!bModified)
		{
			bModified = true;
			Self->Modify();
		}
	};

	for (int32 Index = Components.Num() - 1; Index >= 0; --Index)
	{
		FUIComponentTarget& Target = Components[Index];
		if (Target.GetTargetName() == TargetName)
		{
			UpdateModify();
			Components.RemoveAt(Index);
		}
	}
}

void UUIComponentContainer::MoveComponent(FName TargetName, const UClass* ComponentClassToMove, const UClass* RelativeToComponentClass, bool bMoveAfter)
{
	if (!ensure(!TargetName.IsNone() && ComponentClassToMove != nullptr && ComponentClassToMove != RelativeToComponentClass))
	{
		return;
	}

	FUIComponentTarget TargetToMove;
	for (int i = 0; i < Components.Num(); i++)
	{
		FUIComponentTarget& Target = Components[i];
		if (Target.GetTargetName() == TargetName &&
			Target.GetComponent()->GetClass() == ComponentClassToMove)
		{
			TargetToMove = Target;
											
			Modify();
			Components.RemoveAt(i, EAllowShrinking::No);
			break;			
		}
	}
	// If we didn't find the Component to move we are done
	if (!TargetToMove.IsValid())
	{
		return;
	}

	int IndexToAddTo = 0;
	if (RelativeToComponentClass)
	{
		for (int i = 0; i < Components.Num(); i++)
		{
			FUIComponentTarget& Target = Components[i];
			if (Target.GetTargetName() == TargetName &&
				Target.GetComponent()->GetClass() == RelativeToComponentClass)
			{
				IndexToAddTo = bMoveAfter ? (i+1) : i;				
				break;
			}
		}
	}
	Components.Insert(TargetToMove, IndexToAddTo);	
}

UUIComponent* UUIComponentContainer::GetComponent(const UClass* ComponentClass, FName TargetName) const
{
	for (int32 Index = Components.Num() - 1; Index >= 0; --Index)
	{
		const FUIComponentTarget& Target = Components[Index];
		if (Target.GetTargetName() == TargetName && Target.GetComponent()->GetClass()->IsChildOf(ComponentClass))
		{
			return Target.GetComponent();
		}
	}
	return nullptr;
}

void UUIComponentContainer::ForEachComponent(TFunctionRef<void(UUIComponent*)> Predicate) const
{
	for (const FUIComponentTarget& ComponentTarget : Components)
	{
		if (UUIComponent* Component = ComponentTarget.GetComponent())
		{
			Predicate(Component);
		}
	}
}

void UUIComponentContainer::ForEachComponentTarget(TFunctionRef<void(const FUIComponentTarget&)> Predicate) const
{
	for (const FUIComponentTarget& ComponentTarget : Components)
	{
		Predicate(ComponentTarget);
	}
}

void UUIComponentContainer::InitializeComponents(const UUserWidget* UserWidget)
{
	for (FUIComponentTarget& ComponentTarget : Components)
	{
		if (UUIComponent* Component = ComponentTarget.GetComponent())
		{
			UWidget* Widget = ComponentTarget.Resolve(UserWidget->WidgetTree);
			if (ensure(Widget))
			{
				Component->Initialize(Widget);
			}
		}
	}
}

bool UUIComponentContainer::IsEmpty() const
{
	return Components.IsEmpty();
}

FName UUIComponentContainer::GetPropertyNameForComponent(const UUIComponent* Component, const FName& TargetName)
{
	return FName(Component->GetName() + "_" + TargetName.ToString());
}

#ifdef WITH_EDITOR
void UUIComponentContainer::RenameWidget(const FName& OldName, const FName& NewName)
{
	bool bModified = false;
	auto UpdateModify = [&bModified, Self = this]()
		{
			if (!bModified)
			{
				bModified = true;
				Self->Modify();
			}
		};

	for (FUIComponentTarget& Target : Components)
	{
		if (Target.GetTargetName() == OldName)
		{
			UpdateModify();
			Target.SetTargetName(NewName);
		}
	}
}
void UUIComponentContainer::CleanupUIComponents(const UWidgetTree* WidgetTree)
{
	if (WidgetTree)
	{
		bool bModified = false;
		auto UpdateModify = [&bModified, Self = this]()
		{
			if (!bModified)
			{
				bModified = true;
				Self->Modify();
			}
		};
		
		TArray <FName, TInlineAllocator<4>> TargetNamesToRemove;
		for (int32 Index = Components.Num() - 1; Index >= 0; --Index)
		{
			// Remove components that are invalid
			if (Components[Index].GetComponent() == nullptr)
			{
				UpdateModify();
				Components.RemoveAt(Index);
			}
			else if (!Components[Index].GetTargetName().IsNone())
			{
				// Fill list with all Valid Target Names
				TargetNamesToRemove.AddUnique(Components[Index].GetTargetName());
			}
		}

		// Exclude components in use from removal list
		if (TargetNamesToRemove.Num() > 0)
		{
			WidgetTree->ForEachWidget([&TargetNamesToRemove, this](TObjectPtr<UWidget> Widget) {
				if (Widget)
				{
					for (int32 Index = TargetNamesToRemove.Num() - 1; Index >= 0; Index--)
					{
						const FName TargetName = TargetNamesToRemove[Index];
						if (TargetName == Widget->GetFName())
						{
							TargetNamesToRemove.RemoveSingleSwap(TargetName);
						}
					}
				}
				});
		}

		// Remove all unused component 
		if (TargetNamesToRemove.Num() > 0)
		{
			UpdateModify();

			for (FName TargetName : TargetNamesToRemove)
			{
				RemoveAllComponentsFor(TargetName);
			}
		}
	}
}

#endif //WITH_EDITOR
