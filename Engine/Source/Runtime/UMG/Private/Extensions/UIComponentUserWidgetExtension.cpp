// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/UIComponentUserWidgetExtension.h"

#include "Blueprint/UserWidget.h"
#include "CoreMinimal.h"
#include "Blueprint/WidgetTree.h"
#include "Components/Widget.h"
#include "Extensions/UIComponent.h"
#include "Extensions/UIComponentContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIComponentUserWidgetExtension)

void UUIComponentUserWidgetExtension::Initialize()
{
	ensure(!bInitialized);
	InitializeComponents();
	bInitialized = true;
}

void UUIComponentUserWidgetExtension::PreConstruct(bool bIsDesignTime)
{
	ensure(bInitialized);
	Super::PreConstruct(bIsDesignTime);

	if (ComponentContainer)
	{
		ComponentContainer->ForEachComponent([bIsDesignTime](UUIComponent* Component)
			{
				if(ensure(Component))
				{
					Component->PreConstruct(bIsDesignTime);
				}
			}
		);
	}
}
void UUIComponentUserWidgetExtension::Construct()
{
	Super::Construct();
	if (ComponentContainer)
	{
		ComponentContainer->ForEachComponent([](UUIComponent* Component)
			{
				if (ensure(Component))
				{
					Component->Construct();
				}
			}
		);
	}
}

void UUIComponentUserWidgetExtension::Destruct()
{	
	Super::Destruct();
	if (ComponentContainer)
	{
		ComponentContainer->ForEachComponent([](UUIComponent* Component)
			{
				if (ensure(Component))
				{
					Component->Destruct();
				}
			}
		);
	}
}

void UUIComponentUserWidgetExtension::InitializeContainer(UUIComponentContainer* InComponentContainer)
{
	check(InComponentContainer != nullptr);
	check(ComponentContainer == nullptr);
	ComponentContainer = InComponentContainer;
}

bool UUIComponentUserWidgetExtension::IsContainerInitialized() const
{
	return ComponentContainer != nullptr;
}

void UUIComponentUserWidgetExtension::InitializeComponents()
{
	if (ComponentContainer)
	{
		const UUserWidget* UserWidget = GetUserWidget();
		check(UserWidget);
		ComponentContainer->InitializeComponents(UserWidget);
	}
}

#if WITH_EDITOR

void UUIComponentUserWidgetExtension::InitializeInEditor()
{
	CleanupComponents();
	// We simply call the Initialize function as we need the same code to execute in Editor.
	Initialize();
}

void UUIComponentUserWidgetExtension::RenameWidget(const FName& OldVarName, const FName& NewVarName)
{
	if (ensure(ComponentContainer))
	{
		ComponentContainer->RenameWidget(OldVarName, NewVarName);
	}
}

// Used only to create a Component on the PreviewWidget in the editor, based on the Component Archetype object in the WidgetBlueprint.
void UUIComponentUserWidgetExtension::CreateAndAddComponent(UUIComponent* ArchetypeComponent, FName OwnerName)
{
	ensure(ArchetypeComponent);
	ensure(ComponentContainer);
	if (ArchetypeComponent && ComponentContainer)
	{
		ensure(ComponentContainer && ArchetypeComponent && ArchetypeComponent->HasAllFlags(RF_ArchetypeObject));

		// First, remove any component already on the Widget, replace it with a new one.
		RemoveComponent(ArchetypeComponent->GetClass(), OwnerName);

		// Create the component with the WBP Component as the Archetype.
		UObject* Outer = ComponentContainer;	
		UUIComponent* PreviewWidgetComponent = NewObject<UUIComponent>(Outer,
			ArchetypeComponent->GetClass(),
			NAME_None, RF_Transactional,
			ArchetypeComponent);

		ComponentContainer->AddComponent(OwnerName, PreviewWidgetComponent);
	}
}

void UUIComponentUserWidgetExtension::RemoveComponent(const UClass* ComponentClass, FName OwnerName)
{
	if (ComponentContainer)
	{
		ComponentContainer->RemoveAllComponentsOfType(ComponentClass, OwnerName);
	}
}

void UUIComponentUserWidgetExtension::CleanupComponents()
{
	const UUserWidget* UserWidget = GetUserWidget();
	if (UserWidget && UserWidget->WidgetTree && ComponentContainer)
	{
		ComponentContainer->CleanupUIComponents(UserWidget->WidgetTree);
	}
}

void UUIComponentUserWidgetExtension::MoveComponent(FName OwnerName, const UClass* ComponentClass, const UClass* RelativeToComponentClass, bool bMoveAfter)
{
	if (ComponentContainer)
	{
		ComponentContainer->MoveComponent(OwnerName, ComponentClass, RelativeToComponentClass, bMoveAfter);
	}
}

#endif // WITH_EDITOR

TArray<UUIComponent*> UUIComponentUserWidgetExtension::GetComponentsFor(UWidget* Target) const
{
	ensure(Target);

	TArray<UUIComponent*> Components;
	if(Target)
	{
		ComponentContainer->ForEachComponent([Target, &Components](UUIComponent* Component){			
			if(ensure(Component) && Component->GetOwner() == Target)			
			{
				Components.Push(Component);
			}
		});
	}
	return Components;	
}

UUIComponent* UUIComponentUserWidgetExtension::GetComponent(const UClass* ComponentClass, FName OwnerName) const
{
	check(ComponentContainer);
	return ComponentContainer->GetComponent(ComponentClass, OwnerName);
}
