// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/UIComponentWidgetBlueprintGeneratedClassExtension.h"

#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "CoreMinimal.h"
#include "Blueprint/WidgetTree.h"
#include "Extensions/UIComponent.h"
#include "Extensions/UIComponentContainer.h"
#include "Extensions/UIComponentUserWidgetExtension.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UIComponentWidgetBlueprintGeneratedClassExtension)

UUIComponentContainer* UUIComponentWidgetBlueprintGeneratedClassExtension::DuplicateContainer(UUserWidget* UserWidget) const
{
	check(ComponentContainer);
	check(!ComponentContainer->IsEmpty());
	
	UObject* Outer = UserWidget;
		
	//FObjectInstancingGraph ObjectInstancingGraph;
	UUIComponentContainer* NewContainer = NewObject<UUIComponentContainer>(Outer,
		ComponentContainer->GetClass(),
		NAME_None,
		RF_Transactional,
		ComponentContainer,
		false);

	return NewContainer;
}

bool UUIComponentWidgetBlueprintGeneratedClassExtension::VerifyContainer(const UUserWidget* UserWidget) const
{
	if(const UUIComponentUserWidgetExtension* UserWidgetExtension = UserWidget->GetExtension<UUIComponentUserWidgetExtension>())
	{
		TArray<FUIComponentTarget> Components;
		ComponentContainer->ForEachComponentTarget([&Components](const FUIComponentTarget& ComponentTarget){
			Components.Push(ComponentTarget);
		});

		while(Components.Num() > 0)
		{
			const FUIComponentTarget& Target = Components[Components.Num()-1];
			if(!UserWidgetExtension->GetComponent(Target.GetComponent()->GetClass(), Target.GetTargetName()))
			{
				return false;
			}
			else
			{
				Components.RemoveAt(Components.Num()-1);
			}
		}
		return true;
	}
	return false;	
}

#if WITH_EDITOR

void UUIComponentWidgetBlueprintGeneratedClassExtension::InitializeInEditor(UUserWidget* UserWidget)
{
	// We simply call the Initialize function as we need the same code to execute in Editor.
	Initialize(UserWidget);
}

bool UUIComponentWidgetBlueprintGeneratedClassExtension::VerifyAllWidgetsExists(const UWidgetTree* WidgetTree) const
{	
	TSet<FName> Targets;
	ComponentContainer->ForEachComponentTarget([&Targets](const FUIComponentTarget& ComponentTarget){
		Targets.FindOrAdd(ComponentTarget.GetTargetName());
	});

	for (FName& WidgetName : Targets)
	{
		if (WidgetTree->FindWidget(WidgetName))
		{
			continue;
		}
		return false;
	}
	return true;	
}
#endif // WITH_EDITOR

void UUIComponentWidgetBlueprintGeneratedClassExtension::InitializeContainer(UUIComponentContainer* InComponentContainer)
{
	check(InComponentContainer != nullptr);
	check(ComponentContainer == nullptr);
	ComponentContainer = InComponentContainer;
}

void UUIComponentWidgetBlueprintGeneratedClassExtension::Initialize(UUserWidget* UserWidget)
{
	UUIComponentUserWidgetExtension* UserWidgetExtension = UserWidget->GetExtension<UUIComponentUserWidgetExtension>();
	if (!UserWidgetExtension)
	{
		UserWidgetExtension = UserWidget->AddExtension<UUIComponentUserWidgetExtension>();
	}
    // Since some Widget can be duplicated manually, we could end up not Adding the Extension but still need to Initialize it.
    // When we are in UMG Designer preview widget, it will be initialized by the UUIComponentWidgetBlueprintExtension, in that case, no
	// need to Initialize it.
	if (!UserWidgetExtension->IsContainerInitialized())
	{
		UserWidgetExtension->InitializeContainer(DuplicateContainer(UserWidget));
	}
}


void UUIComponentWidgetBlueprintGeneratedClassExtension::PreConstruct(UUserWidget* UserWidget, bool IsDesignTime)
{
	// This is where we can add the UserWidgetExtension and Duplicate the Component Container from the GeneratedClassExtension

	// If the extension do not exists, Add it.
	UUIComponentUserWidgetExtension* UserWidgetExtension = UserWidget->GetExtension<UUIComponentUserWidgetExtension>();
	if (!UserWidgetExtension)
	{
		UserWidgetExtension = UserWidget->AddExtension<UUIComponentUserWidgetExtension>();
		UserWidgetExtension->InitializeContainer(DuplicateContainer(UserWidget));
	}
	else
	{
		if (UserWidget != UserWidgetExtension->GetOuterUUserWidget())
		{
			UserWidgetExtension->Rename(nullptr, UserWidget);
		}
	}

	if (!IsDesignTime)
	{
		// Set the component object in the corresponding property on the userwidget.
		ComponentContainer->ForEachComponentTarget([UserWidgetExtension, UserWidget](const FUIComponentTarget& ComponentTarget) {
			FName ComponentPropertyName = UUIComponentContainer::GetPropertyNameForComponent(ComponentTarget.GetComponent(), ComponentTarget.GetTargetName());

			FObjectPropertyBase* FoundPanelObjectProperty = FindFProperty<FObjectPropertyBase>(UserWidget->GetClass(), ComponentPropertyName);
			bool bValidPanelObject = FoundPanelObjectProperty && FoundPanelObjectProperty->PropertyClass->IsChildOf(UUIComponent::StaticClass());
			if (ensureAlwaysMsgf(bValidPanelObject, TEXT("The compiler should have added the property")))
			{
				if (ensure(FoundPanelObjectProperty->PropertyClass == ComponentTarget.GetComponent()->GetClass()))
				{
					UUIComponent* UserWidgetComponent = UserWidgetExtension->GetComponent(FoundPanelObjectProperty->PropertyClass, ComponentTarget.GetTargetName());
					FoundPanelObjectProperty->SetObjectPropertyValue_InContainer(UserWidget, UserWidgetComponent);
				}
			}
			});

	}
}
