// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extensions/UIComponents/NavigationUIComponent.h"

#include "Blueprint/WidgetTree.h"
#include "CoreGlobals.h"
#include "Components/Widget.h"
#include "Slate/SObjectWidget.h"
#include "Types/ReflectionMetadata.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationUIComponent)

namespace UIComponent::Private
{
UUserWidget* FindNearestUserWidget(TWeakPtr<SWidget> Widget)
{
	TSharedPtr<SWidget> PinnedWidget = Widget.Pin();
	while (PinnedWidget.IsValid())
	{
		if (PinnedWidget->GetWidgetClass().GetWidgetType() == SObjectWidget::StaticWidgetClass().GetWidgetType())
		{
			return StaticCastSharedPtr<SObjectWidget>(PinnedWidget)->GetWidgetObject();
		}
		PinnedWidget = PinnedWidget->GetParentWidget();
	}
	return nullptr;
}

UWidget* FindUMGWidget(TWeakPtr<SWidget> Widget)
{
	UUserWidget* OwningUserWidget = FindNearestUserWidget(Widget);
	if (OwningUserWidget == nullptr || OwningUserWidget->WidgetTree == nullptr)
	{
		return nullptr;
	}

	UWidget* NearestUMGWidget = nullptr;
	int32 NearestUMGWidgetDistance = INT32_MAX;

	OwningUserWidget->WidgetTree->ForEachWidgetUntil([Widget, &NearestUMGWidget, &NearestUMGWidgetDistance](UWidget* UMGWidget) {
		if (NearestUMGWidget == nullptr)
		{
			NearestUMGWidget = UMGWidget;
		}
		else
		{
			int32 UMGWidgetDistance = 0;

			TSharedPtr<SWidget> PinnedWidget = Widget.Pin();
			while (PinnedWidget.IsValid())
			{
				if (UMGWidget->GetCachedWidget() == PinnedWidget && UMGWidgetDistance < NearestUMGWidgetDistance)
				{
					NearestUMGWidget = UMGWidget;
					NearestUMGWidgetDistance = UMGWidgetDistance;
					
					//break widget search if can not find closer widget
					return NearestUMGWidgetDistance != 0; 
				}

				++UMGWidgetDistance;
				PinnedWidget = PinnedWidget->GetParentWidget();
			}
		}
		// Stop searching when slate widget is found
		return UMGWidget->GetCachedWidget() != Widget;
	});

	return NearestUMGWidget;
}
}

void UNavigationUIComponent::OnConstruct()
{
	TWeakObjectPtr<UWidget> OwningWidget = GetOwner();
	if (!ensure(OwningWidget.IsValid()))
	{
		return;
	}

	UUserWidget* OwningUserWidget = OwningWidget->GetTypedOuter<UUserWidget>();
	if (!ensure(OwningUserWidget))
	{
		return;
	}

	// Bind user provided functions to component delegates

	FScriptDelegate EnteredDelegate;
	EnteredDelegate.BindUFunction(OwningUserWidget, OnNavigationEntered);
	OnNavigationEnteredDelegate.AddUnique(EnteredDelegate);

	FScriptDelegate ExitedDelegate;
	ExitedDelegate.BindUFunction(OwningUserWidget, OnNavigationExited);
	OnNavigationExitedDelegate.AddUnique(ExitedDelegate);

	TSharedPtr<SWidget> Content = OwningUserWidget->GetCachedWidget();
	if (!Content.IsValid())
	{
		return;
	}

	NavigationTransitionMetadata = MakeShared<FNavigationTransitionMetadata>();
	NavigationTransitionMetadata->OnNavigationTransition.BindUObject(this, &UNavigationUIComponent::HandleNavigationTransition);

	Content->AddMetadata(NavigationTransitionMetadata.ToSharedRef());
}

void UNavigationUIComponent::OnDestruct()
{
	OnNavigationEnteredDelegate.Clear();
	OnNavigationExitedDelegate.Clear();
}

void UNavigationUIComponent::OnGraphRenamed(UEdGraph* Graph, const FName& OldName, const FName& NewName)
{
	if (OnNavigationEntered == OldName)
	{
		OnNavigationEntered = NewName;
	}
	if (OnNavigationExited == OldName)
	{
		OnNavigationExited = NewName;
	}
}

void UNavigationUIComponent::HandleNavigationTransition(const FNavigationTransition& NavigationTransition)
{
	// Forward slate navigation transition to user provided functions via component delegates
	
	UWidget* OldUMGWidget = UIComponent::Private::FindUMGWidget(NavigationTransition.OldFocusedWidget);
	UWidget* NewUMGWidget = UIComponent::Private::FindUMGWidget(NavigationTransition.NewFocusedWidget);

	switch (NavigationTransition.Direction)
	{
		case ENavigationTransitionDirection::Incoming:
		{
			OnNavigationEnteredDelegate.Broadcast(NavigationTransition.Type, OldUMGWidget, NewUMGWidget);
			break;
		}
		case ENavigationTransitionDirection::Outgoing:
		{
			OnNavigationExitedDelegate.Broadcast(NavigationTransition.Type, OldUMGWidget, NewUMGWidget);
			break;
		}
	}
}
