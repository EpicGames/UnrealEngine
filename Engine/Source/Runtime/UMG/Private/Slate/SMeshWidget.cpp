// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SMeshWidget.h"
#include "Rendering/DrawElements.h"
#include "Modules/ModuleManager.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Framework/Application/SlateApplication.h"
#include "RHI.h"
#include "UMGPrivate.h"

#include "SlateMaterialBrush.h"
#include "Interfaces/ISlateRHIRendererModule.h"
#include "Slate/SlateVectorArtData.h"
#include "Slate/SlateVectorArtInstanceData.h"



/** Populate OutSlateVerts and OutIndexes with data from this static mesh such that Slate can render it. */
static void SlateMeshToSlateRenderData(const USlateVectorArtData& DataSource, TArray<FSlateVertex>& OutSlateVerts, TArray<SlateIndex>& OutIndexes)
{
	// Populate Index data
	{
		// Note that we do a slow copy because on some platforms the SlateIndex is
		// a 16-bit value, so we cannot do a memcopy.
		const TArray<uint32>& IndexDataSource = DataSource.GetIndexData();
		const int32 NumIndexes = IndexDataSource.Num();
		OutIndexes.Empty();
		OutIndexes.Reserve(NumIndexes);
		for (int32 i = 0; i < NumIndexes; ++i)
		{
			OutIndexes.Add(IndexDataSource[i]);
		}
	}

	// Populate Vertex Data
	{
		const TArray<FSlateMeshVertex> VertexDataSource = DataSource.GetVertexData();
		const uint32 NumVerts = VertexDataSource.Num();
		OutSlateVerts.Empty();
		OutSlateVerts.Reserve(NumVerts);

		for (uint32 i = 0; i < NumVerts; ++i)
		{
			const FSlateMeshVertex& SourceVertex = VertexDataSource[i];
			FSlateVertex& NewVert = OutSlateVerts[OutSlateVerts.AddUninitialized()];

			// Copy Position
			{
				NewVert.Position[0] = SourceVertex.Position.X;
				NewVert.Position[1] = SourceVertex.Position.Y;
			}

			// Copy Color
			{
				NewVert.Color = SourceVertex.Color;
			}

			// Copy all the UVs that we have, and as many as we can fit.
			{
				NewVert.TexCoords[0] = SourceVertex.UV0.X;
				NewVert.TexCoords[1] = SourceVertex.UV0.Y;

				NewVert.TexCoords[2] = SourceVertex.UV1.X;
				NewVert.TexCoords[3] = SourceVertex.UV1.Y;

				NewVert.MaterialTexCoords[0] = SourceVertex.UV2.X;
				NewVert.MaterialTexCoords[1] = SourceVertex.UV2.Y;
			}
		}
	}
}



void SMeshWidget::Construct(const FArguments& Args)
{
	if (Args._MeshData != nullptr)
	{
		AddMesh(*Args._MeshData);
	}
}

static const FVector2D DontCare(FVector2D(64, 64));
uint32 SMeshWidget::AddMesh(USlateVectorArtData& InMeshData)
{	
	InMeshData.EnsureValidData();

	FRenderData& NewRenderData = RenderData[RenderData.Add(FRenderData())];
	UMaterialInterface* MaterialFromMesh = InMeshData.GetMaterial();
	if (MaterialFromMesh != nullptr)
	{
		NewRenderData.Brush = MakeShareable(new FSlateMaterialBrush(*MaterialFromMesh, DontCare));
		NewRenderData.RenderingResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle( *NewRenderData.Brush );
	}
	SlateMeshToSlateRenderData(InMeshData, NewRenderData.VertexData, NewRenderData.IndexData);
	return RenderData.Num()-1;
}

uint32 SMeshWidget::AddMeshWithInstancing(USlateVectorArtData& InMeshData, int32 InitialBufferSize)
{
	const uint32 NewMeshId = AddMesh(InMeshData);
	EnableInstancing(NewMeshId, InitialBufferSize);
	return NewMeshId;
}


UMaterialInstanceDynamic* SMeshWidget::ConvertToMID(uint32 MeshId)
{
	FRenderData& MeshRenderData = RenderData[MeshId];
	UObject* ResourceObject = MeshRenderData.Brush->GetResourceObject();
	UMaterialInterface* Material = Cast<UMaterialInterface>(ResourceObject);

	UMaterialInstanceDynamic* ExistingMID = Cast<UMaterialInstanceDynamic>(Material);
	if (ExistingMID == nullptr)
	{
		UMaterialInstanceDynamic* NewMID = UMaterialInstanceDynamic::Create(Material, nullptr);
		MeshRenderData.Brush->SetResourceObject(NewMID);
		MeshRenderData.RenderingResourceHandle = FSlateApplication::Get().GetRenderer()->GetResourceHandle(*MeshRenderData.Brush);
		return NewMID;
	}
	else
	{
		return ExistingMID;
	}
}


void SMeshWidget::ClearRuns(int32 NumRuns)
{
	RenderRuns.Reset(NumRuns);
}

static const FName SlateRHIModuleName("SlateRHIRenderer");
void SMeshWidget::EnableInstancing(uint32 MeshId, int32 InitialSize)
{
	if (!RenderData[MeshId].PerInstanceBuffer.IsValid())
	{
		RenderData[MeshId].PerInstanceBuffer = FModuleManager::Get().GetModuleChecked<ISlateRHIRendererModule>(SlateRHIModuleName).CreateInstanceBuffer(InitialSize);
	}
}

void SMeshWidget::UpdatePerInstanceBuffer(uint32 MeshId, FSlateInstanceBufferData& Data)
{
	EnableInstancing(MeshId, Data.Num() * Data.GetTypeSize());
	RenderData[MeshId].PerInstanceBuffer->Update(Data);
}


int32 SMeshWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{

	if (RenderRuns.Num() > 0)
	{
		// We have multiple render runs. Assume that we have per-instance data.
		for (int RunIndex = 0; RunIndex < RenderRuns.Num(); ++RunIndex)
		{
			const FRenderRun& Run = RenderRuns[RunIndex];
			const FRenderData& RunRenderData = RenderData[Run.GetMeshIndex()];
			if (RunRenderData.RenderingResourceHandle.IsValid() && RunRenderData.VertexData.Num() > 0 && RunRenderData.IndexData.Num() > 0 && RunRenderData.PerInstanceBuffer.IsValid())
			{
				ensure(Run.GetInstanceOffset() + Run.GetNumInstances() <= RunRenderData.PerInstanceBuffer->GetNumInstances());
				FSlateDrawElement::MakeCustomVerts(OutDrawElements,
					LayerId,
					RunRenderData.RenderingResourceHandle,
					RunRenderData.VertexData,
					RunRenderData.IndexData,
					RunRenderData.PerInstanceBuffer.Get(),
					Run.GetInstanceOffset(),
					Run.GetNumInstances());
			}
			else
			{
				if( !GUsingNullRHI )
				{
					UE_LOGF(LogUMG, Warning, "SMeshWidget did not render a run because of one of these Brush: %ls, InstanceBuffer: %ls, NumVertexes: %d, NumIndexes: %d",
						RunRenderData.RenderingResourceHandle.IsValid() ? TEXT("valid") : TEXT("nullptr"),
						RunRenderData.PerInstanceBuffer.IsValid() ? TEXT("valid") : TEXT("nullptr"),
						RunRenderData.VertexData.Num(),
						RunRenderData.IndexData.Num());
				}
			}
		}
	}
	else
	{
		// We have no render runs. Render all the meshes in order they were added
		for (int i = 0; i < RenderData.Num(); ++i)
		{
			const FRenderData& RunRenderData = RenderData[i];
			if (RunRenderData.RenderingResourceHandle.IsValid() && RunRenderData.VertexData.Num() > 0 && RunRenderData.IndexData.Num() > 0)
			{
				if (RunRenderData.PerInstanceBuffer.IsValid())
				{
					// Drawing instanced widgets
					const int32 NumInstances = RunRenderData.PerInstanceBuffer->GetNumInstances();
					if (NumInstances > 0)
					{
						FSlateDrawElement::MakeCustomVerts(OutDrawElements, LayerId, RunRenderData.RenderingResourceHandle, RunRenderData.VertexData, RunRenderData.IndexData, RunRenderData.PerInstanceBuffer.Get(), 0, NumInstances);
					}
				}
				else
				{
					// Drawing a single widget, no instancing
					FSlateDrawElement::MakeCustomVerts(OutDrawElements, LayerId, RunRenderData.RenderingResourceHandle, RunRenderData.VertexData, RunRenderData.IndexData, nullptr, 0, 0);
				}
			}
			else
			{
				if( !GUsingNullRHI )
				{
					UE_LOGF(LogUMG, Warning, "SMeshWidget did not render a run because of one of these Brush: %ls, NumVertexes: %d, NumIndexes: %d",
						RunRenderData.RenderingResourceHandle.IsValid() ? TEXT("valid") : TEXT("nullptr"),
						RunRenderData.VertexData.Num(),
						RunRenderData.IndexData.Num());
				}
			}
		}
	}

	return LayerId;
}


FVector2D SMeshWidget::ComputeDesiredSize(float) const
{
	return FVector2D(256,256);
}


void SMeshWidget::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (const FRenderData& SomeRenderData : RenderData)
	{
		if (SomeRenderData.Brush.IsValid())
		{
			SomeRenderData.Brush->AddReferencedObjects(Collector);
		}
	}
}


FString SMeshWidget::GetReferencerName() const
{
	return TEXT("SMeshWidget");
}

void SMeshWidget::PushUpdate(uint32 VectorArtId, SMeshWidget& Widget, const FVector2D& Position, float Scale, uint32 BaseAddress)
{
	PushUpdate(VectorArtId, Widget, Position, Scale, static_cast<float>(BaseAddress));
}

void SMeshWidget::PushUpdate(uint32 VectorArtId, SMeshWidget& Widget, const FVector2D& Position, float Scale, float OptionalFloat /*= 0*/)
{
	FSlateVectorArtInstanceData Data;
	Data.SetPosition(Position);
	Data.SetScale(Scale);
	Data.SetBaseAddress(OptionalFloat);

	{
		FSlateInstanceBufferData PerInstanceData;
		PerInstanceData.Add((FVector4f)Data.GetData()); // LWC_TODO: precision loss
		Widget.UpdatePerInstanceBuffer(VectorArtId, PerInstanceData);
	}
}
