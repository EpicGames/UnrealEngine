// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SObjectWidget.h"

#include "UMGPrivate.h"
#include "Blueprint/WidgetLayoutLibrary.h"
#include "Slate/UMGDragDropOp.h"
#include "SlateGlobals.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(SLATECORE_API, Slate);

SLATE_IMPLEMENT_WIDGET(SObjectWidget)
void SObjectWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SObjectWidget::Construct(const FArguments& InArgs, UUserWidget* InWidgetObject)
{
	WidgetObject = InWidgetObject;

	ChildSlot
	[
		InArgs._Content.Widget
	];

	// The user widget will tell us if we can tick but default to false for now
	SetCanTick(false);

	if (WidgetObject)
	{
#if SLATE_VERBOSE_NAMED_EVENTS
		DebugName = WidgetObject->GetFullName();
		DebugPaintEventName = DebugName + TEXT("_Paint");
		DebugTickEventName = DebugName + TEXT("_Tick");
#endif
		WidgetObject->UpdateCanTick();
	}
}

SObjectWidget::SObjectWidget() = default;

SObjectWidget::~SObjectWidget()
{
#if SLATE_VERBOSE_NAMED_EVENTS && !WITH_EDITOR
	// This is only a concern during a running game - design-time instances can be destroyed from GC quite often when recompiling
	if (!WidgetObject || !WidgetObject->IsDesignTime())
	{
		ensureMsgf(GExitPurge || !IsGarbageCollecting(), TEXT("SObjectWidget for '%s' destroyed while collecting garbage.  This can lead to multiple GCs being required to cleanup the object.  Possible causes might be,\n1) ReleaseSlateResources not being implemented for the owner of this pointer.\n2) You may just be holding onto some slate pointers on an actor that don't get reset until the actor is Garbage Collected.  You should avoid doing this, and instead reset those references when the actor is Destroyed."), *DebugName);
	}
#endif

	ResetWidget();
}

void SObjectWidget::ResetWidget()
{
	if ( UObjectInitialized() && WidgetObject )
	{
		UnregisterGCObject();

		if ( CanRouteEvent() )
		{
			WidgetObject->NativeDestruct();
		}

		// NOTE: When the SObjectWidget gets released we know that the User Widget has
		// been removed from the slate widget hierarchy.  When this occurs, we need to 
		// immediately release all slate widget widgets to prevent deletion from taking
		// n-frames due to widget nesting.
		const bool bReleaseChildren = true;
		WidgetObject->ReleaseSlateResources(bReleaseChildren);

		WidgetObject = nullptr;

#if SLATE_VERBOSE_NAMED_EVENTS
		DebugTickEventName = TEXT("");
		DebugPaintEventName = TEXT("");
#endif
	}

	// Remove slate widget from our container
	ChildSlot
	[
		SNullWidget::NullWidget
	];
}

FString SObjectWidget::GetReferencerName() const
{
	TStringBuilder<512> Builder;
	Builder << TEXT("SObjectWidget(");
	if (WidgetObject)
	{
		Builder << WidgetObject->GetFName();
	}
	else
	{
		Builder << TEXT("nullptr");
	}		
	Builder << TEXT(")");
	return Builder.ToString();
}

void SObjectWidget::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddStableReference(&WidgetObject);
}

void SObjectWidget::SetPadding(const TAttribute<FMargin>& InMargin)
{
	ChildSlot.Padding(InMargin);
}

void SObjectWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Note: This tick will not execute unless the UserWidget itself ticks.
#if SLATE_VERBOSE_NAMED_EVENTS
	SCOPED_NAMED_EVENT_FSTRING(DebugTickEventName, FColor::Turquoise);
	CSV_SCOPED_TIMING_STAT(Slate, SObjectWidget_Tick);
#endif

#if WITH_VERY_VERBOSE_SLATE_STATS
	FScopeCycleCounterUObject NativeFunctionScope(WidgetObject);
#endif

	if ( CanRouteEvent() )
	{
		WidgetObject->NativeTick(AllottedGeometry, InDeltaTime);
	}
}

int32 SObjectWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
#if SLATE_VERBOSE_NAMED_EVENTS
	SCOPED_NAMED_EVENT_FSTRING(DebugPaintEventName, FColor::Silver);
#endif

#if WITH_VERY_VERBOSE_SLATE_STATS
	FScopeCycleCounterUObject NativeFunctionScope(WidgetObject);
#endif

	int32 MaxLayer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	if ( CanRoutePaint() )
	{
		return WidgetObject->NativePaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, MaxLayer, InWidgetStyle, bParentEnabled);
	}
	
	return MaxLayer;
}

FVector2D SObjectWidget::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	const FVector2D BaseDesiredSize = SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);

	if (WidgetObject)
	{
		const FVector2D MinimumDesiredSize = WidgetObject->GetMinimumDesiredSize();
		return FVector2D::Max(BaseDesiredSize, MinimumDesiredSize);
	}

	return BaseDesiredSize;
}

bool SObjectWidget::IsInteractable() const
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeIsInteractable();
	}

	return false;
}

bool SObjectWidget::SupportsKeyboardFocus() const
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeSupportsKeyboardFocus();
	}

	return false;
}

FReply SObjectWidget::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnFocusReceived( MyGeometry, InFocusEvent );
	}

	return FReply::Unhandled();
}

void SObjectWidget::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	if ( CanRouteEvent() )
	{
		WidgetObject->NativeOnFocusLost( InFocusEvent );
	}
}

void SObjectWidget::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	if ( CanRouteEvent() )
	{
		WidgetObject->NativeOnFocusChanging(PreviousFocusPath, NewWidgetPath, InFocusEvent);
	}
}

FReply SObjectWidget::OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnKeyChar( MyGeometry, InCharacterEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnPreviewKeyDown( MyGeometry, InKeyEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if ( CanRouteEvent() )
	{
		FReply Result = WidgetObject->NativeOnKeyDown( MyGeometry, InKeyEvent );
		if ( !Result.IsEventHandled() )
		{
			return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
		}

		return Result;
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if ( CanRouteEvent() )
	{
		FReply Result = WidgetObject->NativeOnKeyUp(MyGeometry, InKeyEvent);
		if ( !Result.IsEventHandled() )
		{
			return SCompoundWidget::OnKeyUp(MyGeometry, InKeyEvent);
		}

		return Result;
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent)
{
	if ( CanRouteEvent() )
	{
		FReply Result = WidgetObject->NativeOnAnalogValueChanged(MyGeometry, InAnalogInputEvent);
		if ( !Result.IsEventHandled() )
		{
			return SCompoundWidget::OnAnalogValueChanged(MyGeometry, InAnalogInputEvent);
		}

		return Result;
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnMouseButtonDown( MyGeometry, MouseEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnPreviewMouseButtonDown( MyGeometry, MouseEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnMouseButtonUp( MyGeometry, MouseEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnMouseMove( MyGeometry, MouseEvent );
	}

	return FReply::Unhandled();
}

void SObjectWidget::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// NOTE: Done so that IsHovered() works
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	if ( CanRouteEvent() )
	{
		WidgetObject->NativeOnMouseEnter( MyGeometry, MouseEvent );
	}
}

void SObjectWidget::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	// NOTE: Done so that IsHovered() works
	SCompoundWidget::OnMouseLeave(MouseEvent);

	if ( CanRouteEvent() )
	{
		WidgetObject->NativeOnMouseLeave( MouseEvent );
	}
}

FReply SObjectWidget::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnMouseWheel( MyGeometry, MouseEvent );
	}

	return FReply::Unhandled();
}

FCursorReply SObjectWidget::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnCursorQuery( MyGeometry, CursorEvent );
	}

	return FCursorReply::Unhandled();
}

FReply SObjectWidget::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnMouseButtonDoubleClick( MyGeometry, MouseEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{
	if ( CanRouteEvent() )
	{
		UDragDropOperation* Operation = nullptr;
		WidgetObject->NativeOnDragDetected( MyGeometry, PointerEvent, Operation );

		if ( Operation )
		{
			FVector2D ScreenCursorPos = PointerEvent.GetScreenSpacePosition();
			FVector2D ScreenDrageePosition = MyGeometry.GetAbsolutePosition();

			float DPIScale = UWidgetLayoutLibrary::GetViewportScale(WidgetObject);

			TSharedRef<FUMGDragDropOp> DragDropOp = FUMGDragDropOp::New(Operation, PointerEvent.GetPointerIndex(), ScreenCursorPos, ScreenDrageePosition, DPIScale, SharedThis(this));

			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

void SObjectWidget::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if ( NativeOp.IsValid() )
	{
		if ( CanRouteEvent() )
		{
			WidgetObject->NativeOnDragEnter( MyGeometry, DragDropEvent, NativeOp->GetOperation() );
		}
	}
}

void SObjectWidget::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if ( NativeOp.IsValid() )
	{
		if ( CanRouteEvent() )
		{
			WidgetObject->NativeOnDragLeave( DragDropEvent, NativeOp->GetOperation() );
		}
	}
}

FReply SObjectWidget::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if ( NativeOp.IsValid() )
	{
		if ( CanRouteEvent() )
		{
			if ( WidgetObject->NativeOnDragOver( MyGeometry, DragDropEvent, NativeOp->GetOperation() ) )
			{
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if ( NativeOp.IsValid() )
	{
		if ( CanRouteEvent() )
		{
			if ( WidgetObject->NativeOnDrop( MyGeometry, DragDropEvent, NativeOp->GetOperation() ) )
			{
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

void SObjectWidget::OnDragCancelled(const FDragDropEvent& DragDropEvent, UDragDropOperation* Operation)
{
	TSharedPtr<FUMGDragDropOp> NativeOp = DragDropEvent.GetOperationAs<FUMGDragDropOp>();
	if ( NativeOp.IsValid() )
	{
		if ( CanRouteEvent() )
		{
			WidgetObject->NativeOnDragCancelled( DragDropEvent, NativeOp->GetOperation() );
		}
	}
}

FReply SObjectWidget::OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnTouchGesture( MyGeometry, GestureEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnTouchStarted( MyGeometry, InTouchEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnTouchMoved( MyGeometry, InTouchEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnTouchEnded( MyGeometry, InTouchEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnMotionDetected(const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent)
{
	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnMotionDetected( MyGeometry, InMotionEvent );
	}

	return FReply::Unhandled();
}

FReply SObjectWidget::OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (CanRouteEvent())
	{
		return WidgetObject->NativeOnTouchForceChanged(MyGeometry, InTouchEvent);
	}
	return FReply::Unhandled();
}

FReply SObjectWidget::OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (CanRouteEvent())
	{
		return WidgetObject->NativeOnTouchFirstMove(MyGeometry, InTouchEvent);
	}
	return FReply::Unhandled();
}

void SObjectWidget::OnFinishedPointerInput()
{
	if (CanRouteEvent())
	{
		WidgetObject->NativeOnFinishedPointerInput();
	}
}

FNavigationReply SObjectWidget::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	if (WidgetObject && WidgetObject->NativeSupportsCustomNavigation())
	{
		return WidgetObject->NativeOnNavigation(MyGeometry, InNavigationEvent);
	}
	FNavigationReply Reply = SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);

	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnNavigation(MyGeometry, InNavigationEvent, Reply);
	}

	return Reply;
}

void SObjectWidget::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	SCompoundWidget::OnMouseCaptureLost(CaptureLostEvent);

	if ( CanRouteEvent() )
	{
		return WidgetObject->NativeOnMouseCaptureLost(CaptureLostEvent);
	}
}
