// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SPostBufferUpdate.h"

#include "Engine/AssetManager.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"

// Exclude SlateRHIRenderer related includes & implementations from server builds since the module is not a dependency / will not link for UMG on the server
#if !UE_SERVER
#include "FX/SlateFXSubsystem.h"
#include "FX/SlateRHIPostBufferProcessor.h"
#include "Interfaces/SlateRHIRenderingPolicyInterface.h"
#include "Rendering/DrawElements.h"
#include "Rendering/ElementBatcher.h"
#include "Math/MathFwd.h"
#include "RenderUtils.h"
#include "RHICommandList.h"
#include "RHIFwd.h"
#include "RHIResources.h"
#include "RHIUtilities.h"
#include "SlateRHIRendererSettings.h"
#include "TextureResource.h"

/**
 * Custom Slate drawer to update slate post buffer
 */
class FPostBufferUpdater : public ICustomSlateElement
{
public:

	//~ Begin ICustomSlateElement interface
	virtual void Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs) override;
	virtual void PostCustomElementAdded(class FSlateElementBatcher& ElementBatcher) const override;
	//~ End ICustomSlateElement interface

public:

	/** True if buffers to update has been initialized. Used to ensure 'BuffersToUpdate_Renderthread' is only set once at initialization. */
	bool bBuffersToUpdateInitialized = false;

	/** True if we should use the subregion rect */
	bool bUsePaintGeometry_Renderthread = false;

	/** 
	 * Buffers that we should update, all of these buffers will be affected by 'bPerformDefaultPostBufferUpdate' if disabled 
	 * This value is read by the Renderthread, so all non-initialization updates must be done via a render command.
	 * See 'FSlatePostBufferBlurProxy::OnUpdateValuesRenderThread' as an example.
	 * 
	 * Additionally, note that this value should be masked against the currently enabled buffers in 'USlateRHIRendererSettings'
	 */
	ESlatePostRT BuffersToUpdate_Renderthread = ESlatePostRT::None;

	/** Proxies used to update a post processor within a frame */
	TMap<ESlatePostRT, TSharedPtr<FSlatePostProcessorUpdaterProxy>> ProcessorUpdaters;

	/** Paint geometry to use for subregion processing */
	FPaintGeometry PaintGeometry_Renderthread = FPaintGeometry();
};

/////////////////////////////////////////////////////
// FPostBufferUpdater

void FPostBufferUpdater::Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs)
{
	const USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::Get();
	if (!RendererSettings)
	{
		return;
	}

	struct FActivePostBuffer
	{
		FRDGTexture* Texture = nullptr;
		TSharedPtr<FSlateRHIPostBufferProcessorProxy> Proxy;
		int32 Downscale;
	};

	// Issue internal / external access mode calls in batches before and after to reduce the number of RDG passes.
	TStaticArray<FActivePostBuffer, (int32)ESlatePostRT::Num> ActivePostBuffers;
	int32 SlatePostBufferIndex = 0;

	for (ESlatePostRT SlatePostBufferBit : MakeFlagsRange(Inputs.UsedSlatePostBuffers & BuffersToUpdate_Renderthread))
	{
		ON_SCOPE_EXIT
		{
			SlatePostBufferIndex++;
		};

		UTextureRenderTarget2D* SlatePostBuffer = Cast<UTextureRenderTarget2D>(RendererSettings->TryGetPostBufferRT(SlatePostBufferBit));
		if (!SlatePostBuffer)
		{
			continue;
		}

		TSharedPtr<FSlateRHIPostBufferProcessorProxy> PostProcessorProxy = USlateFXSubsystem::GetPostProcessorProxy(SlatePostBufferBit);

		if (PostProcessorProxy)
		{
			if (TSharedPtr<FSlatePostProcessorUpdaterProxy>* ProcessorUpdaterItr = ProcessorUpdaters.Find(SlatePostBufferBit))
			{
				if (TSharedPtr<FSlatePostProcessorUpdaterProxy> ProcessorUpdater = *ProcessorUpdaterItr)
				{
					ProcessorUpdater->UpdateProcessor_RenderThread(PostProcessorProxy);
				}
			}
		}

		// Force the first barrier to be immediate to handle edge case where a prior slate render batch can be referencing an older version of this resource.
		FRDGTexture* Texture = RegisterExternalTexture(GraphBuilder, SlatePostBuffer->GetRenderTargetResource()->GetTextureRHI(), TEXT("SlatePostProcessTexture"), ERDGTextureFlags::ForceImmediateFirstBarrier);

		const int32 Downscale = RendererSettings->GetSlatePostBufferDownscaleFactor(SlatePostBufferBit);

		ActivePostBuffers[SlatePostBufferIndex] = FActivePostBuffer
		{
			  .Texture = Texture
			, .Proxy = MoveTemp(PostProcessorProxy)
			, .Downscale = Downscale
		};

		GraphBuilder.UseInternalAccessMode(Texture);
	}

	const FIntPoint ViewRectMinimumSize = FIntPoint(1, 1);
	for (const FActivePostBuffer& ActivePostBuffer : ActivePostBuffers)
	{
		if (ActivePostBuffer.Texture)
		{
			// Provided output texture is actually the input into our custom post process texture.
			FScreenPassTexture InputTexture(Inputs.OutputTexture, Inputs.SceneViewRect);
			FScreenPassTexture OutputTexture(ActivePostBuffer.Texture);
	
			if (ActivePostBuffer.Proxy)
			{
				// If we are using the paint geometry instead, then override the input / output view rects.
				if (bUsePaintGeometry_Renderthread && !FMath::IsNearlyZero(PaintGeometry_Renderthread.GetLocalSize().X) && !FMath::IsNearlyZero(PaintGeometry_Renderthread.GetLocalSize().Y))
				{
					const FSlateRenderTransform& RenderTransform = PaintGeometry_Renderthread.GetAccumulatedRenderTransform();
					const FIntPoint DownscalePt(ActivePostBuffer.Downscale);
					const float InvDownscale = 1.f / float(ActivePostBuffer.Downscale);

					const FVector2f WorldTopLeft = (InvDownscale * TransformPoint(RenderTransform, FVector2f::ZeroVector)).RoundToVector();
					const FVector2f WorldBottomRight = (InvDownscale * TransformPoint(RenderTransform, FVector2f(PaintGeometry_Renderthread.GetLocalSize()))).RoundToVector();

					const FVector2f InputOffset = InvDownscale * FVector2f(InputTexture.ViewRect.Min.X, InputTexture.ViewRect.Min.Y);
					const FVector2f OutputOffset = InvDownscale * FVector2f(OutputTexture.ViewRect.Min.X, OutputTexture.ViewRect.Min.Y);

					const FIntPoint InputTextureTopLeft = DownscalePt * WorldTopLeft.IntPoint();
					const FIntPoint InputTextureBottomRight = DownscalePt * WorldBottomRight.IntPoint();
					InputTexture.ViewRect = FIntRect(InputTextureTopLeft, InputTextureBottomRight.ComponentMax(InputTextureTopLeft + ViewRectMinimumSize));

					// Subtract the input offset because the geometry world transform will have it implicitly added.
					const FIntPoint OutputTextureTopLeft = (WorldTopLeft - InputOffset + OutputOffset).IntPoint();
					const FIntPoint OutputTextureBottomRight = (WorldBottomRight - InputOffset + OutputOffset).IntPoint();
					OutputTexture.ViewRect = FIntRect(OutputTextureTopLeft, OutputTextureBottomRight.ComponentMax(OutputTextureTopLeft + ViewRectMinimumSize));
				}
				ActivePostBuffer.Proxy->PostProcess_Renderthread(GraphBuilder, InputTexture, OutputTexture);
			}
			else
			{
				AddSlatePostProcessCopy(GraphBuilder, InputTexture, OutputTexture);
			}
		}
	}

	for (const FActivePostBuffer& ActivePostBuffer : ActivePostBuffers)
	{
		if (ActivePostBuffer.Texture)
		{
			GraphBuilder.UseExternalAccessMode(ActivePostBuffer.Texture, ERHIAccess::SRVMask);
		}
	}
}

void FPostBufferUpdater::PostCustomElementAdded(FSlateElementBatcher& ElementBatcher) const
{
	ESlatePostRT PrevResourceUpdatingPostBuffers = ElementBatcher.GetResourceUpdatingPostBuffers();
	ElementBatcher.SetResourceUpdatingPostBuffers(PrevResourceUpdatingPostBuffers | BuffersToUpdate_Renderthread);

	// Give proxies a chance to update their renderthread values.
	if (const USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::Get())
	{
		for (ESlatePostRT SlatePostBufferBit : MakeFlagsRange(BuffersToUpdate_Renderthread))
		{
			if (!RendererSettings->GetSlatePostSetting(SlatePostBufferBit).bEnabled)
			{
				continue;
			}

			if (TSharedPtr<FSlateRHIPostBufferProcessorProxy> PostProcessorProxy = USlateFXSubsystem::GetPostProcessorProxy(SlatePostBufferBit))
			{
				PostProcessorProxy->OnUpdateValuesRenderThread();
			}
		}
	}
}

#endif // !UE_SERVER

/////////////////////////////////////////////////////
// SPostBufferUpdate

SLATE_IMPLEMENT_WIDGET(SPostBufferUpdate)
void SPostBufferUpdate::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

SPostBufferUpdate::SPostBufferUpdate()
	: bUsePaintGeometry(false)
	, BuffersToUpdate({})
	, PostBufferUpdater(nullptr)
{
}

void SPostBufferUpdate::Construct(const FArguments& InArgs)
{
#if !UE_SERVER
	bUsePaintGeometry = InArgs._bUsePaintGeometry;

	BuffersToUpdate = {};

	PostBufferUpdater = MakeShared<FPostBufferUpdater>();
	if (PostBufferUpdater)
	{
		// Safe to set RT values here on construct
		PostBufferUpdater->bUsePaintGeometry_Renderthread = bUsePaintGeometry;
	}
#endif // !UE_SERVER
}

void SPostBufferUpdate::SetUsePaintGeometry(bool bInUsePaintGeometry)
{
#if !UE_SERVER
	bUsePaintGeometry = bInUsePaintGeometry;

	if (PostBufferUpdater)
	{
		TWeakPtr<FPostBufferUpdater> TempWeakPostBufferUpdater = PostBufferUpdater;
		bool bTempUsePaintGeometry = bUsePaintGeometry;

		ENQUEUE_RENDER_COMMAND(FUpdateValuesRenderThreadFX_UsePaintGeometry)([TempWeakPostBufferUpdater, bTempUsePaintGeometry](FRHICommandListImmediate& RHICmdList)
		{
			if (TSharedPtr<FPostBufferUpdater> PinnedPostBufferUpdater = TempWeakPostBufferUpdater.Pin())
			{
				PinnedPostBufferUpdater->bUsePaintGeometry_Renderthread = bTempUsePaintGeometry;
			}
		});
	}
#endif // !UE_SERVER
}

void SPostBufferUpdate::SetBuffersToUpdate(const TArrayView<ESlatePostRT> InBuffersToUpdate)
{
#if !UE_SERVER
	BuffersToUpdate = InBuffersToUpdate;

	if (PostBufferUpdater && !PostBufferUpdater->bBuffersToUpdateInitialized)
	{
		PostBufferUpdater->BuffersToUpdate_Renderthread = ESlatePostRT::None;
		if (const USlateRHIRendererSettings* RendererSettings = USlateRHIRendererSettings::Get())
		{
			for (ESlatePostRT BufferToUpdate : BuffersToUpdate)
			{
				if (RendererSettings->GetSlatePostSetting(BufferToUpdate).bEnabled)
				{
					PostBufferUpdater->BuffersToUpdate_Renderthread |= BufferToUpdate;
				}
			}
		}

		PostBufferUpdater->bBuffersToUpdateInitialized = true;
	}
#endif // !UE_SERVER
}

void SPostBufferUpdate::SetProcessorUpdaters(TMap<ESlatePostRT, TSharedPtr<FSlatePostProcessorUpdaterProxy>> InProcessorUpdaters)
{
#if !UE_SERVER
	if (PostBufferUpdater)
	{
		PostBufferUpdater->ProcessorUpdaters = InProcessorUpdaters;
	}
#endif // !UE_SERVER
}

const TArrayView<const ESlatePostRT> SPostBufferUpdate::GetBuffersToUpdate() const
{
	return MakeArrayView(BuffersToUpdate);
}

UMG_API void SPostBufferUpdate::ReleasePostBufferUpdater()
{
#if !UE_SERVER
	// Copy the pointer onto a lambda to defer the final deletion to after any pending uses on the renderthread
	TSharedPtr<FPostBufferUpdater> ReleaseMe = PostBufferUpdater;
	ENQUEUE_RENDER_COMMAND(ReleaseCommand)([ReleaseMe](FRHICommandList& RHICmdList) mutable
	{
		ReleaseMe.Reset();
	});

	PostBufferUpdater.Reset();
#endif // !UE_SERVER
}

int32 SPostBufferUpdate::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
#if !UE_SERVER
	FSlateRect RenderBoundingRect = AllottedGeometry.GetRenderBoundingRect();
	FPaintGeometry PaintGeometry(RenderBoundingRect.GetTopLeft(), RenderBoundingRect.GetSize(), AllottedGeometry.GetAccumulatedLayoutTransform().GetScale());

	if (PostBufferUpdater && bUsePaintGeometry)
	{
		PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

		TWeakPtr<FPostBufferUpdater> TempWeakPostBufferUpdater = PostBufferUpdater;

		ENQUEUE_RENDER_COMMAND(FUpdateValuesRenderThreadFX_PaintGeometry)([TempWeakPostBufferUpdater, PaintGeometry](FRHICommandListImmediate& RHICmdList)
		{
			if (TSharedPtr<FPostBufferUpdater> PinnedPostBufferUpdater = TempWeakPostBufferUpdater.Pin())
			{
				PinnedPostBufferUpdater->PaintGeometry_Renderthread = PaintGeometry;
				PinnedPostBufferUpdater->bUsePaintGeometry_Renderthread = true;
			}
		});
	}

	FSlateDrawElement::MakeCustom(OutDrawElements, LayerId, PostBufferUpdater);
#endif // !UE_SERVER

	// Increment LayerId to ensure items afterwards are not processed
	return ++LayerId;
}

FVector2D SPostBufferUpdate::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return FVector2D::ZeroVector;
}

