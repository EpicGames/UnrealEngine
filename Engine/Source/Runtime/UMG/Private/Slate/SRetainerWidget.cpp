// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SRetainerWidget.h"
#include "Misc/App.h"
#include "UObject/Package.h"
#include "Framework/Application/SlateApplication.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RenderDeferredCleanup.h"
#include "RHI.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "UMGPrivate.h"
#include "Widgets/SNullWidget.h"

DECLARE_CYCLE_STAT(TEXT("Retainer Widget Tick"), STAT_SlateRetainerWidgetTick, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Retainer Widget Paint"), STAT_SlateRetainerWidgetPaint, STATGROUP_Slate);

#if !UE_BUILD_SHIPPING
FOnRetainedModeChanged SRetainerWidget::OnRetainerModeChangedDelegate;
#endif

static void HandleDesignerRetainedRenderingToggled(IConsoleVariable* CVar)
{
	FSlateApplication::Get().InvalidateAllWidgets(false);
}

/** True if we should do retained rendering in designer. */
bool GEnableDesignerRetainedRendering = true;
FAutoConsoleVariableRef EnableDesignerRetainedRendering(
	TEXT("Slate.EnableDesignerRetainedRendering"),
	GEnableDesignerRetainedRendering,
	TEXT("Controls if retainer renders in designer; 0 - Never; 1 - Per Widget Properties"),
	FConsoleVariableDelegate::CreateStatic(&HandleDesignerRetainedRenderingToggled)
);

/** True if we should allow widgets to be cached in the UI at all. */
int32 GEnableRetainedRendering = 1;
FAutoConsoleVariableRef EnableRetainedRendering(
	TEXT("Slate.EnableRetainedRendering"),
	GEnableRetainedRendering,
	TEXT("Whether to attempt to render things in SRetainerWidgets to render targets first.") 
);

bool GSlateEnableRetainedRenderingHittestRemap = true;
FAutoConsoleVariableRef CVarGlateEnableRetainedRenderingHittestRemap(
	TEXT("Slate.EnableRetainedRenderingHittestRemap"),
	GSlateEnableRetainedRenderingHittestRemap,
	TEXT("If enables, the rotation delta of the SRetainer widget will be applied to the hittest grid. Used to resolve an issue with layout and paint space deviations in retained widgets")
);


static bool IsRetainedRenderingEnabled()
{
	return GEnableRetainedRendering != 0;
}

/** Whether or not the platform should have deferred retainer widget render target updating enabled by default */
#define PLATFORM_REQUIRES_DEFERRED_RETAINER_UPDATE PLATFORM_IOS || PLATFORM_ANDROID;

/**
 * If this is true the retained rendering render thread work will happen during normal slate render thread rendering after the back buffer has been presented
 * in order to avoid extra render target switching in the middle of the frame. The downside is that the UI update will be a frame late
 */
int32 GDeferRetainedRenderingRenderThread = PLATFORM_REQUIRES_DEFERRED_RETAINER_UPDATE;
FAutoConsoleVariableRef DeferRetainedRenderingRT(
	TEXT("Slate.DeferRetainedRenderingRenderThread"),
	GDeferRetainedRenderingRenderThread,
	TEXT("Whether or not to defer retained rendering to happen at the same time as the rest of slate render thread work"));


class FRetainerWidgetRenderingResources : public FDeferredCleanupInterface, public FGCObject
{
public:
	FRetainerWidgetRenderingResources()
		: WidgetRenderer(nullptr)
		, RenderTarget(nullptr)
		, DynamicEffect(nullptr)
	{}

	~FRetainerWidgetRenderingResources()
	{
		// Note not using deferred cleanup for widget renderer here as it is already in deferred cleanup
		if (WidgetRenderer)
		{
			delete WidgetRenderer;
		}
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(RenderTarget);
		Collector.AddReferencedObject(DynamicEffect);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("FRetainerWidgetRenderingResources");
	}
	
public:
	FWidgetRenderer* WidgetRenderer;
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;
	TObjectPtr<UMaterialInstanceDynamic> DynamicEffect;
};

TArray<SRetainerWidget*, TInlineAllocator<3>> SRetainerWidget::Shared_WaitingToRender;
int32 SRetainerWidget::Shared_MaxRetainerWorkPerFrame(0);
TFrameValue<int32> SRetainerWidget::Shared_RetainerWorkThisFrame(0);


SRetainerWidget::SRetainerWidget()
	: PreviousRenderSize(FIntPoint::NoneValue)
	, PreviousClipRectSize(FIntPoint::NoneValue)
	, PreviousColorAndOpacity(FColor::Transparent)
	, LastIncomingLayerId(0)
	, VirtualWindow(SNew(SVirtualWindow))
	, HittestGrid(MakeShared<FHittestGrid>())
	, RenderingResources(new FRetainerWidgetRenderingResources)
{
	FSlateApplicationBase::Get().OnGlobalInvalidationToggled().AddRaw(this, &SRetainerWidget::OnGlobalInvalidationToggled);
	if (FSlateApplication::IsInitialized())
	{
#if !UE_BUILD_SHIPPING
		OnRetainerModeChangedDelegate.RemoveAll(this);
#endif
	}
	bHasCustomPrepass = true;
	SetInvalidationRootWidget(*this);
	SetInvalidationRootHittestGrid(HittestGrid.Get());
	SetCanTick(false);
}

SRetainerWidget::~SRetainerWidget()
{
	if( FSlateApplication::IsInitialized() )
	{
		FSlateApplicationBase::Get().OnGlobalInvalidationToggled().RemoveAll(this);
#if !UE_BUILD_SHIPPING
		OnRetainerModeChangedDelegate.RemoveAll( this );
#endif
	}

	// Begin deferred cleanup of rendering resources.  DO NOT delete here.  Will be deleted when safe
	BeginCleanup(RenderingResources);

	Shared_WaitingToRender.Remove(this);
}

void SRetainerWidget::UpdateWidgetRenderer()
{
	// We can't write out linear.  If we write out linear, then we end up with premultiplied alpha
	// in linear space, which blending with gamma space later is difficult...impossible? to get right
	// since the rest of slate does blending in gamma space.
	const bool bWriteContentInGammaSpace = true;

	if (!RenderingResources->WidgetRenderer)
	{
		RenderingResources->WidgetRenderer = new FWidgetRenderer(bWriteContentInGammaSpace);
	}

	UTextureRenderTarget2D* RenderTarget = RenderingResources->RenderTarget;
	FWidgetRenderer* WidgetRenderer = RenderingResources->WidgetRenderer;

	WidgetRenderer->SetUseGammaCorrection(bWriteContentInGammaSpace);

	// This will be handled by the main slate rendering pass
	WidgetRenderer->SetApplyColorDeficiencyCorrection(false);

	WidgetRenderer->SetIsPrepassNeeded(false);
	WidgetRenderer->SetClearHitTestGrid(false);

	// Update the render target to match the current gamma rendering preferences.
	if (RenderTarget && RenderTarget->SRGB != !bWriteContentInGammaSpace)
	{
		// Note, we do the opposite here of whatever write is, if we we're writing out gamma,
		// then sRGB writes were not supported, so it won't be an sRGB texture.
		RenderTarget->TargetGamma = !bWriteContentInGammaSpace ? 0.0f : 1.0;
		RenderTarget->SRGB = !bWriteContentInGammaSpace;

		RenderTarget->UpdateResource();
	}
}

void SRetainerWidget::Construct(const FArguments& InArgs)
{
	STAT(MyStatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Slate>(InArgs._StatId);)

	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>();
	RenderTarget->ClearColor = FLinearColor::Transparent;
	RenderTarget->RenderTargetFormat = RTF_RGBA8_SRGB;

	RenderingResources->RenderTarget = RenderTarget;
	SurfaceBrush.SetResourceObject(RenderTarget);

	VirtualWindow->SetVisibility(EVisibility::SelfHitTestInvisible);  // deubanks: We don't want Retainer Widgets blocking hit testing for tooltips
	VirtualWindow->SetShouldResolveDeferred(false);

	UpdateWidgetRenderer();

	MyWidget = InArgs._Content.Widget;

	RenderOnPhase = InArgs._RenderOnPhase;
	RenderOnInvalidation = InArgs._RenderOnInvalidation;

	Phase = InArgs._Phase;
	PhaseCount = InArgs._PhaseCount;

	LastDrawTime = FApp::GetCurrentTime();
	LastTickedFrame = 0;

	bEnableRetainedRenderingDesire = true;
	bEnableRetainedRendering = false;
	SetVolatilePrepass(bEnableRetainedRendering);

#if WITH_EDITOR
	bIsDesignTime = false;
	bShowEffectsInDesigner = true;
	bWarnOnInvalidSize = InArgs._bWarnOnInvalidSize;
#endif // WITH_EDITOR

	RefreshRenderingMode();
	bRenderRequested = true;
	bInvalidSizeLogged = false;

	ChildSlot
	[
		MyWidget.ToSharedRef()
	];

	if ( FSlateApplication::IsInitialized() )
	{
#if !UE_BUILD_SHIPPING
		OnRetainerModeChangedDelegate.AddRaw(this, &SRetainerWidget::OnRetainerModeChanged);

		static bool bStaticInit = false;

		if ( !bStaticInit )
		{
			bStaticInit = true;
			EnableRetainedRendering->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&SRetainerWidget::OnRetainerModeCVarChanged));
		}
#endif
	}
}

bool SRetainerWidget::ShouldBeRenderingOffscreen() const
{
	return bEnableRetainedRenderingDesire && IsRetainedRenderingEnabled();
}

bool SRetainerWidget::IsAnythingVisibleToRender() const
{
	return MyWidget.IsValid() && MyWidget->GetVisibility().IsVisible();
}

void SRetainerWidget::OnRetainerModeChanged()
{
	if (MyWidget.IsValid())
	{
		InvalidateChildRemovedFromTree(*MyWidget.Get());
	}

	// Invalidate myself
	Advanced_ResetInvalidation(true);

	// Invalidate my invalidation root, since all my children were once it's children
	// it needs to force a generation bump just like me.
	if (FSlateInvalidationRoot* MyInvalidationRoot = GetProxyHandle().GetInvalidationRootHandle().GetInvalidationRoot())
	{
		MyInvalidationRoot->Advanced_ResetInvalidation(true);
	}

	RefreshRenderingMode();

	bRenderRequested = true;
}

void SRetainerWidget::OnRootInvalidated()
{
	RequestRender();
}

#if !UE_BUILD_SHIPPING

void SRetainerWidget::OnRetainerModeCVarChanged( IConsoleVariable* CVar )
{
	OnRetainerModeChangedDelegate.Broadcast();
}

#endif

void SRetainerWidget::SetRetainedRendering(bool bRetainRendering)
{
	if (bEnableRetainedRenderingDesire != bRetainRendering)
	{
		bEnableRetainedRenderingDesire = bRetainRendering;
		OnRetainerModeChanged();
	}
}

void SRetainerWidget::RefreshRenderingMode()
{
	const bool bShouldBeRenderingOffscreen = ShouldBeRenderingOffscreen();

	if ( bEnableRetainedRendering != bShouldBeRenderingOffscreen )
	{
		bEnableRetainedRendering = bShouldBeRenderingOffscreen;
		SetVolatilePrepass(bEnableRetainedRendering);
		InvalidateRootChildOrder();
	}
}

void SRetainerWidget::SetContent(const TSharedRef< SWidget >& InContent)
{
	MyWidget = InContent;
	ChildSlot
	[
		InContent
	];
}

UMaterialInstanceDynamic* SRetainerWidget::GetEffectMaterial() const
{
	return RenderingResources->DynamicEffect;
}

void SRetainerWidget::SetEffectMaterial(UMaterialInterface* EffectMaterial)
{
	if ( EffectMaterial )
	{
		UMaterialInstanceDynamic* DynamicEffect = Cast<UMaterialInstanceDynamic>(EffectMaterial);
		if ( !DynamicEffect )
		{
			DynamicEffect = UMaterialInstanceDynamic::Create(EffectMaterial, GetTransientPackage());
		}
		RenderingResources->DynamicEffect = DynamicEffect;

		SurfaceBrush.SetResourceObject(RenderingResources->DynamicEffect);
	}
	else
	{
		RenderingResources->DynamicEffect = nullptr;
		SurfaceBrush.SetResourceObject(RenderingResources->RenderTarget);
	}

	UpdateWidgetRenderer();
}

void SRetainerWidget::SetTextureParameter(FName TextureParameter)
{
	DynamicEffectTextureParameter = TextureParameter;
}
 
void SRetainerWidget::SetWorld(UWorld* World)
{
	OuterWorld = World;
}

FChildren* SRetainerWidget::GetChildren()
{
	if (bEnableRetainedRendering)
	{
		return &FNoChildren::NoChildrenInstance;
	}
	else
	{
		return SCompoundWidget::GetChildren();
	}
}

#if WITH_SLATE_DEBUGGING
FChildren* SRetainerWidget::Debug_GetChildrenForReflector()
{
	return SCompoundWidget::GetChildren();
}
#endif

void SRetainerWidget::SetRenderingPhase(int32 InPhase, int32 InPhaseCount)
{
	Phase = InPhase;
	PhaseCount = InPhaseCount;
}

void SRetainerWidget::RequestRender()
{
	bRenderRequested = true;
	InvalidateRootChildOrder();
}

#if WITH_EDITOR
void SRetainerWidget::SetIsDesignTime(bool bInIsDesignTime)
{
	bIsDesignTime = bInIsDesignTime;
}

void SRetainerWidget::SetShowEffectsInDesigner(bool bInShowEffectsInDesigner)
{
	bShowEffectsInDesigner = bInShowEffectsInDesigner;
}
#endif // WITH_EDITOR

SRetainerWidget::EPaintRetainedContentResult SRetainerWidget::PaintRetainedContentImpl(const FSlateInvalidationContext& Context, const FGeometry& AllottedGeometry, int32 LayerId)
{
	if (RenderOnPhase)
	{
		if (LastTickedFrame != GFrameCounter && (GFrameCounter % PhaseCount) == Phase)
		{
			// If doing some phase based invalidation, just redraw everything again
			bRenderRequested = true;
			InvalidateRootLayout();
		}
	}

	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();
	FSlateRenderTransform AccumulatedRenderTransform = PaintGeometry.GetAccumulatedRenderTransform();
	const FVector2f RenderSize = FVector2f(PaintGeometry.GetLocalSize()) * AccumulatedRenderTransform.GetMatrix().GetScale().GetVector();
	const FIntPoint RoundedRenderSize = RenderSize.IntPoint();

	if (RenderOnInvalidation)
	{
		// the invalidation root will take care of whether or not we actually rendered
		bRenderRequested = true;

		const FIntPoint ClipRectSize = Context.CullingRect.GetSize().IntPoint();
		const TOptional<FSlateClippingState> ClippingState = Context.WindowElementList->GetClippingState();
		const FColor ColorAndOpacityTint = Context.WidgetStyle.GetColorAndOpacityTint().ToFColor(false);


		// Aggressively re-layout when a base state changes.
		if (RoundedRenderSize != PreviousRenderSize
			|| AllottedGeometry != PreviousAllottedGeometry
			|| AllottedGeometry.GetAccumulatedRenderTransform() != PreviousAllottedGeometry.GetAccumulatedRenderTransform()
			|| ClipRectSize != PreviousClipRectSize
			|| ClippingState != PreviousClippingState)
		{
			PreviousRenderSize = RoundedRenderSize;
			PreviousAllottedGeometry = AllottedGeometry;
			PreviousClipRectSize = ClipRectSize;
			PreviousClippingState = ClippingState;

			InvalidateRootLayout();
		}

		// Aggressively repaint when a base state changes.
		if (LayerId != LastIncomingLayerId
			|| ColorAndOpacityTint != PreviousColorAndOpacity)
		{
			LastIncomingLayerId = LayerId;
			PreviousColorAndOpacity = ColorAndOpacityTint;

			GetRootWidget()->Invalidate(EInvalidateWidgetReason::Paint);
		}
	}
	else if (RoundedRenderSize != PreviousRenderSize)
	{
		bRenderRequested = true;
		InvalidateRootLayout();
		PreviousRenderSize = RoundedRenderSize;
	}

	if (Shared_MaxRetainerWorkPerFrame > 0)
	{
		if (Shared_RetainerWorkThisFrame.TryGetValue(0) > Shared_MaxRetainerWorkPerFrame)
		{
			Shared_WaitingToRender.AddUnique(this);
			return EPaintRetainedContentResult::Queued;
		}
	}

	if (bRenderRequested)
	{
		// SRetainerWidget defines a relationship between a UWorld and a Slate hierarchy (VirtualWindow and its children).  Therefore it is
		// responsible for pushing its world's scene onto the stack, so that any child widgets that use Material Parameter Collection will resolve to
		// the MPC instance associated with that scene.
		UE::Slate::FSceneRegistrationScope SceneScope;
		UWorld* TickWorld = OuterWorld.Get();
		if (TickWorld && TickWorld->Scene && IsInGameThread())
		{
			SceneScope = FSlateApplication::Get().GetRenderer()->RegisterCurrentScene(TickWorld->Scene);
		}

		// Update the number of retainers we've drawn this frame.
		Shared_RetainerWorkThisFrame = Shared_RetainerWorkThisFrame.TryGetValue(0) + 1;

		LastTickedFrame = GFrameCounter;
		const double TimeSinceLastDraw = FApp::GetCurrentTime() - LastDrawTime;

		// Size must be a positive integer to allocate the RenderTarget
		const uint32 RenderTargetWidth  = FMath::RoundToInt(FMath::Abs(RenderSize.X));
		const uint32 RenderTargetHeight = FMath::RoundToInt(FMath::Abs(RenderSize.Y));
		const bool bTextureIsTooLarge = FMath::Max(RenderTargetWidth, RenderTargetHeight) > GetMax2DTextureDimension();
		const bool bTextureSizeZero = (RenderTargetWidth == 0 || RenderTargetHeight == 0);

		if (bTextureIsTooLarge || bTextureSizeZero)
		{
			// if bTextureTooLarge then the user probably have a layout issue. Warn the user.
			if (!bInvalidSizeLogged)
			{
				bInvalidSizeLogged = true;

				const bool bEnableWarnOnInvalidSize =
#if WITH_EDITOR
					bWarnOnInvalidSize;
#else
					true;
#endif
				if (bTextureIsTooLarge)
				{
					if (bEnableWarnOnInvalidSize)
					{
						UE_LOGF(LogUMG, Warning, "The requested size for SRetainerWidget is too large. W:%i H:%i", RenderTargetWidth, RenderTargetHeight);
					}
				}
				else
				{
					if (bEnableWarnOnInvalidSize)
					{
						UE_LOGF(LogUMG, Warning, "The requested size for SRetainerWidget is 0. W:%i H:%i", RenderTargetWidth, RenderTargetHeight);
					}
				}
			}
			return bTextureIsTooLarge ? EPaintRetainedContentResult::TextureSizeTooBig : EPaintRetainedContentResult::TextureSizeZero;
		}
		bInvalidSizeLogged = false;

		if (RenderTargetWidth >= 1 && RenderTargetHeight >= 1)
		{
			const FVector2D ViewOffset = PaintGeometry.GetAccumulatedRenderTransform().GetTranslation();

			UTextureRenderTarget2D* RenderTarget = RenderingResources->RenderTarget;
			FWidgetRenderer* WidgetRenderer = RenderingResources->WidgetRenderer;

			if ( MyWidget->GetVisibility().IsVisible() )
			{
				if ( (int32)RenderTarget->GetSurfaceWidth() != (int32)RenderTargetWidth ||
					 (int32)RenderTarget->GetSurfaceHeight() != (int32)RenderTargetHeight )
				{
					
					// If the render target resource already exists just resize it.  Calling InitCustomFormat flushes render commands which could result in a huge hitch
					if(RenderTarget->GameThread_GetRenderTargetResource() && RenderTarget->OverrideFormat == PF_B8G8R8A8)
					{
						RenderTarget->ResizeTarget(RenderTargetWidth, RenderTargetHeight);
					}
					else
					{
						const bool bForceLinearGamma = false;
						RenderTarget->InitCustomFormat(RenderTargetWidth, RenderTargetHeight, PF_B8G8R8A8, bForceLinearGamma);
						RenderTarget->UpdateResourceImmediate();
					}
				}

				const float Scale = AllottedGeometry.Scale;

				const FVector2D DrawSize = FVector2D(RenderTargetWidth, RenderTargetHeight);
				//const FGeometry WindowGeometry = FGeometry::MakeRoot(DrawSize * ( 1 / Scale ), FSlateLayoutTransform(Scale, PaintGeometry.DrawPosition));

				// Update the surface brush to match the latest size.
				SurfaceBrush.ImageSize = DrawSize;

				WidgetRenderer->ViewOffset = -ViewOffset;
				WidgetRenderer->SetIsPrepassNeeded(false);

				FVector2f WindowSize(RenderSize);
				SWindow* PaintWindow = Context.WindowElementList->GetPaintWindow();
				if (PaintWindow)
				{
					const FVector2f ViewportSize = PaintWindow->GetViewportSize();
					WindowSize.X = FMath::Max(WindowSize.X, ViewportSize.X);
					WindowSize.Y = FMath::Max(WindowSize.Y, ViewportSize.Y);
				}
				VirtualWindow->Resize(WindowSize);

				bool bRepaintedWidgets = WidgetRenderer->DrawInvalidationRoot(VirtualWindow, RenderTarget, *this, Context, GDeferRetainedRenderingRenderThread != 0);
				bRenderRequested = false;
				Shared_WaitingToRender.Remove(this);

				LastDrawTime = FApp::GetCurrentTime();

				return bRepaintedWidgets ? EPaintRetainedContentResult::Painted : EPaintRetainedContentResult::NotPainted;
			}
		}
	}

	return EPaintRetainedContentResult::NotPainted;
}

int32 SRetainerWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	STAT(FScopeCycleCounter PaintCycleCounter(MyStatId););

	SRetainerWidget* MutableThis = const_cast<SRetainerWidget*>(this);

	bool bShouldRetainRendering = bEnableRetainedRendering;

#if WITH_EDITOR
	bool bShouldSkipDesignerRendering = bIsDesignTime && (!bShowEffectsInDesigner || !GEnableDesignerRetainedRendering);
	bShouldRetainRendering = bEnableRetainedRendering && !bShouldSkipDesignerRendering;
#endif // WITH_EDITOR

	if (bShouldRetainRendering && IsAnythingVisibleToRender())
	{
		SCOPE_CYCLE_COUNTER(STAT_SlateRetainerWidgetPaint);

		// Copy hit test grid settings from the root
		const bool bHittestCleared = HittestGrid->SetHittestArea(Args.RootGrid.GetGridOrigin(), Args.RootGrid.GetGridSize(), Args.RootGrid.GetGridWindowOrigin());
		if (bHittestCleared)
		{
			MutableThis->RequestRender();
		}
		HittestGrid->SetOwner(this);
		HittestGrid->SetCullingRect(MyCullingRect);

		FPaintArgs NewArgs = Args.WithNewHitTestGrid(HittestGrid.Get());

		// Copy the current user index into the new grid since nested hittest grids should inherit their parents user id
		NewArgs.GetHittestGrid().SetUserIndex(Args.RootGrid.GetUserIndex());

		FSlateInvalidationContext Context(OutDrawElements, InWidgetStyle);
		Context.bParentEnabled = bParentEnabled;
		Context.bAllowFastPathUpdate = true;
		Context.LayoutScaleMultiplier = GetPrepassLayoutScaleMultiplier();
		Context.PaintArgs = &NewArgs;
		Context.IncomingLayerId = LayerId;
		Context.CullingRect = MyCullingRect;

		EPaintRetainedContentResult PaintResult = MutableThis->PaintRetainedContentImpl(Context, AllottedGeometry, LayerId);

#if WITH_SLATE_DEBUGGING
		if (PaintResult == EPaintRetainedContentResult::NotPainted
			|| PaintResult == EPaintRetainedContentResult::TextureSizeZero
			|| PaintResult == EPaintRetainedContentResult::TextureSizeTooBig)
		{
			MutableThis->SetLastPaintType(ESlateInvalidationPaintType::None);
		}
#endif

		if (PaintResult == EPaintRetainedContentResult::TextureSizeTooBig)
		{
			return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		}
		else if (PaintResult == EPaintRetainedContentResult::TextureSizeZero)
		{
			// Nothing was drawn, pass the caller's layer through unchanged.
			return LayerId;
		}
		else
		{
			UTextureRenderTarget2D* RenderTarget = RenderingResources->RenderTarget;
			check(RenderTarget);

			if (RenderTarget->GetSurfaceWidth() >= 1 && RenderTarget->GetSurfaceHeight() >= 1)
			{
				const FLinearColor ComputedColorAndOpacity(Context.WidgetStyle.GetColorAndOpacityTint() * GetColorAndOpacity() * SurfaceBrush.GetTint(Context.WidgetStyle));
				// Retainer widget uses pre-multiplied alpha, so pre-multiply the color by the alpha to respect opacity.
				const FLinearColor PremultipliedColorAndOpacity(ComputedColorAndOpacity * ComputedColorAndOpacity.A);

				FWidgetRenderer* WidgetRenderer = RenderingResources->WidgetRenderer;
				UMaterialInstanceDynamic* DynamicEffect = RenderingResources->DynamicEffect;

				const bool bDynamicMaterialInUse = (DynamicEffect != nullptr);
				if (bDynamicMaterialInUse)
				{
					DynamicEffect->SetTextureParameterValue(DynamicEffectTextureParameter, RenderTarget);
				}

				FSlateDrawElement::MakeBox(
					*Context.WindowElementList,
					Context.IncomingLayerId,
					AllottedGeometry.ToPaintGeometry(),
					&SurfaceBrush,
					// We always write out the content in gamma space, so when we render the final version we need to
					// render without gamma correction enabled.
					ESlateDrawEffect::PreMultipliedAlpha | ESlateDrawEffect::NoGamma,
					FLinearColor(PremultipliedColorAndOpacity.R, PremultipliedColorAndOpacity.G, PremultipliedColorAndOpacity.B, PremultipliedColorAndOpacity.A)
				);
			}

			const bool bInheritedHittestability = Args.GetInheritedHittestability();
			const bool bOutgoingHittestability = bInheritedHittestability && GetVisibility().AreChildrenHitTestVisible();

			// add our widgets to the root hit test grid
			if (bOutgoingHittestability)
			{
				Args.GetHittestGrid().AddGrid(HittestGrid);
			}

			return GetCachedMaxLayerId();
		}
	}
	else
	{
		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}
}

FVector2D SRetainerWidget::ComputeDesiredSize(float LayoutScaleMuliplier) const
{
	if ( bEnableRetainedRendering )
	{
		return MyWidget->GetDesiredSize();
	}
	else
	{
		return SCompoundWidget::ComputeDesiredSize(LayoutScaleMuliplier);
	}
}

void SRetainerWidget::OnGlobalInvalidationToggled(bool bGlobalInvalidationEnabled)
{
	InvalidateRootChildOrder();

	ClearAllFastPathData(true);
}

bool SRetainerWidget::CustomPrepass(float LayoutScaleMultiplier)
{
	if (bEnableRetainedRendering)
	{
		if (NeedsPrepass())
		{
			SetNeedsSlowPath(true);
		}
		ProcessInvalidation();
		if (NeedsSlowPath())
		{
			FChildren* Children = SCompoundWidget::GetChildren();
			Prepass_ChildLoop(LayoutScaleMultiplier, Children);
		}
		return false;
	}
	else
	{
		return true;
	}
}

TSharedRef<SWidget> SRetainerWidget::GetRootWidget()
{
	return bEnableRetainedRendering ? SCompoundWidget::GetChildren()->GetChildAt(0) : SNullWidget::NullWidget;
}

int32 SRetainerWidget::PaintSlowPath(const FSlateInvalidationContext& Context)
{
	HittestGrid->Clear();

	const FGeometry& OriginalPaintSpaceGeometry = GetPaintSpaceGeometry();
	FSlateRenderTransform SimplifiedRenderTransform(OriginalPaintSpaceGeometry.GetAccumulatedRenderTransform().GetMatrix().GetScale(), OriginalPaintSpaceGeometry.GetAccumulatedRenderTransform().GetTranslation());
	FTransform2D ScalePosTransform(OriginalPaintSpaceGeometry.Scale, OriginalPaintSpaceGeometry.AbsolutePosition);
	SimplifiedRenderTransform = Concatenate(SimplifiedRenderTransform, Inverse(ScalePosTransform));
	const FGeometry NewPaintSpaceGeometry = FGeometry::MakeRoot(OriginalPaintSpaceGeometry.GetLocalSize(), FSlateLayoutTransform(OriginalPaintSpaceGeometry.Scale, OriginalPaintSpaceGeometry.AbsolutePosition)).MakeChild(SimplifiedRenderTransform, FVector2D::ZeroVector);

	if (GSlateEnableRetainedRenderingHittestRemap)
	{
		// Get the delta from simplified render transform to the original (should contain just the missing rotation about retainer pivot)
		const FSlateRenderTransform NewToOriginalPaintSpace =
			Concatenate(
				Inverse(NewPaintSpaceGeometry.GetAccumulatedRenderTransform()),
				OriginalPaintSpaceGeometry.GetAccumulatedRenderTransform());

		HittestGrid->SetPaintToHitTestTransform(NewToOriginalPaintSpace);
	}
	else
	{
		HittestGrid->SetPaintToHitTestTransform(FSlateRenderTransform());
	}

	return SCompoundWidget::OnPaint(*Context.PaintArgs, NewPaintSpaceGeometry, Context.CullingRect, *Context.WindowElementList, Context.IncomingLayerId, Context.WidgetStyle, Context.bParentEnabled);
}
