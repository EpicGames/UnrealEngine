// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SWorldWidgetScreenLayer.h"
#include "Widgets/Layout/SBox.h"
#include "Components/WidgetComponent.h"
#include "Blueprint/WidgetLayoutLibrary.h"
#include "Blueprint/SlateBlueprintLibrary.h"
#include "Engine/GameViewportClient.h"
#include "Widgets/SViewport.h"
#include "SceneView.h"
#include "Slate/SGameLayerManager.h"

static int32 GSlateWorldWidgetZOrder = 1;
static FAutoConsoleVariableRef CVarSlateWorldWidgetZOrder(
	TEXT("Slate.WorldWidgetZOrder"),
	GSlateWorldWidgetZOrder,
	TEXT("Whether to re-order world widgets projected to screen by their view point distance\n")
	TEXT(" 0: Disable re-ordering\n")
	TEXT(" 1: Re-order by distance (default, less batching, less artifacts when widgets overlap)"),
	ECVF_Default
	);

static bool GSlateWorldWidgetIgnoreNotVisibleWidgets = false;
static FAutoConsoleVariableRef CVarSlateWorldWidgetIgnoreNotVisibleWidgets(
	TEXT("Slate.WorldWidgetIgnoreNotVisibleWidgets"),
	GSlateWorldWidgetIgnoreNotVisibleWidgets,
	TEXT("Whether to not update the position of world widgets if they are not visible - to prevent invalidating the whole layer unnecessarily")
	);

SWorldWidgetScreenLayer::FComponentEntry::FComponentEntry()
	: Slot(nullptr)
{
}

SWorldWidgetScreenLayer::FComponentEntry::~FComponentEntry()
{
	Widget.Reset();
	ContainerWidget.Reset();
}

void SWorldWidgetScreenLayer::Construct(const FArguments& InArgs, const FLocalPlayerContext& InPlayerContext)
{
	PlayerContext = InPlayerContext;

	bCanSupportFocus = false;
	DrawSize = FVector2D(0, 0);
	Pivot = FVector2D(0.5f, 0.5f);

	ChildSlot
	[
		SAssignNew(Canvas, SConstraintCanvas)
	];
}

void SWorldWidgetScreenLayer::SetWidgetDrawSize(FVector2D InDrawSize)
{
	DrawSize = InDrawSize;
}

void SWorldWidgetScreenLayer::SetWidgetPivot(FVector2D InPivot)
{
	Pivot = InPivot;
}

void SWorldWidgetScreenLayer::AddComponent(USceneComponent* Component, TSharedRef<SWidget> Widget)
{
	if ( Component )
	{
		FComponentEntry& Entry = ComponentMap.FindOrAdd(FObjectKey(Component));
		Entry.Component = Component;
		Entry.WidgetComponent = Cast<UWidgetComponent>(Component);
		Entry.Widget = Widget;

		Canvas->AddSlot()
		.Expose(Entry.Slot)
		[
			SAssignNew(Entry.ContainerWidget, SBox)
			[
				Widget
			]
		];
	}
}

void SWorldWidgetScreenLayer::RemoveComponent(USceneComponent* Component)
{
	if (ensure(Component))
	{
		if (FComponentEntry* EntryPtr = ComponentMap.Find(Component))
		{
			if (!EntryPtr->bRemoving)
			{
				RemoveEntryFromCanvas(*EntryPtr);
				ComponentMap.Remove(Component);
			}
		}
	}
}

void SWorldWidgetScreenLayer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(SWorldWidgetScreenLayer_Tick);

	if ( APlayerController* PlayerController = PlayerContext.GetPlayerController() )
	{
		if ( UGameViewportClient* ViewportClient = PlayerController->GetWorld()->GetGameViewport() )
		{
			const FGeometry& ViewportGeometry = ViewportClient->GetGameLayerManager()->GetViewportWidgetHostGeometry();

			// cache projection data here and avoid calls to UWidgetLayoutLibrary.ProjectWorldLocationToWidgetPositionWithDistance
			FSceneViewProjectionData ProjectionData;
			FMatrix ViewProjectionMatrix;
			bool bHasProjectionData = false;

			ULocalPlayer const* const LP = PlayerController->GetLocalPlayer();
			if (LP && LP->ViewportClient)
			{
				bHasProjectionData = LP->GetProjectionData(ViewportClient->Viewport, /*out*/ ProjectionData);
				if (bHasProjectionData)
				{
					ViewProjectionMatrix = ProjectionData.ComputeViewProjectionMatrix();
				}
			}

			for ( auto It = ComponentMap.CreateIterator(); It; ++It )
			{
				FComponentEntry& Entry = It.Value();

				if ( USceneComponent* SceneComponent = Entry.Component.Get() )
				{
					FVector WorldLocation = SceneComponent->GetComponentLocation();

					FVector2D ScreenPosition2D;
					const bool bProjected = [&Entry, bHasProjectionData, &WorldLocation, &ScreenPosition2D, &ProjectionData, &ViewProjectionMatrix]()
					{
						if (!bHasProjectionData)
						{
							return false;
						}

						if (GSlateWorldWidgetIgnoreNotVisibleWidgets && 
							Entry.WidgetComponent && 
							!Entry.WidgetComponent->IsWidgetVisible())
						{
							return false;
						}

						return FSceneView::ProjectWorldToScreen(WorldLocation, ProjectionData.GetConstrainedViewRect(), ViewProjectionMatrix, ScreenPosition2D);
					}();

					if (bProjected)
					{
						const double ViewportDist = FVector::Dist(ProjectionData.ViewOrigin, WorldLocation);
						const FVector2D RoundedPosition2D(FMath::RoundToDouble(ScreenPosition2D.X), FMath::RoundToDouble(ScreenPosition2D.Y));

						// If the root widget has pixel snapping disabled, then don't pixel snap the screen coordinates either otherwise
						// it'll always jump between pixels. This saves needing an explicit flag on the widget component, and is probably 
						// a better delegation of responsibility anyway, since changing the widget type can change the snapping as it wants
						bool bDisablePixelSnapping = Entry.Widget->GetPixelSnapping() == EWidgetPixelSnapping::Disabled;
						const FVector2D ScreenPositionToUse = bDisablePixelSnapping ? ScreenPosition2D : RoundedPosition2D;
						
						FVector2D ViewportPosition2D;
						USlateBlueprintLibrary::ScreenToViewport(PlayerController, ScreenPositionToUse, OUT ViewportPosition2D);

						const FVector ViewportPosition(ViewportPosition2D.X, ViewportPosition2D.Y, ViewportDist);

						Entry.ContainerWidget->SetVisibility(EVisibility::SelfHitTestInvisible);

						if ( SConstraintCanvas::FSlot* CanvasSlot = Entry.Slot )
						{
							FVector2D AbsoluteProjectedLocation = ViewportGeometry.LocalToAbsolute(FVector2D(ViewportPosition.X, ViewportPosition.Y));
							FVector2D LocalPosition = AllottedGeometry.AbsoluteToLocal(AbsoluteProjectedLocation);

							if ( Entry.WidgetComponent )
							{
								LocalPosition = Entry.WidgetComponent->ModifyProjectedLocalPosition(ViewportGeometry, LocalPosition);

								FVector2D ComponentDrawSize = Entry.WidgetComponent->GetDrawSize();
								FVector2D ComponentPivot = Entry.WidgetComponent->GetPivot();
								
								CanvasSlot->SetAutoSize(ComponentDrawSize.IsZero() || Entry.WidgetComponent->GetDrawAtDesiredSize());
								CanvasSlot->SetOffset(FMargin(LocalPosition.X, LocalPosition.Y, ComponentDrawSize.X, ComponentDrawSize.Y));
								CanvasSlot->SetAnchors(FAnchors(0, 0, 0, 0));
								CanvasSlot->SetAlignment(ComponentPivot);
								
								if (GSlateWorldWidgetZOrder != 0)
								{
									CanvasSlot->SetZOrder(static_cast<float>(- ViewportPosition.Z));
								}
							}
							else
							{
								CanvasSlot->SetAutoSize(DrawSize.IsZero());
								CanvasSlot->SetOffset(FMargin(LocalPosition.X, LocalPosition.Y, DrawSize.X, DrawSize.Y));
								CanvasSlot->SetAnchors(FAnchors(0, 0, 0, 0));
								CanvasSlot->SetAlignment(Pivot);

								if (GSlateWorldWidgetZOrder != 0)
								{
									CanvasSlot->SetZOrder(static_cast<float>( - ViewportPosition.Z));
								}
							}
						}
					}
					else
					{
						Entry.ContainerWidget->SetVisibility(EVisibility::Collapsed);
					}
				}
				else
				{
					RemoveEntryFromCanvas(Entry);
					It.RemoveCurrent();
					continue;
				}
			}

			// Done
			return;
		}
	}

	if (GSlateIsOnFastUpdatePath)
	{
		// Hide everything if we are unable to do any of the work.
		for (auto It = ComponentMap.CreateIterator(); It; ++It)
		{
			FComponentEntry& Entry = It.Value();
			Entry.ContainerWidget->SetVisibility(EVisibility::Collapsed);
		}
	}
}

void SWorldWidgetScreenLayer::RemoveEntryFromCanvas(SWorldWidgetScreenLayer::FComponentEntry& Entry)
{
	// Mark the component was being removed, so we ignore any other remove requests for this component.
	Entry.bRemoving = true;

	if (TSharedPtr<SWidget> ContainerWidget = Entry.ContainerWidget)
	{
		Canvas->RemoveSlot(ContainerWidget.ToSharedRef());
	}
}

FVector2D SWorldWidgetScreenLayer::ComputeDesiredSize(float) const
{
	return FVector2D(0, 0);
}
