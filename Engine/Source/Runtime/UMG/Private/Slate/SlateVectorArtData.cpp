// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SlateVectorArtData.h"

#include "RawIndexBuffer.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UMGPrivate.h"
#include "StaticMeshResources.h"
#include "Engine/StaticMesh.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateVectorArtData)

static void StaticMeshToSlateRenderData(const UStaticMesh& DataSource, TArray<FSlateMeshVertex>& OutSlateVerts, TArray<uint32>& OutIndexes, FVector2D& OutExtentMin, FVector2D& OutExtentMax )
{
	OutExtentMin = FVector2D(FLT_MAX, FLT_MAX);
	OutExtentMax = FVector2D(-FLT_MAX, -FLT_MAX);

	const FStaticMeshLODResources& LOD = DataSource.GetRenderData()->LODResources[0];
	const int32 NumSections = LOD.Sections.Num();
	if (NumSections > 1)
	{
		UE_LOGF(LogUMG, Warning, "StaticMesh %ls has %d sections. SMeshWidget expects a static mesh with 1 section.", *DataSource.GetName(), NumSections);
	}
	else
	{
		// Populate Vertex Data
		{
			const uint32 NumVerts = LOD.VertexBuffers.PositionVertexBuffer.GetNumVertices();
			OutSlateVerts.Empty();
			OutSlateVerts.Reserve(NumVerts);

			static const int32 MAX_SUPPORTED_UV_SETS = 6;
			const int32 TexCoordsPerVertex = LOD.GetNumTexCoords();
			if (TexCoordsPerVertex > MAX_SUPPORTED_UV_SETS)
			{
				UE_LOGF(LogStaticMesh, Warning, "[%ls] has %d UV sets; slate vertex data supports at most %d", *DataSource.GetName(), TexCoordsPerVertex, MAX_SUPPORTED_UV_SETS);
			}

			for (uint32 i = 0; i < NumVerts; ++i)
			{
				// Copy Position
				const FVector3f& Position = LOD.VertexBuffers.PositionVertexBuffer.VertexPosition(i);
				OutExtentMin.X = FMath::Min(Position.X, OutExtentMin.X);
				OutExtentMin.Y = FMath::Min(Position.Y, OutExtentMin.Y);
				OutExtentMax.X = FMath::Max(Position.X, OutExtentMax.X);
				OutExtentMax.Y = FMath::Max(Position.Y, OutExtentMax.Y);
				
				// Copy Color
				FColor Color = (LOD.VertexBuffers.ColorVertexBuffer.GetNumVertices() > 0) ? LOD.VertexBuffers.ColorVertexBuffer.VertexColor(i) : FColor::White;
				
				// Copy all the UVs that we have, and as many as we can fit.
				const FVector2f& UV0 = (TexCoordsPerVertex > 0) ? LOD.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 0) : FVector2f(1, 1);

				const FVector2f& UV1 = (TexCoordsPerVertex > 1) ? LOD.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 1) : FVector2f(1, 1);

				const FVector2f& UV2 = (TexCoordsPerVertex > 2) ? LOD.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 2) : FVector2f(1, 1);

				const FVector2f& UV3 = (TexCoordsPerVertex > 3) ? LOD.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 3) : FVector2f(1, 1);

				const FVector2f& UV4 = (TexCoordsPerVertex > 4) ? LOD.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 4) : FVector2f(1, 1);

				const FVector2f& UV5 = (TexCoordsPerVertex > 5) ? LOD.VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(i, 5) : FVector2f(1, 1);

				OutSlateVerts.Add(FSlateMeshVertex(
					FVector2f(Position.X, Position.Y),
					Color,
					UV0,
					UV1,
					UV2,
					UV3,
					UV4,
					UV5
				));
			}
		}

		// Populate Index data
		{
			FIndexArrayView SourceIndexes = LOD.IndexBuffer.GetArrayView();
			const int32 NumIndexes = SourceIndexes.Num();
			OutIndexes.Empty();
			OutIndexes.Reserve(NumIndexes);
			for (int32 i = 0; i < NumIndexes; ++i)
			{
				OutIndexes.Add(SourceIndexes[i]);
			}


			// Sort the index buffer such that verts are drawn in Z-order.
			// Assume that all triangles are coplanar with Z == SomeValue.
			ensure(NumIndexes % 3 == 0);
			for (int32 a = 0; a < NumIndexes; a += 3)
			{
				for (int32 b = 0; b < NumIndexes; b += 3)
				{
					const float VertADepth = LOD.VertexBuffers.PositionVertexBuffer.VertexPosition(OutIndexes[a]).Z;
					const float VertBDepth = LOD.VertexBuffers.PositionVertexBuffer.VertexPosition(OutIndexes[b]).Z;
					if ( VertADepth < VertBDepth )
					{
						// Swap the order in which triangles will be drawn
						Swap(OutIndexes[a + 0], OutIndexes[b + 0]);
						Swap(OutIndexes[a + 1], OutIndexes[b + 1]);
						Swap(OutIndexes[a + 2], OutIndexes[b + 2]);
					}
				}
			}
		}
	}
}


USlateVectorArtData::USlateVectorArtData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

const TArray<FSlateMeshVertex>& USlateVectorArtData::GetVertexData() const
{
	return VertexData;
}

const TArray<uint32>& USlateVectorArtData::GetIndexData() const
{
	return IndexData;
}

UMaterialInterface* USlateVectorArtData::GetMaterial() const
{
	return Material;
}

UMaterialInstanceDynamic* USlateVectorArtData::ConvertToMaterialInstanceDynamic()
{
	EnsureValidData();
	UMaterialInstanceDynamic* ExistingMID = Cast<UMaterialInstanceDynamic>(Material);
	if (ExistingMID == nullptr)
	{
		UMaterialInstanceDynamic* NewMID = UMaterialInstanceDynamic::Create(Material, this);
		Material = NewMID;
		return NewMID;
	}
	else
	{
		return ExistingMID;
	}
}

void USlateVectorArtData::EnsureValidData()
{
#if WITH_EDITORONLY_DATA
	if (MeshAsset)
	{
		InitFromStaticMesh(*MeshAsset);
	}
	else
	{
		SourceMaterial = nullptr;
		VertexData.Reset();
		IndexData.Reset();
		Material = nullptr;
		ExtentMin = FVector2D(FLT_MAX, FLT_MAX);
		ExtentMax = FVector2D(-FLT_MAX, -FLT_MAX);
	}
#endif
}

void USlateVectorArtData::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);
	EnsureValidData();
}

#if WITH_EDITORONLY_DATA
void USlateVectorArtData::InitFromStaticMesh(const UStaticMesh& InSourceMesh)
{
	if (SourceMaterial != InSourceMesh.GetMaterial(0))
	{
		SourceMaterial = InSourceMesh.GetMaterial(0);
		Material = SourceMaterial;
	}

	ensureMsgf(Material != nullptr, TEXT("USlateVectorArtData::InitFromStaticMesh() expected %s to have a material assigned."), *InSourceMesh.GetFullName());

	StaticMeshToSlateRenderData(InSourceMesh, VertexData, IndexData, ExtentMin, ExtentMax);
}
#endif

FVector2D USlateVectorArtData::GetDesiredSize() const
{
	return GetExtentMax() - GetExtentMin();
}

FVector2D USlateVectorArtData::GetExtentMin() const
{
	return ExtentMin;
}

FVector2D USlateVectorArtData::GetExtentMax() const
{
	return ExtentMax;
}

