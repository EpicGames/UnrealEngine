// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SlateVectorArtInstanceData.h"

void FSlateVectorArtInstanceData::SetPositionFixedPoint16(FVector2D Position)
{
	// We are effectively doing fixed point positioning
	// With 13 bits for the whole part and 3 bits of decimal precision.
	// xxxxxxxxxxxxx.xxx
	//     [0..8191].[0..7]
	{
		uint32 CurrentDataX = static_cast<uint32>(GetData().X) & 0xFFFF0000;
		GetData().X = CurrentDataX | FMath::RoundToInt32(Position.X*8.0f) << 0;
	}

	{
		uint32 CurrentDataY = static_cast<uint32>(GetData().Y) & 0xFFFF0000;
		GetData().Y = CurrentDataY | FMath::RoundToInt32(Position.Y*8.0f) << 0;
	}
}

void FSlateVectorArtInstanceData::SetScaleFixedPoint16(float Scale)
{
	// We need some fractional precision, so use 8 bits whole and 8 bits fractional.
	//     [0..256].[0..256]
	uint32 CurrentDataZ = static_cast<uint32>(GetData().Z) & 0xFFFF0000;
	GetData().Z = CurrentDataZ | FMath::RoundToInt(Scale*256.0f) << 0;
}

void FSlateVectorArtInstanceData::SetPosition(FVector2D Position)
{
	GetData().X = Position.X;
	GetData().Y = Position.Y;
}

void FSlateVectorArtInstanceData::SetScale(float Scale)
{
	GetData().Z = Scale;
}

void FSlateVectorArtInstanceData::SetBaseAddress(float Address)
{
	GetData().W = Address;
}
