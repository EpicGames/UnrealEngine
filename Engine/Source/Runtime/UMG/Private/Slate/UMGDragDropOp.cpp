// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/UMGDragDropOp.h"
#include "Application/SlateApplicationBase.h"
#include "Engine/GameViewportClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Blueprint/DragDropOperation.h"
#include "Slate/SObjectWidget.h"

#include "Widgets/Layout/SDPIScaler.h"


//////////////////////////////////////////////////////////////////////////
// FUMGDragDropOp

FUMGDragDropOp::FUMGDragDropOp()
	: DragOperation(nullptr)
	, GameViewport(nullptr)
{
	StartTime = FSlateApplicationBase::Get().GetCurrentTime();
}

void FUMGDragDropOp::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(DragOperation);
}

FString FUMGDragDropOp::GetReferencerName() const
{
	return TEXT("FUMGDragDropOp");
}

void FUMGDragDropOp::Construct()
{

}

bool FUMGDragDropOp::AffectedByPointerEvent(const FPointerEvent& PointerEvent)
{
	return DragOperation && PointerEvent.GetPointerIndex() == PointerIndex;
}

void FUMGDragDropOp::OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent )
{
	if ( DragOperation )
	{
		if ( bDropWasHandled && MouseEvent.GetPointerIndex() == PointerIndex )
		{
			DragOperation->Drop(MouseEvent);
		}
		else
		{
			if ( TSharedPtr<SObjectWidget> SourceUserWidgetPtr = SourceUserWidget.Pin() )
			{
				SourceUserWidgetPtr->OnDragCancelled(FDragDropEvent(MouseEvent, AsShared()), DragOperation);
			}

			DragOperation->DragCancelled(MouseEvent);
		}
	}
	
	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);
}

void FUMGDragDropOp::OnDragged( const class FDragDropEvent& DragDropEvent )
{
	if ( DragOperation && DragDropEvent.GetPointerIndex() == PointerIndex)
	{
		DragOperation->Dragged(DragDropEvent);

		FVector2D CachedDesiredSize = DecoratorWidget->GetDesiredSize();
	
		FVector2D Position = DragDropEvent.GetScreenSpacePosition();
		Position += CachedDesiredSize * DragOperation->Offset;
	
		switch ( DragOperation->Pivot )
		{
		case EDragPivot::MouseDown:
			Position += MouseDownOffset;
			break;
	
		case EDragPivot::TopLeft:
			// Position is already Top Left.
			break;
		case EDragPivot::TopCenter:
			Position -= CachedDesiredSize * FVector2D(0.5f, 0);
			break;
		case EDragPivot::TopRight:
			Position -= CachedDesiredSize * FVector2D(1, 0);
			break;
	
		case EDragPivot::CenterLeft:
			Position -= CachedDesiredSize * FVector2D(0, 0.5f);
			break;
		case EDragPivot::CenterCenter:
			Position -= CachedDesiredSize * FVector2D(0.5f, 0.5f);
			break;
		case EDragPivot::CenterRight:
			Position -= CachedDesiredSize * FVector2D(1.0f, 0.5f);
			break;
	
		case EDragPivot::BottomLeft:
			Position -= CachedDesiredSize * FVector2D(0, 1);
			break;
		case EDragPivot::BottomCenter:
			Position -= CachedDesiredSize * FVector2D(0.5f, 1);
			break;
		case EDragPivot::BottomRight:
			Position -= CachedDesiredSize * FVector2D(1, 1);
			break;
		}
	
		const double AnimationTime = 0.150;
	
		double DeltaTime = FSlateApplicationBase::Get().GetCurrentTime() - StartTime;
	
		if ( DeltaTime < AnimationTime )
		{
			double T = DeltaTime / AnimationTime;
			FVector2D LerpPosition = ( Position - StartingScreenPos ) * T;
			
			DecoratorPosition = StartingScreenPos + LerpPosition;
		}
		else
		{
			DecoratorPosition = Position;
		}
	}
}

TSharedPtr<SWidget> FUMGDragDropOp::GetDefaultDecorator() const
{
	return DecoratorWidget;
}

FCursorReply FUMGDragDropOp::OnCursorQuery()
{
	FCursorReply CursorReply = FGameDragDropOperation::OnCursorQuery();

	if ( !CursorReply.IsEventHandled() )
	{
		CursorReply = CursorReply.Cursor(EMouseCursor::Default);
	}

	if ( UGameViewportClient* GameViewportPtr = GameViewport.Get() )
	{
		TOptional<TSharedRef<SWidget>> CursorWidget = GameViewportPtr->MapCursor(nullptr, CursorReply);
		if ( CursorWidget.IsSet() )
		{
			CursorReply.SetCursorWidget(GameViewportPtr->GetWindow(), CursorWidget.GetValue());
		}
	}

	if (TSharedPtr<SObjectWidget> SourceUserWidgetObj = SourceUserWidget.Pin())
	{
		if (UUserWidget* SourceUserWidgetPtr = SourceUserWidgetObj->GetWidgetObject())
		{
			if (SourceUserWidgetPtr->bOverride_Cursor)
			{
				CursorReply = CursorReply.Cursor(SourceUserWidgetPtr->GetCursor());
			}
		}
	}

	return CursorReply;
}

TSharedRef<FUMGDragDropOp> FUMGDragDropOp::New(UDragDropOperation* InOperation, const int32 PointerIndex, const FVector2D &PointerPosition, const FVector2D &ScreenPositionOfDragee, float DPIScale, TSharedPtr<SObjectWidget> SourceUserWidget)
{
	check(InOperation);
	check(SourceUserWidget);

	TSharedRef<FUMGDragDropOp> Operation = MakeShareable(new FUMGDragDropOp());
	Operation->PointerIndex = PointerIndex;
	Operation->MouseDownOffset = ScreenPositionOfDragee - PointerPosition;
	Operation->StartingScreenPos = ScreenPositionOfDragee;
	Operation->SourceUserWidget = SourceUserWidget;
	Operation->GameViewport = SourceUserWidget->GetWidgetObject()->GetWorld()->GetGameViewport();
	Operation->DragOperation = InOperation;

	TSharedPtr<SWidget> DragVisual;
	if ( InOperation->DefaultDragVisual == nullptr )
	{
		DragVisual = SNew(STextBlock)
			.Text(FText::FromString(InOperation->Tag));
	}
	else
	{
		//TODO Make sure users are not trying to add a widget that already exists elsewhere.
		DragVisual = InOperation->DefaultDragVisual->TakeWidget();
	}

	Operation->DecoratorWidget =
		SNew(SDPIScaler)
		.DPIScale(DPIScale)
		[
			DragVisual.ToSharedRef()
		];

	Operation->DecoratorWidget->SlatePrepass();

	Operation->Construct();

	return Operation;
}
