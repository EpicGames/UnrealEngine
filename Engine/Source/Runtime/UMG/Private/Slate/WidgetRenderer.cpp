// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/WidgetRenderer.h"
#include "TextureResource.h"
#include "Input/HittestGrid.h"
#include "Layout/ArrangedChildren.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SlateDrawBuffer.h"
#include "TextureResource.h"
#include "Engine/TextureRenderTarget2D.h"

#if !UE_SERVER
#include "Interfaces/ISlateRHIRendererModule.h"
#endif // !UE_SERVER

#include "Widgets/LayerManager/STooltipPresenter.h"
#include "Widgets/Layout/SPopup.h"
#include "Framework/Application/SlateApplication.h"

FWidgetRenderer::FWidgetRenderer(bool bUseGammaCorrection, bool bInClearTarget)
	: bPrepassNeeded(true)
	, bClearHitTestGrid(true)
	, bUseGammaSpace(bUseGammaCorrection)
	, bClearTarget(bInClearTarget)
	, ViewOffset(0.0f, 0.0f)
{
#if !UE_SERVER
	if ( LIKELY(FApp::CanEverRender()) )
	{
		Renderer = FModuleManager::Get().LoadModuleChecked<ISlateRHIRendererModule>("SlateRHIRenderer").CreateSlate3DRenderer(bUseGammaSpace);
	}
#endif
}

FWidgetRenderer::~FWidgetRenderer()
{
}

ISlate3DRenderer* FWidgetRenderer::GetSlateRenderer()
{
	return Renderer.Get();
}

void FWidgetRenderer::SetUseGammaCorrection(bool bInUseGammaSpace)
{
	bUseGammaSpace = bInUseGammaSpace;

#if !UE_SERVER
	if (LIKELY(FApp::CanEverRender()))
	{
		Renderer->SetUseGammaCorrection(bInUseGammaSpace);
	}
#endif
}

void FWidgetRenderer::SetApplyColorDeficiencyCorrection(bool bInApplyColorCorrection)
{
#if !UE_SERVER
	if (LIKELY(FApp::CanEverRender()))
	{
		Renderer->SetApplyColorDeficiencyCorrection(bInApplyColorCorrection);
	}
#endif
}

UTextureRenderTarget2D* FWidgetRenderer::DrawWidget(const TSharedRef<SWidget>& Widget, FVector2D DrawSize)
{
	if ( LIKELY(FApp::CanEverRender()) )
	{
		UTextureRenderTarget2D* RenderTarget = FWidgetRenderer::CreateTargetFor(DrawSize, TF_Bilinear, bUseGammaSpace);

		DrawWidget(RenderTarget, Widget, DrawSize, 0, false);

		return RenderTarget;
	}

	return nullptr;
}

UTextureRenderTarget2D* FWidgetRenderer::CreateTargetFor(FVector2D DrawSize, TextureFilter InFilter, bool bUseGammaCorrection)
{
	if ( LIKELY(FApp::CanEverRender()) )
	{
		const bool bIsLinearSpace = !bUseGammaCorrection;
		const EPixelFormat requestedFormat = FSlateApplication::Get().GetRenderer()->GetSlateRecommendedColorFormat();

		UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>();
		RenderTarget->Filter = InFilter;
		RenderTarget->ClearColor = FLinearColor::Transparent;
		RenderTarget->SRGB = bIsLinearSpace;
		RenderTarget->TargetGamma = 1;
		RenderTarget->InitCustomFormat(FMath::FloorToInt32(DrawSize.X), FMath::FloorToInt32(DrawSize.Y), requestedFormat, bIsLinearSpace);
		RenderTarget->UpdateResourceImmediate(true);

		return RenderTarget;
	}

	return nullptr;
}

void FWidgetRenderer::DrawWidget(FRenderTarget* RenderTarget, const TSharedRef<SWidget>& Widget, FVector2D DrawSize, float DeltaTime, bool bDeferRenderTargetUpdate)
{
	DrawWidget(RenderTarget, Widget, 1.f, DrawSize, DeltaTime, bDeferRenderTargetUpdate);
}

void FWidgetRenderer::DrawWidget(UTextureRenderTarget2D* RenderTarget, const TSharedRef<SWidget>& Widget, FVector2D DrawSize, float DeltaTime, bool bDeferRenderTargetUpdate)
{
	DrawWidget(RenderTarget->GameThread_GetRenderTargetResource(), Widget, DrawSize, DeltaTime, bDeferRenderTargetUpdate);
}

void FWidgetRenderer::DrawWidget(
	FRenderTarget* RenderTarget,
	const TSharedRef<SWidget>& Widget,
	float Scale,
	FVector2D DrawSize,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
	TSharedRef<SVirtualWindow> Window = SNew(SVirtualWindow).Size(DrawSize);
	TUniquePtr<FHittestGrid> HitTestGrid = MakeUnique<FHittestGrid>();

	TSharedPtr<SWidget> OldParent = Widget->GetParentWidget();
	Window->SetContent(Widget);
	Window->Resize(DrawSize);

	DrawWindow(RenderTarget, *HitTestGrid, Window, Scale, DrawSize, DeltaTime, bDeferRenderTargetUpdate);

	// Calling Window->SetContent will set Widget's parent to the Window, so we need to reset to the original parent.
	// A better solution would be to have a way to render widgets without assigning them to a widget first.
	if (OldParent.IsValid())
	{
		Widget->AssignParentWidget(OldParent);
	}
}

void FWidgetRenderer::DrawWidget(
	UTextureRenderTarget2D* RenderTarget,
	const TSharedRef<SWidget>& Widget,
	float Scale,
	FVector2D DrawSize,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
	DrawWidget(RenderTarget->GameThread_GetRenderTargetResource(), Widget, Scale, DrawSize, DeltaTime, bDeferRenderTargetUpdate);
}

void FWidgetRenderer::DrawWindow(
	FRenderTarget* RenderTarget,
	FHittestGrid& HitTestGrid,
	TSharedRef<SWindow> Window,
	float Scale,
	FVector2D DrawSize,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
	FGeometry WindowGeometry = FGeometry::MakeRoot(DrawSize * (1.0f / Scale), FSlateLayoutTransform(Scale));

	DrawWindow
	(
		RenderTarget,
		HitTestGrid,
		Window,
		WindowGeometry,
		WindowGeometry.GetLayoutBoundingRect(),
		DeltaTime,
		bDeferRenderTargetUpdate
	);
}

void FWidgetRenderer::DrawWindow(
	UTextureRenderTarget2D* RenderTarget,
	FHittestGrid& HitTestGrid,
	TSharedRef<SWindow> Window,
	float Scale,
	FVector2D DrawSize,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
	DrawWindow(RenderTarget->GameThread_GetRenderTargetResource(), HitTestGrid, Window, Scale, DrawSize, DeltaTime, bDeferRenderTargetUpdate);
}

void FWidgetRenderer::DrawWindow(
	FRenderTarget* RenderTarget,
	FHittestGrid& HitTestGrid,
	TSharedRef<SWindow> Window,
	FGeometry WindowGeometry,
	FSlateRect WindowClipRect,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
	FPaintArgs PaintArgs(nullptr, HitTestGrid, FVector2D::ZeroVector, FApp::GetCurrentTime(), DeltaTime);
	DrawWindow(PaintArgs, RenderTarget, Window, WindowGeometry, WindowClipRect, DeltaTime, bDeferRenderTargetUpdate);
}

void FWidgetRenderer::DrawWindow(
	UTextureRenderTarget2D* RenderTarget,
	FHittestGrid& HitTestGrid,
	TSharedRef<SWindow> Window,
	FGeometry WindowGeometry,
	FSlateRect WindowClipRect,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
	DrawWindow(RenderTarget->GameThread_GetRenderTargetResource(), HitTestGrid, Window, WindowGeometry, WindowClipRect, DeltaTime, bDeferRenderTargetUpdate);
}

void FWidgetRenderer::DrawWindow(
	const FPaintArgs& PaintArgs,
	FRenderTarget* RenderTarget,
	TSharedRef<SWindow> Window,
	FGeometry WindowGeometry,
	FSlateRect WindowClipRect,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
#if !UE_SERVER
	FSlateRenderer* MainSlateRenderer = FSlateApplication::Get().GetRenderer();
	FScopeLock ScopeLock(MainSlateRenderer->GetResourceCriticalSection());

	if (LIKELY(FApp::CanEverRender()))
	{
		if (bPrepassNeeded)
		{
			// Ticking can cause geometry changes.  Recompute
			Window->SlatePrepass(WindowGeometry.Scale);
		}

		PaintArgs.GetHittestGrid().SetHittestArea(WindowClipRect.GetTopLeft(), WindowClipRect.GetSize());

		if (bClearHitTestGrid)
		{
			// Prepare the test grid 
			PaintArgs.GetHittestGrid().Clear();
		}

		{
			// Get the free buffer & add our virtual window
			ISlate3DRenderer::FScopedAcquireDrawBuffer ScopedDrawBuffer{ *Renderer, bDeferRenderTargetUpdate };
			FSlateWindowElementList& WindowElementList = ScopedDrawBuffer.GetDrawBuffer().AddWindowElementList(Window);

			// Paint the window
			int32 MaxLayerId = Window->Paint(
				PaintArgs,
				WindowGeometry, WindowClipRect,
				WindowElementList,
				0,
				FWidgetStyle(),
				Window->IsEnabled());

			//MaxLayerId = WindowElementList.PaintDeferred(MaxLayerId);
			DeferredPaints = WindowElementList.GetDeferredPaintList();

			Renderer->DrawWindow_GameThread(ScopedDrawBuffer.GetDrawBuffer());

			ScopedDrawBuffer.GetDrawBuffer().ViewOffset = ViewOffset;

			FRenderThreadUpdateContext RenderThreadUpdateContext =
			{
				&(ScopedDrawBuffer.GetDrawBuffer()),
				(FApp::GetCurrentTime() - GStartTime),
				static_cast<float>(FApp::GetDeltaTime()),
				(FPlatformTime::Seconds() - GStartTime),
				static_cast<float>(FApp::GetDeltaTime()),
				RenderTarget,
				Renderer.Get(),
				bClearTarget
			};

			MainSlateRenderer->AddWidgetRendererUpdate(RenderThreadUpdateContext, bDeferRenderTargetUpdate);
		}
	}
#endif // !UE_SERVER
}

void FWidgetRenderer::DrawWindow(
	const FPaintArgs& PaintArgs,
	UTextureRenderTarget2D* RenderTarget,
	TSharedRef<SWindow> Window,
	FGeometry WindowGeometry,
	FSlateRect WindowClipRect,
	float DeltaTime,
	bool bDeferRenderTargetUpdate)
{
	DrawWindow(PaintArgs, RenderTarget->GameThread_GetRenderTargetResource(), Window, WindowGeometry, WindowClipRect, DeltaTime, bDeferRenderTargetUpdate);
}

bool FWidgetRenderer::DrawInvalidationRoot(TSharedRef<SVirtualWindow>& VirtualWindow, UTextureRenderTarget2D* RenderTarget, FSlateInvalidationRoot& Root, const FSlateInvalidationContext& Context, bool bDeferRenderTargetUpdate)
{
	bool bRepaintedWidgets = false;
#if !UE_SERVER
	FSlateRenderer* MainSlateRenderer = FSlateApplication::Get().GetRenderer();
	FScopeLock ScopeLock(MainSlateRenderer->GetResourceCriticalSection());


	if (LIKELY(FApp::CanEverRender()))
	{

		if (bPrepassNeeded)
		{
			VirtualWindow->ProcessWindowInvalidation();
			VirtualWindow->SlatePrepass(Context.LayoutScaleMultiplier);
		}

		// Need to set a new window element list so make a copy
		FSlateInvalidationContext ContextCopy = Context;
		ContextCopy.ViewOffset = ViewOffset;

		{
			// Get the free buffer & add our virtual window
			ISlate3DRenderer::FScopedAcquireDrawBuffer ScopedDrawBuffer{ *Renderer, bDeferRenderTargetUpdate };
			FSlateWindowElementList& WindowElementList = ScopedDrawBuffer.GetDrawBuffer().AddWindowElementList(VirtualWindow);

			// Populates cached element data lists using the provide invalidation root
			ContextCopy.WindowElementList = &WindowElementList;
			FSlateInvalidationResult Result = Root.PaintInvalidationRoot(ContextCopy);

			if (Result.bRepaintedWidgets)
			{
				DeferredPaints = WindowElementList.GetDeferredPaintList();

				Renderer->DrawWindow_GameThread(ScopedDrawBuffer.GetDrawBuffer());

				ScopedDrawBuffer.GetDrawBuffer().ViewOffset = Result.ViewOffset;

				FRenderThreadUpdateContext RenderThreadUpdateContext =
				{
					&(ScopedDrawBuffer.GetDrawBuffer()),
					(FApp::GetCurrentTime() - GStartTime),
					static_cast<float>(FApp::GetDeltaTime()),
					(FPlatformTime::Seconds() - GStartTime),
					static_cast<float>(FApp::GetDeltaTime()),
					static_cast<FRenderTarget*>(RenderTarget->GameThread_GetRenderTargetResource()),
					Renderer.Get(),
					bClearTarget
				};

				bRepaintedWidgets = Result.bRepaintedWidgets;
				FSlateApplication::Get().GetRenderer()->AddWidgetRendererUpdate(RenderThreadUpdateContext, bDeferRenderTargetUpdate);


				// Any deferred painted elements of the retainer should be drawn directly by the main renderer, not rendered into the render target,
				// as most of those sorts of things will break the rendering rect, things like tooltips, and popup menus.
				for (auto& DeferredPaint : DeferredPaints)
				{
					Context.WindowElementList->QueueDeferredPainting(DeferredPaint->Copy(*Context.PaintArgs));
				}
			}
			else
			{
				WindowElementList.ResetElementList();
			}
		}
	}
#endif // !UE_SERVER

	return bRepaintedWidgets;
}

bool FWidgetRenderer::DrawInvalidationRoot(TSharedRef<SVirtualWindow>& VirtualWindow, UTextureRenderTarget2D* RenderTarget, FPaintArgs PaintArgs, float DrawScale, FVector2D DrawSize, bool bDeferRenderTargetUpdate)
{
	{
		const bool bIsInvalidationRoot = VirtualWindow->Advanced_IsInvalidationRoot();
		ensure(bIsInvalidationRoot);
		if (!bIsInvalidationRoot)
		{
			return false;
		}
	}

	bool bRepaintedWidgets = false;
#if !UE_SERVER
	FSlateRenderer* MainSlateRenderer = FSlateApplication::Get().GetRenderer();
	FScopeLock ScopeLock(MainSlateRenderer->GetResourceCriticalSection());

	if (LIKELY(FApp::CanEverRender()))
	{
		if (bPrepassNeeded)
		{
			VirtualWindow->ProcessWindowInvalidation();
			VirtualWindow->SlatePrepass(DrawScale);
		}

		{
			FGeometry WindowGeometry = FGeometry::MakeRoot(DrawSize * (1 / DrawScale), FSlateLayoutTransform(DrawScale));
			PaintArgs.GetHittestGrid().SetHittestArea(WindowGeometry.GetLayoutBoundingRect().GetTopLeft(), WindowGeometry.GetLayoutBoundingRect().GetSize());
		}

		{
			// Get the free buffer & add our virtual window
			ISlate3DRenderer::FScopedAcquireDrawBuffer ScopedDrawBuffer{ *Renderer, bDeferRenderTargetUpdate };
			FSlateWindowElementList& WindowElementList = ScopedDrawBuffer.GetDrawBuffer().AddWindowElementList(VirtualWindow);

			// Populates cached element data lists using the provide invalidation root
			FWidgetStyle InitialWidgetStyle;
			FSlateInvalidationContext InvalidationContext(WindowElementList, InitialWidgetStyle);
			InvalidationContext.bParentEnabled = true;
			InvalidationContext.bAllowFastPathUpdate = true;
			InvalidationContext.LayoutScaleMultiplier = DrawScale;
			InvalidationContext.PaintArgs = &PaintArgs;
			InvalidationContext.IncomingLayerId = 0;
			InvalidationContext.CullingRect = VirtualWindow->GetClippingRectangleInWindow();

			FSlateInvalidationResult Result = VirtualWindow->PaintInvalidationRoot(InvalidationContext);
			if (Result.bRepaintedWidgets)
			{
				DeferredPaints = WindowElementList.GetDeferredPaintList();

				Renderer->DrawWindow_GameThread(ScopedDrawBuffer.GetDrawBuffer());

				ScopedDrawBuffer.GetDrawBuffer().ViewOffset = Result.ViewOffset;

				FRenderThreadUpdateContext RenderThreadUpdateContext =
				{
					&(ScopedDrawBuffer.GetDrawBuffer()),
					(FApp::GetCurrentTime() - GStartTime),
					static_cast<float>(FApp::GetDeltaTime()),
					(FPlatformTime::Seconds() - GStartTime),
					static_cast<float>(FApp::GetDeltaTime()),
					static_cast<FRenderTarget*>(RenderTarget->GameThread_GetRenderTargetResource()),
					Renderer.Get(),
					bClearTarget
				};

				bRepaintedWidgets = Result.bRepaintedWidgets;
				FSlateApplication::Get().GetRenderer()->AddWidgetRendererUpdate(RenderThreadUpdateContext, bDeferRenderTargetUpdate);


				// Any deferred painted elements of the retainer should be drawn directly by the main renderer, not rendered into the render target,
				// as most of those sorts of things will break the rendering rect, things like tooltips, and popup menus.
				for (auto& DeferredPaint : DeferredPaints)
				{
					InvalidationContext.WindowElementList->QueueDeferredPainting(DeferredPaint->Copy(*InvalidationContext.PaintArgs));
				}
			}
			else
			{
				WindowElementList.ResetElementList();
			}
		}
	}
#endif // !UE_SERVER

	return bRepaintedWidgets;
}

