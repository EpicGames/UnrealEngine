// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/SlateBlueprintLibrary.h"
#include "EngineGlobals.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"

#include "Engine/UserInterfaceSettings.h"
#include "Slate/SlateBrushAsset.h"
#include "Engine/UserInterfaceSettings.h"
#include "Blueprint/SlateBlueprintLibrary.h"
#include "Slate/SceneViewport.h"
#include "Blueprint/WidgetLayoutLibrary.h"
#include "Slate/SGameLayerManager.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateBlueprintLibrary)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// USlateBlueprintLibrary

USlateBlueprintLibrary::USlateBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

bool USlateBlueprintLibrary::IsUnderLocation(const FGeometry& Geometry, const FVector2D& AbsoluteCoordinate)
{
	return Geometry.IsUnderLocation(AbsoluteCoordinate);
}

FVector2D USlateBlueprintLibrary::AbsoluteToLocal(const FGeometry& Geometry, FVector2D AbsoluteCoordinate)
{
	return Geometry.AbsoluteToLocal(AbsoluteCoordinate);
}

FVector2D USlateBlueprintLibrary::LocalToAbsolute(const FGeometry& Geometry, FVector2D LocalCoordinate)
{
	return Geometry.LocalToAbsolute(LocalCoordinate);
}

FVector2D USlateBlueprintLibrary::GetLocalTopLeft(const FGeometry& Geometry)
{
	const FVector2D TopLeft(0.0f, 0.0f);
	return Geometry.GetLocalPositionAtCoordinates(TopLeft);
}

FVector2D USlateBlueprintLibrary::GetLocalSize(const FGeometry& Geometry)
{
	return Geometry.GetLocalSize();
}

FVector2D USlateBlueprintLibrary::GetAbsoluteSize(const FGeometry& Geometry)
{
	return TransformVector(Geometry.GetAccumulatedRenderTransform(), Geometry.GetLocalSize());
}

bool USlateBlueprintLibrary::EqualEqual_SlateBrush(const FSlateBrush& A, const FSlateBrush& B)
{
	return A == B;
}

void USlateBlueprintLibrary::LocalToViewport(const UObject* WorldContextObject, const FGeometry& Geometry, FVector2D LocalCoordinate, FVector2D& PixelPosition, FVector2D& ViewportPosition)
{
	FVector2D AbsoluteCoordinate = Geometry.LocalToAbsolute(LocalCoordinate);
	AbsoluteToViewport(WorldContextObject, AbsoluteCoordinate, PixelPosition, ViewportPosition);
}

void USlateBlueprintLibrary::AbsoluteToViewport(const UObject* WorldContextObject, FVector2D AbsoluteDesktopCoordinate, FVector2D& PixelPosition, FVector2D& ViewportPosition)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if ( World && World->IsGameWorld() )
	{
		if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
		{
			TSharedPtr<IGameLayerManager> GameLayerManager = ViewportClient->GetGameLayerManager();
			if (GameLayerManager.IsValid())
			{
				FVector2D ViewportSize;
				ViewportClient->GetViewportSize(ViewportSize);

				const FGeometry& ViewportGeometry = GameLayerManager->GetViewportWidgetHostGeometry();

				ViewportPosition = ViewportGeometry.AbsoluteToLocal(AbsoluteDesktopCoordinate);
				PixelPosition = (ViewportPosition / ViewportGeometry.GetLocalSize()) * ViewportSize;

				return;
			}
		}
	}

	PixelPosition = FVector2D(0, 0);
	ViewportPosition = FVector2D(0, 0);
}

void USlateBlueprintLibrary::ScreenToWidgetLocal(const UObject* WorldContextObject, const FGeometry& Geometry, FVector2D ScreenPosition, FVector2D& LocalCoordinate, bool bIncludeWindowPosition /*= false*/)
{
	FVector2D AbsoluteCoordinate;
	ScreenToWidgetAbsolute(WorldContextObject, ScreenPosition, AbsoluteCoordinate, bIncludeWindowPosition);

	LocalCoordinate = Geometry.AbsoluteToLocal(AbsoluteCoordinate);
}

void USlateBlueprintLibrary::ScreenToWidgetAbsolute(const UObject* WorldContextObject, FVector2D ScreenPosition, FVector2D& AbsoluteCoordinate, bool bIncludeWindowPosition /*= false*/)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if ( World && World->IsGameWorld() )
	{
		if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
		{
			TSharedPtr<IGameLayerManager> GameLayerManager = ViewportClient->GetGameLayerManager();
			if (GameLayerManager.IsValid())
			{
				FVector2D ViewportSize;
				ViewportClient->GetViewportSize(ViewportSize);

				const FGeometry& ViewportGeometry = GameLayerManager->GetViewportWidgetHostGeometry();
				const FVector2D ViewportPosition = ViewportGeometry.GetLocalSize() * (ScreenPosition / ViewportSize);

				AbsoluteCoordinate = ViewportGeometry.LocalToAbsolute(ViewportPosition);
				if (bIncludeWindowPosition)
				{
					if (SWindow* Window = ViewportClient->GetWindow().Get())
					{
						AbsoluteCoordinate -= Window->GetPositionInScreen();
					}
				}
				return;
			}
		}
	}

	AbsoluteCoordinate = FVector2D(0, 0);
}

void USlateBlueprintLibrary::ScreenToViewport(const UObject* WorldContextObject, FVector2D ScreenPosition, FVector2D& ViewportPosition)
{
	FVector2D AbsolutePosition;
	USlateBlueprintLibrary::ScreenToWidgetAbsolute(WorldContextObject, ScreenPosition, AbsolutePosition);
	FVector2D PixelPosition;
	USlateBlueprintLibrary::AbsoluteToViewport(WorldContextObject, AbsolutePosition, PixelPosition, ViewportPosition);
}

float USlateBlueprintLibrary::Scalar_AbsoluteToLocal(const FGeometry& Geometry, float AbsoluteScalar)
{
	return Inverse(Geometry.GetAccumulatedRenderTransform()).TransformVector(FVector2f(AbsoluteScalar, 0.0f)).Size();
}

float USlateBlueprintLibrary::Scalar_LocalToAbsolute(const FGeometry& Geometry, float LocalScalar)
{
	return Geometry.GetAccumulatedRenderTransform().TransformVector(FVector2f(LocalScalar, 0.0f)).Size();
}

FVector2D USlateBlueprintLibrary::Vector_AbsoluteToLocal(const FGeometry& Geometry, FVector2D AbsoluteVector)
{
	return Inverse(Geometry.GetAccumulatedRenderTransform()).TransformVector(AbsoluteVector);
}

FVector2D USlateBlueprintLibrary::Vector_LocalToAbsolute(const FGeometry& Geometry, FVector2D LocalVector)
{
	return Geometry.GetAccumulatedRenderTransform().TransformVector(LocalVector);
}

float USlateBlueprintLibrary::TransformScalarAbsoluteToLocal(const FGeometry& Geometry, float AbsoluteScalar)
{
	return Geometry.GetAccumulatedRenderTransform().TransformVector(FVector2f(AbsoluteScalar, 0.0f)).Size();
}

float USlateBlueprintLibrary::TransformScalarLocalToAbsolute(const FGeometry& Geometry, float LocalScalar)
{
	return Inverse(Geometry.GetAccumulatedRenderTransform()).TransformVector(FVector2f(LocalScalar, 0.0f)).Size();
}

FVector2D USlateBlueprintLibrary::TransformVectorAbsoluteToLocal(const FGeometry& Geometry, FVector2D AbsoluteVector)
{
	return Geometry.GetAccumulatedRenderTransform().TransformVector(AbsoluteVector);
}

FVector2D USlateBlueprintLibrary::TransformVectorLocalToAbsolute(const FGeometry& Geometry, FVector2D LocalVector)
{
	return Inverse(Geometry.GetAccumulatedRenderTransform()).TransformVector(LocalVector);
}

#undef LOCTEXT_NAMESPACE

