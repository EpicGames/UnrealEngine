// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IUMGModule.h"
#include "UMGStyle.h"
#include "UMGPrivate.h"

#include "EntitySystem/MovieSceneEntityManager.h"

DEFINE_LOG_CATEGORY(LogUMG);

class FUMGModule : public IUMGModule
{
public:
	/** Constructor, set up console commands and variables **/
	FUMGModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			FUMGStyle::Initialize();
		}

		// This is done so that the compiler is available in non-cooked builds when the widget blueprint is 
		// compiled again in the running game.
		FModuleManager::Get().LoadModule(TEXT("UMGEditor"));
#endif
	}

	/** Called before the module is unloaded, right before the module object is destroyed. */
	virtual void ShutdownModule() override
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			FUMGStyle::Shutdown();
		}
#endif
	}

	TArray<int32> CustomFloatPropertyIDs;
	TArray<int32> CustomTransformPropertyIDs;
};

IMPLEMENT_MODULE(FUMGModule, UMG);
