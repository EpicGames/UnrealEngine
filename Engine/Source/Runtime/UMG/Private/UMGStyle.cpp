// Copyright Epic Games, Inc. All Rights Reserved.

#include "UMGStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"


TSharedPtr< FSlateStyleSet > FUMGStyle::UMGStyleInstance = NULL;

void FUMGStyle::Initialize()
{
	if ( !UMGStyleInstance.IsValid() )
	{
		UMGStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle( *UMGStyleInstance );
	}
}

void FUMGStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle( *UMGStyleInstance );
	ensure( UMGStyleInstance.IsUnique() );
	UMGStyleInstance.Reset();
}

FName FUMGStyle::GetStyleSetName()
{
	static FName StyleSetName("UMGStyle");
	return StyleSetName;
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(Style->RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon64x64(64.0f, 64.0f);

TSharedRef< FSlateStyleSet > FUMGStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("UMGStyle"));
	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/UMG"));
	
	Style->Set("MarchingAnts", new BORDER_BRUSH( TEXT("NonMarchingAnts"), FMargin(0.25f), FLinearColor(1,1,1,0.5) ));

	Style->Set("ClassIcon.Widget", new IMAGE_BRUSH(TEXT("Widget"), Icon16x16));
	Style->Set("ClassIcon.CheckBox", new IMAGE_BRUSH_SVG(TEXT("Checkbox"), Icon16x16));
	Style->Set("ClassIcon.Button", new IMAGE_BRUSH_SVG(TEXT("Button"), Icon16x16));
	Style->Set("ClassIcon.EditableTextBox", new IMAGE_BRUSH_SVG(TEXT("TextBox"), Icon16x16));
	Style->Set("ClassIcon.EditableText", new IMAGE_BRUSH_SVG(TEXT("EditableTextBox"), Icon16x16));
	Style->Set("ClassIcon.HorizontalBox", new IMAGE_BRUSH_SVG(TEXT("HorizontalBox"), Icon16x16));
	Style->Set("ClassIcon.VerticalBox", new IMAGE_BRUSH_SVG(TEXT("VerticalBox"), Icon16x16));
	Style->Set("ClassIcon.Image", new IMAGE_BRUSH_SVG(TEXT("Image"), Icon16x16));
	Style->Set("ClassIcon.CanvasPanel", new IMAGE_BRUSH_SVG(TEXT("CanvasPanel"), Icon16x16));
	Style->Set("ClassIcon.TextBlock", new IMAGE_BRUSH_SVG(TEXT("Text"), Icon16x16));
	Style->Set("ClassIcon.Border", new IMAGE_BRUSH_SVG(TEXT("Border"), Icon16x16));
	Style->Set("ClassIcon.Slider", new IMAGE_BRUSH_SVG(TEXT("Slider"), Icon16x16));
	Style->Set("ClassIcon.Spacer", new IMAGE_BRUSH_SVG(TEXT("Spacer"), Icon16x16));
	Style->Set("ClassIcon.ExpandableArea", new IMAGE_BRUSH_SVG(TEXT("ExpandableArea"), Icon16x16));
	Style->Set("ClassIcon.Scrollbox", new IMAGE_BRUSH_SVG(TEXT("Scrollbox"), Icon16x16));
	Style->Set("ClassIcon.ProgressBar", new IMAGE_BRUSH_SVG(TEXT("ProgressBar"), Icon16x16));
	Style->Set("ClassIcon.RichTextBlock", new IMAGE_BRUSH_SVG(TEXT("RichText"), Icon16x16));
	Style->Set("ClassIcon.SafeZone", new IMAGE_BRUSH_SVG(TEXT("SafeZone"), Icon16x16));
	Style->Set("ClassIcon.MenuAnchor", new IMAGE_BRUSH_SVG(TEXT("MenuAnchor"), Icon16x16));
	Style->Set("ClassIcon.InvalidationBox", new IMAGE_BRUSH_SVG(TEXT("MenuAnchor"), Icon16x16));
	Style->Set("ClassIcon.RetainerBox", new IMAGE_BRUSH_SVG(TEXT("MenuAnchor"), Icon16x16));
	Style->Set("ClassIcon.ScrollBar", new IMAGE_BRUSH_SVG(TEXT("ScrollBar"), Icon16x16));
	Style->Set("ClassIcon.UniformGridPanel", new IMAGE_BRUSH_SVG(TEXT("UniformGridPanel"), Icon16x16));
	Style->Set("ClassIcon.WidgetSwitcher", new IMAGE_BRUSH_SVG(TEXT("WidgetSwitcher"), Icon16x16));
	Style->Set("ClassIcon.MultiLineEditableText", new IMAGE_BRUSH_SVG(TEXT("EditableTextBoxMultiline"), Icon16x16));
	Style->Set("ClassIcon.MultilineEditableTextBox", new IMAGE_BRUSH_SVG(TEXT("TextBoxMultiline"), Icon16x16));
	Style->Set("ClassIcon.Viewport", new IMAGE_BRUSH_SVG(TEXT("Viewport"), Icon16x16));
	Style->Set("ClassIcon.ComboBox", new IMAGE_BRUSH_SVG(TEXT("ComboBox"), Icon16x16));
	Style->Set("ClassIcon.ComboBoxKey", new IMAGE_BRUSH_SVG(TEXT("ComboBox"), Icon16x16));
	Style->Set("ClassIcon.ComboBoxString", new IMAGE_BRUSH_SVG(TEXT("ComboBox"), Icon16x16));
	Style->Set("ClassIcon.ListView", new IMAGE_BRUSH_SVG(TEXT("ListView"), Icon16x16));
	Style->Set("ClassIcon.TileView", new IMAGE_BRUSH_SVG(TEXT("TileView"), Icon16x16));
	Style->Set("ClassIcon.TreeView", new IMAGE_BRUSH_SVG(TEXT("TreeView"), Icon16x16));
	Style->Set("ClassIcon.Overlay", new IMAGE_BRUSH_SVG(TEXT("Overlay"), Icon16x16));
	Style->Set("ClassIcon.Throbber", new IMAGE_BRUSH_SVG(TEXT("Throbber"), Icon16x16));
	Style->Set("ClassIcon.CircularThrobber", new IMAGE_BRUSH_SVG(TEXT("CircularThrobber"), Icon16x16));
	Style->Set("ClassIcon.NativeWidgetHost", new IMAGE_BRUSH_SVG(TEXT("NativeWidgetHost"), Icon16x16));
	Style->Set("ClassIcon.ScaleBox", new IMAGE_BRUSH_SVG(TEXT("ScaleBox"), Icon16x16));
	Style->Set("ClassIcon.Sizebox", new IMAGE_BRUSH_SVG(TEXT("Sizebox"), Icon16x16));
	Style->Set("ClassIcon.StackBox", new IMAGE_BRUSH_SVG(TEXT("StackBox"), Icon16x16));
	Style->Set("ClassIcon.SpinBox", new IMAGE_BRUSH_SVG(TEXT("SpinBox"), Icon16x16));
	Style->Set("ClassIcon.GridPanel", new IMAGE_BRUSH_SVG(TEXT("GridPanel"), Icon16x16));
	Style->Set("ClassIcon.WrapBox", new IMAGE_BRUSH_SVG(TEXT("WrapBox"), Icon16x16));
	Style->Set("ClassIcon.NamedSlot", new IMAGE_BRUSH_SVG(TEXT("NamedSlot"), Icon16x16));

	Style->Set("ClassIcon.UserWidget", new IMAGE_BRUSH(TEXT("UserWidget"), Icon16x16));

	Style->Set("ClassIcon.DetailsView", new IMAGE_BRUSH_SVG(TEXT("DetailsView"), Icon16x16));
	Style->Set("ClassIcon.SinglePropertyView", new IMAGE_BRUSH_SVG(TEXT("SinglePropertyView"), Icon16x16));

	Style->Set("Animations.TabIcon", new IMAGE_BRUSH(TEXT("Animations_16x"), Icon16x16));
	Style->Set("Designer.TabIcon", new IMAGE_BRUSH(TEXT("Designer_16x"), Icon16x16));
	Style->Set("Palette.TabIcon", new IMAGE_BRUSH(TEXT("Palette_16x"), Icon16x16));
	Style->Set("Sequencer.TabIcon", new IMAGE_BRUSH(TEXT("Timeline_16x"), Icon16x16));

	Style->Set("Animations.Icon", new IMAGE_BRUSH(TEXT("Animations_40x"), Icon40x40));
	Style->Set("Animations.Icon.Small", new IMAGE_BRUSH(TEXT("Animations_40x"), Icon20x20));

	Style->Set("Designer.Icon", new IMAGE_BRUSH(TEXT("Designer_40x"), Icon40x40));
	Style->Set("Designer.Icon.Small", new IMAGE_BRUSH(TEXT("Designer_40x"), Icon20x20));

	Style->Set("Palette.Icon", new IMAGE_BRUSH(TEXT("Palette_40x"), Icon40x40));
	Style->Set("Palette.Icon.Small", new IMAGE_BRUSH(TEXT("Palette_40x"), Icon20x20));

	Style->Set("Timeline.Icon", new IMAGE_BRUSH(TEXT("Timeline_40x"), Icon40x40));
	Style->Set("Timeline.Icon.Small", new IMAGE_BRUSH(TEXT("Timeline_40x"), Icon20x20));

	// Thumbnails
	Style->Set("ClassThumbnail.Border", new IMAGE_BRUSH_SVG(TEXT("Border_64"), Icon64x64));
	Style->Set("ClassThumbnail.Button", new IMAGE_BRUSH_SVG(TEXT("Button_64"), Icon64x64));
	Style->Set("ClassThumbnail.CanvasPanel", new IMAGE_BRUSH_SVG(TEXT("CanvasPanel_64"), Icon64x64));
	Style->Set("ClassThumbnail.CheckBox", new IMAGE_BRUSH_SVG(TEXT("Checkmark_64"), Icon64x64));
	Style->Set("ClassThumbnail.CircularThrobber", new IMAGE_BRUSH_SVG(TEXT("CircularThrobber_64"), Icon64x64));
	Style->Set("ClassThumbnail.Viewport", new IMAGE_BRUSH_SVG(TEXT("Viewport_64"), Icon64x64));
	Style->Set("ClassThumbnail.ComboBox", new IMAGE_BRUSH_SVG(TEXT("ComboBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.ComboBoxKey", new IMAGE_BRUSH_SVG(TEXT("ComboBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.ComboBoxString", new IMAGE_BRUSH_SVG(TEXT("ComboBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.MultilineEditableText", new IMAGE_BRUSH_SVG(TEXT("EditableTextBoxMultiline_64"), Icon64x64));
	Style->Set("ClassThumbnail.EditableText", new IMAGE_BRUSH_SVG(TEXT("EditableTextBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.ExpandableArea", new IMAGE_BRUSH_SVG(TEXT("ExpandableArea_64"), Icon64x64));
	Style->Set("ClassThumbnail.Spacer", new IMAGE_BRUSH_SVG(TEXT("Spacer_64"), Icon64x64));
	Style->Set("ClassThumbnail.GridPanel", new IMAGE_BRUSH_SVG(TEXT("GridPanel_64"), Icon64x64));
	Style->Set("ClassThumbnail.HorizontalBox", new IMAGE_BRUSH_SVG(TEXT("HorizontalBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.Image", new IMAGE_BRUSH_SVG(TEXT("Image_64"), Icon64x64));
	Style->Set("ClassThumbnail.List", new IMAGE_BRUSH_SVG(TEXT("List_64"), Icon64x64));
	Style->Set("ClassThumbnail.ListView", new IMAGE_BRUSH_SVG(TEXT("List_64"), Icon64x64));
	Style->Set("ClassThumbnail.InvalidationBox", new IMAGE_BRUSH_SVG(TEXT("MenuAnchor_64"), Icon64x64));
	Style->Set("ClassThumbnail.RetainerBox", new IMAGE_BRUSH_SVG(TEXT("MenuAnchor_64"), Icon64x64));
	Style->Set("ClassThumbnail.MenuAnchor", new IMAGE_BRUSH_SVG(TEXT("MenuAnchor_64"), Icon64x64));
	Style->Set("ClassThumbnail.NamedSlot", new IMAGE_BRUSH_SVG(TEXT("NamedSlot_64"), Icon64x64));
	Style->Set("ClassThumbnail.Overlay", new IMAGE_BRUSH_SVG(TEXT("Overlay_64"), Icon64x64));
	Style->Set("ClassThumbnail.ProgressBar", new IMAGE_BRUSH_SVG(TEXT("ProgressBar_64"), Icon64x64));
	Style->Set("ClassThumbnail.RichTextBlock", new IMAGE_BRUSH_SVG(TEXT("RichText_64"), Icon64x64));
	Style->Set("ClassThumbnail.SafeZone", new IMAGE_BRUSH_SVG(TEXT("SafeZone_64"), Icon64x64));
	Style->Set("ClassThumbnail.NativeWidgetHost", new IMAGE_BRUSH_SVG(TEXT("NativeHostWidget_64"), Icon64x64));
	Style->Set("ClassThumbnail.Scalebox", new IMAGE_BRUSH_SVG(TEXT("Scalebox_64"), Icon64x64));
	Style->Set("ClassThumbnail.Scrollbox", new IMAGE_BRUSH_SVG(TEXT("Scrollbox_64"), Icon64x64));
	Style->Set("ClassThumbnail.Sizebox", new IMAGE_BRUSH_SVG(TEXT("Sizebox_64"), Icon64x64));
	Style->Set("ClassThumbnail.Slider", new IMAGE_BRUSH_SVG(TEXT("Slider_64"), Icon64x64));
	Style->Set("ClassThumbnail.SpinBox", new IMAGE_BRUSH_SVG(TEXT("SpinBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.StackBox", new IMAGE_BRUSH_SVG(TEXT("StackBox"), Icon64x64));
	Style->Set("ClassThumbnail.MultilineEditableTextBox", new IMAGE_BRUSH_SVG(TEXT("TextBoxMultiline_64"), Icon64x64));
	Style->Set("ClassThumbnail.EditableTextBox", new IMAGE_BRUSH_SVG(TEXT("TextBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.TextBlock", new IMAGE_BRUSH_SVG(TEXT("Text_64"), Icon64x64));
	Style->Set("ClassThumbnail.Throbber", new IMAGE_BRUSH_SVG(TEXT("Throbber_64"), Icon64x64));
	Style->Set("ClassThumbnail.TileView", new IMAGE_BRUSH_SVG(TEXT("TileView_64"), Icon64x64));
	Style->Set("ClassThumbnail.TreeView", new IMAGE_BRUSH_SVG(TEXT("TreeView_64"), Icon64x64));
	Style->Set("ClassThumbnail.ScrollBar", new IMAGE_BRUSH_SVG(TEXT("ScrollBar_64"), Icon64x64));
	Style->Set("ClassThumbnail.UniformGridPanel", new IMAGE_BRUSH_SVG(TEXT("UniformGridPanel_64"), Icon64x64));
	Style->Set("ClassThumbnail.VerticalBox", new IMAGE_BRUSH_SVG(TEXT("VerticalBox_64"), Icon64x64));
	Style->Set("ClassThumbnail.WidgetSwitcher", new IMAGE_BRUSH_SVG(TEXT("WidgetSwitcher_64"), Icon64x64));
	Style->Set("ClassThumbnail.WrapBox", new IMAGE_BRUSH_SVG(TEXT("WrapBox_64"), Icon64x64));

	Style->Set("ClassThumbnail.DetailsView", new IMAGE_BRUSH_SVG(TEXT("DetailsView_64"), Icon64x64));
	Style->Set("ClassThumbnail.SinglePropertyView", new IMAGE_BRUSH_SVG(TEXT("SinglePropertyView_64"), Icon64x64));

	return Style;
}

#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH

void FUMGStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FUMGStyle::Get()
{
	return *UMGStyleInstance;
}
