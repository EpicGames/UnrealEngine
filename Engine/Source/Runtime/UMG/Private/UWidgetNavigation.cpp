// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "Input/NavigationReply.h"
#include "Types/NavigationMetaData.h"
#include "Input/NavigationMethod.h"
#include "Blueprint/WidgetNavigation.h"
#include "Blueprint/WidgetTree.h"

void FWidgetNavigationData::Resolve(UUserWidget* InInstance, UWidgetTree* WidgetTree)
{
	switch (Rule)
	{
	case EUINavigationRule::Explicit:
		Widget = WidgetTree->FindWidget(WidgetToFocus);
		break;
	case EUINavigationRule::Custom:
	case EUINavigationRule::CustomBoundary:
		CustomDelegate.BindUFunction(InInstance, WidgetToFocus);
		break;
	}
}

#if WITH_EDITOR

void FWidgetNavigationData::TryToRenameBinding(FName OldName, FName NewName)
{
	if (WidgetToFocus == OldName)
	{
		WidgetToFocus = NewName;
	}
}

#endif

/////////////////////////////////////////////////////
// UWidgetNavigation

UWidgetNavigation::UWidgetNavigation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

FWidgetNavigationData& UWidgetNavigation::GetNavigationData(EUINavigation Nav)
{
	switch ( Nav )
	{
	case EUINavigation::Up:
		return Up;
	case EUINavigation::Down:
		return Down;
	case EUINavigation::Left:
		return Left;
	case EUINavigation::Right:
		return Right;
	case EUINavigation::Next:
		return Next;
	case EUINavigation::Previous:
		return Previous;
	default:
		break;
	}

	// Should never happen
	check(false);

	return Up;
}

EUINavigationRule UWidgetNavigation::GetNavigationRule(EUINavigation Nav)
{
	switch ( Nav )
	{
	case EUINavigation::Up:
		return Up.Rule;
	case EUINavigation::Down:
		return Down.Rule;
	case EUINavigation::Left:
		return Left.Rule;
	case EUINavigation::Right:
		return Right.Rule;
	case EUINavigation::Next:
		return Next.Rule;
	case EUINavigation::Previous:
		return Previous.Rule;
	}
	return EUINavigationRule::Escape;
}

void UWidgetNavigation::TryToRenameBinding(FName OldName, FName NewName)
{
	Up.TryToRenameBinding(OldName, NewName);
	Down.TryToRenameBinding(OldName, NewName);
	Left.TryToRenameBinding(OldName, NewName);
	Right.TryToRenameBinding(OldName, NewName);
	Next.TryToRenameBinding(OldName, NewName);
	Previous.TryToRenameBinding(OldName, NewName);
}

#endif

void UWidgetNavigation::ResolveRules(UUserWidget* InOuter, UWidgetTree* WidgetTree)
{
	Up.Resolve(InOuter, WidgetTree);
	Down.Resolve(InOuter, WidgetTree);
	Left.Resolve(InOuter, WidgetTree);
	Right.Resolve(InOuter, WidgetTree);
	Next.Resolve(InOuter, WidgetTree);
	Previous.Resolve(InOuter, WidgetTree);
}

void UWidgetNavigation::UpdateMetaData(TSharedRef<FNavigationMetaData> MetaData)
{
	UpdateMetaDataEntry(MetaData, Up, EUINavigation::Up);
	UpdateMetaDataEntry(MetaData, Down, EUINavigation::Down);
	UpdateMetaDataEntry(MetaData, Left, EUINavigation::Left);
	UpdateMetaDataEntry(MetaData, Right, EUINavigation::Right);
	UpdateMetaDataEntry(MetaData, Next, EUINavigation::Next);
	UpdateMetaDataEntry(MetaData, Previous, EUINavigation::Previous);

	MetaData->SetNavigationMethodStruct(NavigationMethod);
	MetaData->SetNavigationRoutingPolicy(RoutingPolicy);
}

bool UWidgetNavigation::IsDefaultNavigation() const
{
	return Up.Rule == EUINavigationRule::Escape &&
		Down.Rule == EUINavigationRule::Escape &&
		Left.Rule == EUINavigationRule::Escape &&
		Right.Rule == EUINavigationRule::Escape &&
		Next.Rule == EUINavigationRule::Escape &&
		Previous.Rule == EUINavigationRule::Escape &&
		RoutingPolicy == EWidgetNavigationRoutingPolicy::Default &&
		!NavigationMethod.IsValid();
}

void UWidgetNavigation::UpdateMetaDataEntry(TSharedRef<FNavigationMetaData> MetaData, const FWidgetNavigationData & NavData, EUINavigation Nav)
{
	switch ( NavData.Rule )
	{
	case EUINavigationRule::Escape:
		MetaData->SetNavigationEscape(Nav);
		break;
	case EUINavigationRule::Stop:
		MetaData->SetNavigationStop(Nav);
		break;
	case EUINavigationRule::Wrap:
		MetaData->SetNavigationWrap(Nav);
		break;
	case EUINavigationRule::Explicit:
		if ( NavData.Widget.IsValid() )
		{
			MetaData->SetNavigationExplicit(Nav, NavData.Widget.Get()->GetCachedWidget());
		}
		break;
	case EUINavigationRule::Custom:
	case EUINavigationRule::CustomBoundary:
		if (NavData.CustomDelegate.IsBound())
		{
			FCustomWidgetNavigationDelegate CustomDelegate = NavData.CustomDelegate;
			MetaData->SetNavigationCustom(Nav, NavData.Rule, FNavigationDelegate::CreateLambda([CustomDelegate](EUINavigation CustomNav) {
				UWidget* CustomWidget = CustomDelegate.Execute(CustomNav);
				if (CustomWidget)
				{
					return CustomWidget->GetCachedWidget();
				}
				return TSharedPtr<SWidget>();
			}));
		}
		break;
	}
}
