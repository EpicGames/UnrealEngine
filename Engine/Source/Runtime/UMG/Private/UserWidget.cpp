// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/UserWidget.h"

#include "Engine/GameInstance.h"
#include "Rendering/DrawElements.h"
#include "Sound/SoundBase.h"
#include "Sound/SlateSound.h"
#include "Framework/Application/SlateApplication.h"
#include "Trace/SlateMemoryTags.h"
#include "UObject/ObjectInstancingGraph.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Components/NamedSlot.h"
#include "Slate/SObjectWidget.h"
#include "Blueprint/WidgetTree.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Animation/UMGSequencePlayer.h"
#include "Animation/UMGSequenceTickManager.h"
#include "Extensions/UserWidgetExtension.h"
#include "Extensions/WidgetBlueprintGeneratedClassExtension.h"
#include "UObject/UnrealType.h"
#include "Blueprint/WidgetNavigation.h"
#include "Animation/WidgetAnimation.h"
#include "MovieScene.h"
#include "Interfaces/ITargetPlatform.h"
#include "Blueprint/GameViewportSubsystem.h"
#include "Blueprint/WidgetBlueprintLibrary.h"
#include "Blueprint/WidgetChild.h"
#include "UObject/EditorObjectVersion.h"
#include "UMGPrivate.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectHash.h"
#include "UObject/PropertyPortFlags.h"
#include "TimerManager.h"
#include "UObject/Package.h"
#include "Editor/WidgetCompilerLog.h"
#include "GameFramework/InputSettings.h"
#include "Engine/InputDelegateBinding.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserWidget)

#define LOCTEXT_NAMESPACE "UMG"

namespace UE::UMG::Private
{
	static thread_local uint32 InitializingFromWidgetTree = 0;
}

uint32& UUserWidget::GetInitializingFromWidgetTree()
{
	return UE::UMG::Private::InitializingFromWidgetTree;
}

static FGeometry NullGeometry;
static FSlateRect NullRect;
static FWidgetStyle NullStyle;

FSlateWindowElementList& GetNullElementList()
{
	static FSlateWindowElementList NullElementList(nullptr);
	return NullElementList;
}

FPaintContext::FPaintContext()
	: AllottedGeometry(NullGeometry)
	, MyCullingRect(NullRect)
	, OutDrawElements(GetNullElementList())
	, LayerId(0)
	, WidgetStyle(NullStyle)
	, bParentEnabled(true)
	, MaxLayer(0)
{
}

UUMGSequencePlayer* UUserWidgetFunctionLibrary::Conv_UMGSequencePlayer(const FWidgetAnimationHandle& WidgetAnimationHandle)
{
	return WidgetAnimationHandle.GetSequencePlayer();
}

UUserWidget* UUserWidgetFunctionLibrary::GetOuterUserWidget(const UWidget* Widget)
{
	return Widget != nullptr? Widget->GetTypedOuter<UUserWidget>() : nullptr;
}

/////////////////////////////////////////////////////
// UUserWidget
UUserWidget::UUserWidget(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAutomaticallyRegisterInputOnConstruction(false)
	, bHasScriptImplementedTick(true)
	, bHasScriptImplementedPaint(true)
	, bInitialized(false)
	, bAreExtensionsPreConstructed(false)
	, bAreExtensionsConstructed(false)
	, bStoppingAllAnimations(false)
	, TickFrequency(EWidgetTickFrequency::Auto)
{
	SetVisibilityInternal(ESlateVisibility::SelfHitTestInvisible);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bIsFocusable = false;
	ColorAndOpacity = FLinearColor::White;
	ForegroundColor = FSlateColor::UseForeground();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MinimumDesiredSize = FVector2D(0, 0);

#if WITH_EDITORONLY_DATA
	DesignTimeSize = FVector2D(100, 100);
	PaletteCategory = LOCTEXT("UserCreated", "User Created");
	DesignSizeMode = EDesignPreviewSizeMode::FillScreen;
#endif

	static bool bStaticInit = false;
	if (!bStaticInit)
	{
		bStaticInit = true;
		FLatentActionManager::OnLatentActionsChanged().AddStatic(&UUserWidget::OnLatentActionsChanged);
	}
}

UWidgetBlueprintGeneratedClass* UUserWidget::GetWidgetTreeOwningClass() const
{
	UWidgetBlueprintGeneratedClass* WidgetClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass());
	if (WidgetClass != nullptr)
	{
		WidgetClass = WidgetClass->FindWidgetTreeOwningClass();
	}

	return WidgetClass;
}

bool UUserWidget::Initialize()
{
	// If it's not initialized initialize it, as long as it's not the CDO, we never initialize the CDO.
	if (!bInitialized && !HasAnyFlags(RF_ClassDefaultObject))
	{
		// If this is a sub-widget of another UserWidget, default designer flags and player context to match those of the owning widget
		if (UUserWidget* OwningUserWidget = GetTypedOuter<UUserWidget>())
		{
#if WITH_EDITOR
			SetDesignerFlags(OwningUserWidget->GetDesignerFlags());
#endif
			SetPlayerContext(OwningUserWidget->GetPlayerContext());
		}

		UWidgetBlueprintGeneratedClass* BGClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass());
		// Only do this if this widget is of a blueprint class
		if (BGClass)
		{
			BGClass->InitializeWidget(this);
		}
		else
		{
			InitializeNativeClassData();
		}

		if ( WidgetTree == nullptr )
		{
			WidgetTree = NewObject<UWidgetTree>(this, TEXT("WidgetTree"), RF_Transient);
		}
		else
		{
			WidgetTree->SetFlags(RF_Transient);

			InitializeNamedSlots();
		}

		// For backward compatibility, run the initialize event on widget that doesn't have a player context only when the class authorized it.
		bool bClassWantsToRunInitialized = BGClass && BGClass->bCanCallInitializedWithoutPlayerContext;
		if (!IsDesignTime() && (PlayerContext.IsValid() || bClassWantsToRunInitialized))
		{
			NativeOnInitialized();
		}
#if WITH_EDITOR
		else if (IsDesignTime())
		{
			if (UWidgetBlueprintGeneratedClass* BPClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
			{
				BPClass->ForEachExtension([this](UWidgetBlueprintGeneratedClassExtension* Extension)
					{
						Extension->InitializeInEditor(this);
					});
			}

			// Extension can add other extensions. Use index loop to initialize them all.
			for (int32 Index = 0; Index < Extensions.Num(); ++Index)
			{
				UUserWidgetExtension* Extension = Extensions[Index];
				check(Extension);
				Extension->InitializeInEditor();
			}			
		}
#endif // WITH_EDITOR
		bInitialized = true;
		return true;
	}

	return false;
}

void UUserWidget::InitializeNamedSlots()
{
	for (const FNamedSlotBinding& Binding : NamedSlotBindings )
	{
		if ( UWidget* BindingContent = Binding.Content )
		{
			FObjectPropertyBase* NamedSlotProperty = FindFProperty<FObjectPropertyBase>(GetClass(), Binding.Name);
#if !WITH_EDITOR
			// In editor, renaming a NamedSlot widget will cause this ensure in UpdatePreviewWidget of widget that use that namedslot
			ensure(NamedSlotProperty);
#endif
			if ( NamedSlotProperty ) 
			{
				UNamedSlot* NamedSlot = Cast<UNamedSlot>(NamedSlotProperty->GetObjectPropertyValue_InContainer(this));
				if ( ensure(NamedSlot) )
				{
					NamedSlot->ClearChildren();
					NamedSlot->AddChild(BindingContent);
				}
			}
		}
	}
}

void UUserWidget::DuplicateAndInitializeFromWidgetTree(UWidgetTree* InWidgetTree, const TMap<FName, UWidget*>& NamedSlotContentToMerge)
{
	TScopeCounter<uint32> ScopeInitializingFromWidgetTree(GetInitializingFromWidgetTree());

	if ( ensure(InWidgetTree) && !HasAnyFlags(RF_NeedPostLoad))
	{
		FObjectInstancingGraph ObjectInstancingGraph;
		WidgetTree = NewObject<UWidgetTree>(this, InWidgetTree->GetClass(), NAME_None, RF_Transactional, InWidgetTree, false, &ObjectInstancingGraph);
		WidgetTree->SetFlags(RF_Transient | RF_DuplicateTransient);

		// After using the widget tree as a template, we need to loop over the instanced sub-objects and
		// initialize any UserWidgets, so that they can repeat the process for their children.
		ObjectInstancingGraph.ForEachObjectInstance([this](UObject* Instanced) {
			// Make sure all widgets inherit the designer flags.
#if WITH_EDITOR
			if (UWidget* InstancedWidget = Cast<UWidget>(Instanced))
			{
				InstancedWidget->SetDesignerFlags(GetDesignerFlags());
			}
#endif

			if (UUserWidget* InstancedSubUserWidget = Cast<UUserWidget>(Instanced))
			{
				InstancedSubUserWidget->SetPlayerContext(GetPlayerContext());
				InstancedSubUserWidget->Initialize();
			}
		});

		TArray<UWidget*> AllNamedSlotContentWidgets;
		NamedSlotContentToMerge.GenerateValueArray(AllNamedSlotContentWidgets);

		TSet<FName> const* ConflictingWidgetNames = nullptr;
#if WITH_EDITOR
		if (UWidgetBlueprintGeneratedClass* BGClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
		{
			ConflictingWidgetNames = &(BGClass->NameClashingInHierarchy);
		}
#endif

		auto SetContentWidgetForNamedSlot = [this,&ConflictingWidgetNames](FName NamedSlotName, UWidget* TemplateSlotContent)
		{
			FObjectInstancingGraph NamedSlotInstancingGraph;
			// We need to add a mapping from the template's widget tree to the new widget tree, that way
			// as we instance the widget hierarchy it's grafted onto the new widget tree.
			NamedSlotInstancingGraph.AddNewObject(WidgetTree, TemplateSlotContent->GetTypedOuter<UWidgetTree>());

			FName TemplateSlotContentName = TemplateSlotContent->GetFName();
			// ConflictingWidgetNames is an optional parameter. If we find an item with the name we were about to create in the widget tree, we remove the NamedSlot to avoid the corrupted tree we would get otherwise
			if (ConflictingWidgetNames == nullptr || !ConflictingWidgetNames->Contains(TemplateSlotContentName))
			{
				// Instance the new widget from the foreign tree, but do it in a way that grafts it onto the tree we're instancing.
				UWidget* Content = NewObject<UWidget>(WidgetTree, TemplateSlotContent->GetClass(), TemplateSlotContentName, RF_Transactional, TemplateSlotContent, false, &NamedSlotInstancingGraph);
				Content->SetFlags(RF_Transient | RF_DuplicateTransient);

				// Insert the newly constructed widget into the named slot that corresponds.  The above creates
				// it as if it was always part of the widget tree, but this actually puts it into a widget's
				// slot for the named slot.
				SetContentForSlot(NamedSlotName, Content);
			}
			else
			{
				SetContentForSlot(NamedSlotName, nullptr);
			}
		};

		// This block controls merging named slot content specified in a child class for the widget we're templated after.
		for (const TPair<FName, UWidget*>& KVP_SlotContent : NamedSlotContentToMerge)
		{
			// Don't insert the named slot content if the named slot is filled already.  This is a problematic
			// scenario though, if someone inserted content, but we have class default instances, we sorta leave
			// ourselves in a strange situation, because there are now potentially class variables that won't
			// have an instance assigned.
			if (!GetContentForSlot(KVP_SlotContent.Key))
			{
				if (UWidget* TemplateSlotContent = KVP_SlotContent.Value)
				{
					TArray<TPair<FName, UWidget*>> NamedSlotContentCreationStack;
					FName OwningNamedSlot = KVP_SlotContent.Key;
					NamedSlotContentCreationStack.Add(TTuple<FName, UWidget*>(OwningNamedSlot, TemplateSlotContent));

					// Search for the owning Namedslot to see if it is the content of another Namedslot itself.
					// If so, we need to ensure it is added to the widget tree prior to its content.
					// Repeat until the owning Namedslot is no longer found as the content of another.
					while (UWidget** FoundContentWidget = AllNamedSlotContentWidgets.FindByPredicate([OwningNamedSlot](const UWidget* Content) {return Content ? Content->GetFName() == OwningNamedSlot : false;}))
					{
						UWidget* NestedNamedSlotContent = *FoundContentWidget;
						OwningNamedSlot = *NamedSlotContentToMerge.FindKey(NestedNamedSlotContent);

						// Make sure we have not already iterated on this Namedslot.
						if (!GetContentForSlot(OwningNamedSlot) && !NamedSlotContentCreationStack.ContainsByPredicate([OwningNamedSlot](const TTuple<FName, UWidget*>& Content) {return Content.Key == OwningNamedSlot;}))
						{
							NamedSlotContentCreationStack.Add(TPair<FName, UWidget*>(OwningNamedSlot, NestedNamedSlotContent));
						}
						else 
						{
							break;
						}
					}

					// Go through the namedslot/content pair in hierarchy order and add them to the widget tree.
					for (int32 Index = NamedSlotContentCreationStack.Num() - 1; Index >= 0; Index--)
					{
						TPair<FName, UWidget*>& KVP = NamedSlotContentCreationStack[Index];
						SetContentWidgetForNamedSlot(KVP.Key, KVP.Value);
					}
				}
			}
		}
	}
}

void UUserWidget::BeginDestroy()
{
	Super::BeginDestroy();

	// At this point, the widget is being completely destroyed. We want to ensure that
	// the input component is no longer being processed so that any input delegate nodes
	// are no longer going to trigger
	DestroyInputComponent();

	// Ensure that we have removed all the delegate bindings that we set up
	StopListeningForPlayerControllerChanges();
	
	TearDownAnimations();

	if (AnimationTickManager)
	{
		AnimationTickManager->RemoveWidget(this);
		AnimationTickManager = nullptr;
	}

	//TODO: Investigate why this would ever be called directly, RemoveFromParent isn't safe to call during GC,
	// as the widget structure may be in a partially destroyed state.

	// If anyone ever calls BeginDestroy explicitly on a widget we need to immediately remove it from
	// the the parent as it may be owned currently by a slate widget.  As long as it's the viewport we're
	// fine.
	RemoveFromParent();

	// If it's not owned by the viewport we need to take more extensive measures.  If the GC widget still
	// exists after this point we should just reset the widget, which will forcefully cause the SObjectWidget
	// to lose access to this UObject.
	TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();
	if ( SafeGCWidget.IsValid() )
	{
		SafeGCWidget->ResetWidget();
	}
}

void UUserWidget::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	
	if ( GetInitializingFromWidgetTree() != 0 )
	{
		// If this is a sub-widget of another UserWidget, default designer flags to match those of the owning widget before initialize.
		if (UUserWidget* OwningUserWidget = GetTypedOuter<UUserWidget>())
		{
#if WITH_EDITOR
			SetDesignerFlags(OwningUserWidget->GetDesignerFlags());	
#endif
			SetPlayerContext(OwningUserWidget->GetPlayerContext());
		}
		Initialize();
	}
}

void UUserWidget::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	UWidget* RootWidget = GetRootWidget();
	if ( RootWidget )
	{
		RootWidget->ReleaseSlateResources(bReleaseChildren);
	}
}

void UUserWidget::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	// We get the GCWidget directly because MyWidget could be the fullscreen host widget if we've been added
	// to the viewport.
	TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();
	if ( SafeGCWidget.IsValid() )
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TAttribute<FLinearColor> ColorBinding = PROPERTY_BINDING(FLinearColor, ColorAndOpacity);
		TAttribute<FSlateColor> ForegroundColorBinding = PROPERTY_BINDING(FSlateColor, ForegroundColor);

		SafeGCWidget->SetColorAndOpacity(ColorBinding);
		SafeGCWidget->SetForegroundColor(ForegroundColorBinding);
		SafeGCWidget->SetPadding(Padding);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UUserWidget::SetColorAndOpacity(FLinearColor InColorAndOpacity)
{
	ColorAndOpacity = InColorAndOpacity;

	TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();
	if ( SafeGCWidget.IsValid() )
	{
		SafeGCWidget->SetColorAndOpacity(ColorAndOpacity);
	}
}

const FLinearColor& UUserWidget::GetColorAndOpacity() const
{
	return ColorAndOpacity;
}

void UUserWidget::SetForegroundColor(FSlateColor InForegroundColor)
{
	ForegroundColor = InForegroundColor;

	TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();
	if ( SafeGCWidget.IsValid() )
	{
		SafeGCWidget->SetForegroundColor(ForegroundColor);
	}
}

const FSlateColor& UUserWidget::GetForegroundColor() const
{
	return ForegroundColor;
}

void UUserWidget::SetPadding(FMargin InPadding)
{
	Padding = InPadding;

	TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();
	if ( SafeGCWidget.IsValid() )
	{
		SafeGCWidget->SetPadding(Padding);
	}
}

FMargin UUserWidget::GetPadding() const
{
	return Padding;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UWorld* UUserWidget::GetWorld() const
{
	if ( UWorld* LastWorld = CachedWorld.Get() )
	{
		return LastWorld;
	}

	if ( HasAllFlags(RF_ClassDefaultObject) )
	{
		// If we are a CDO, we must return nullptr instead of calling Outer->GetWorld() to fool UObject::ImplementsGetWorld.
		return nullptr;
	}

	// Use the Player Context's world, if a specific player context is given, otherwise fall back to
	// following the outer chain.
	if ( PlayerContext.IsValid() )
	{
		if ( UWorld* World = PlayerContext.GetWorld() )
		{
			CachedWorld = World;
			return World;
		}
	}

	// Could be a GameInstance, could be World, could also be a WidgetTree, so we're just going to follow
	// the outer chain to find the world we're in.
	UObject* Outer = GetOuter();

	while ( Outer )
	{
		UWorld* World = Outer->GetWorld();
		if ( World )
		{
			CachedWorld = World;
			return World;
		}

		Outer = Outer->GetOuter();
	}

	return nullptr;
}

TSharedPtr<const FWidgetAnimationState> UUserWidget::GetAnimationState(const UWidgetAnimation* InAnimation) const
{
	for (auto It = ActiveAnimations.CreateConstIterator(); It; ++It)
	{
		TSharedRef<FWidgetAnimationState> State(*It);
		if (State->GetAnimation() == InAnimation && State->IsValid())
		{
			return State;
		}
	};
	return nullptr;
}

TSharedPtr<FWidgetAnimationState> UUserWidget::GetAnimationState(const UWidgetAnimation* InAnimation)
{
	for (auto It = ActiveAnimations.CreateConstIterator(); It; ++It)
	{
		TSharedRef<FWidgetAnimationState> State(*It);
		if (State->GetAnimation() == InAnimation && State->IsValid())
		{
			return State;
		}
	};
	return nullptr;
}

TSharedPtr<FWidgetAnimationState> UUserWidget::GetOrAddAnimationState(UWidgetAnimation* InAnimation)
{
	if (InAnimation && !bStoppingAllAnimations)
	{
		if (!AnimationTickManager)
		{
			AnimationTickManager = UUMGSequenceTickManager::Get(this);
		}

		// Always ensure that this widget's animations are ticked for at least the first frame
		// If this widget is currently offscreen it would very well not be being tracked by the tick manager
		AnimationTickManager->AddWidget(this);

		TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation);
		if (!FoundState || FoundState->IsPendingDelete())
		{
			// Create a new state and initialize it.
			TSharedRef<FWidgetAnimationState> NewState = MakeShared<FWidgetAnimationState>();
			ActiveAnimations.Add(NewState);
			NewState->Initialize(InAnimation, this);

			return NewState;
		}
		else
		{
			return FoundState;
		}
	}

	return nullptr;
}

void UUserWidget::ExecuteQueuedAnimationTransitions()
{
	// In case any users queue animations in respose to animation transitions, operate on a copy array
	TArray<FQueuedWidgetAnimationTransition, TInlineAllocator<8>> CurrentWidgetAnimationTransitions(QueuedWidgetAnimationTransitions);

	for (FQueuedWidgetAnimationTransition& QueuedWidgetAnimationTransition : CurrentWidgetAnimationTransitions)
	{
		switch (QueuedWidgetAnimationTransition.TransitionMode)
		{
		case EQueuedWidgetAnimationMode::Play:
			PlayAnimation(QueuedWidgetAnimationTransition.WidgetAnimation
				, QueuedWidgetAnimationTransition.StartAtTime.GetValue()
				, QueuedWidgetAnimationTransition.NumLoopsToPlay.GetValue()
				, QueuedWidgetAnimationTransition.PlayMode.GetValue()
				, QueuedWidgetAnimationTransition.PlaybackSpeed.GetValue()
				, QueuedWidgetAnimationTransition.bRestoreState.GetValue());
			break;
		case EQueuedWidgetAnimationMode::PlayTo:
			PlayAnimationTimeRange(QueuedWidgetAnimationTransition.WidgetAnimation
				, QueuedWidgetAnimationTransition.StartAtTime.GetValue()
				, QueuedWidgetAnimationTransition.EndAtTime.GetValue()
				, QueuedWidgetAnimationTransition.NumLoopsToPlay.GetValue()
				, QueuedWidgetAnimationTransition.PlayMode.GetValue()
				, QueuedWidgetAnimationTransition.PlaybackSpeed.GetValue()
				, QueuedWidgetAnimationTransition.bRestoreState.GetValue());
			break;
		case EQueuedWidgetAnimationMode::Forward:
			PlayAnimationForward(QueuedWidgetAnimationTransition.WidgetAnimation
				, QueuedWidgetAnimationTransition.PlaybackSpeed.GetValue()
				, QueuedWidgetAnimationTransition.bRestoreState.GetValue());
			break;
		case EQueuedWidgetAnimationMode::Reverse:
			PlayAnimationReverse(QueuedWidgetAnimationTransition.WidgetAnimation
				, QueuedWidgetAnimationTransition.PlaybackSpeed.GetValue()
				, QueuedWidgetAnimationTransition.bRestoreState.GetValue());
			break;
		case EQueuedWidgetAnimationMode::Stop:
			StopAnimation(QueuedWidgetAnimationTransition.WidgetAnimation);
			break;
		case EQueuedWidgetAnimationMode::Pause:
			PauseAnimation(QueuedWidgetAnimationTransition.WidgetAnimation);
			break;
		}
	}

	if (QueuedWidgetAnimationTransitions.Num() > 0)
	{
		QueuedWidgetAnimationTransitions.Empty();
		UpdateCanTick();
	}
}

void UUserWidget::ConditionalTearDownAnimations()
{
	ensureMsgf(!bIsTickingAnimations, TEXT("When active animations are ticked, animations can only be added, not removed"));

	for (auto It = ActiveAnimations.CreateIterator(); It; ++It)
	{
		TSharedRef<FWidgetAnimationState> State(*It);
		if (!State->IsValid())
		{
			It.RemoveCurrent();
		}
		else if (!State->IsStopping())
		{
			State->TearDown();
		}
	}
}

void UUserWidget::TearDownAnimations()
{
	ensureMsgf(!bIsTickingAnimations, TEXT("When active animations are ticked, animations can only be added, not removed"));

	for (auto It = ActiveAnimations.CreateIterator(); It; ++It)
	{
		TSharedRef<FWidgetAnimationState> State(*It);
		State->TearDown();
	}

	ActiveAnimations.Reset();
}

void UUserWidget::DisableAnimations()
{
	for (auto It = ActiveAnimations.CreateIterator(); It; ++It)
	{
		TSharedRef<FWidgetAnimationState> State(*It);
		State->RemoveEvaluationData();
	}
}

void UUserWidget::Invalidate(EInvalidateWidgetReason InvalidateReason)
{
	if (TSharedPtr<SWidget> CachedWidget = GetCachedWidget())
	{
		UpdateCanTick();
		CachedWidget->Invalidate(InvalidateReason);
	}
}

bool UUserWidget::IsPlayingAnimation() const
{
	return ActiveAnimations.Num() > 0;
}

void UUserWidget::QueuePlayAnimation(UWidgetAnimation* InAnimation, float StartAtTime, int32 NumLoopsToPlay, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed, bool bRestoreState)
{
	if (!InAnimation)
	{
		return;
	}

	FQueuedWidgetAnimationTransition* QueuedTransitionPtr = QueuedWidgetAnimationTransitions.FindByPredicate([&](const FQueuedWidgetAnimationTransition& QueuedTransition) { return QueuedTransition.WidgetAnimation == InAnimation; });
	FQueuedWidgetAnimationTransition& QueuedTransition = QueuedTransitionPtr ? *QueuedTransitionPtr : QueuedWidgetAnimationTransitions.AddDefaulted_GetRef();

	QueuedTransition = FQueuedWidgetAnimationTransition();
	QueuedTransition.WidgetAnimation = InAnimation;
	QueuedTransition.TransitionMode = EQueuedWidgetAnimationMode::Play;
	QueuedTransition.StartAtTime = StartAtTime;
	QueuedTransition.NumLoopsToPlay = NumLoopsToPlay;
	QueuedTransition.PlayMode = PlayMode;
	QueuedTransition.PlaybackSpeed = PlaybackSpeed;
	QueuedTransition.bRestoreState = bRestoreState;

	if (!AnimationTickManager)
	{
		AnimationTickManager = UUMGSequenceTickManager::Get(this);
		AnimationTickManager->AddWidget(this);
	}

	UpdateCanTick();
}

void UUserWidget::QueuePlayAnimationTimeRange(UWidgetAnimation* InAnimation, float StartAtTime, float EndAtTime, int32 NumLoopsToPlay, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed, bool bRestoreState)
{
	if (!InAnimation)
	{
		return;
	}

	FQueuedWidgetAnimationTransition* QueuedTransitionPtr = QueuedWidgetAnimationTransitions.FindByPredicate([&](const FQueuedWidgetAnimationTransition& QueuedTransition) { return QueuedTransition.WidgetAnimation == InAnimation; });
	FQueuedWidgetAnimationTransition& QueuedTransition = QueuedTransitionPtr ? *QueuedTransitionPtr : QueuedWidgetAnimationTransitions.AddDefaulted_GetRef();

	QueuedTransition.WidgetAnimation = InAnimation;
	QueuedTransition.TransitionMode = EQueuedWidgetAnimationMode::PlayTo;
	QueuedTransition.StartAtTime = StartAtTime;
	QueuedTransition.EndAtTime = EndAtTime;
	QueuedTransition.NumLoopsToPlay = NumLoopsToPlay;
	QueuedTransition.PlayMode = PlayMode;
	QueuedTransition.PlaybackSpeed = PlaybackSpeed;
	QueuedTransition.bRestoreState = bRestoreState;

	if (!AnimationTickManager)
	{
		AnimationTickManager = UUMGSequenceTickManager::Get(this);
		AnimationTickManager->AddWidget(this);
	}

	UpdateCanTick();
}

void UUserWidget::QueuePlayAnimationForward(UWidgetAnimation* InAnimation, float PlaybackSpeed, bool bRestoreState)
{
	if (!InAnimation)
	{
		return;
	}

	FQueuedWidgetAnimationTransition* QueuedTransitionPtr = QueuedWidgetAnimationTransitions.FindByPredicate([&](const FQueuedWidgetAnimationTransition& QueuedTransition) { return QueuedTransition.WidgetAnimation == InAnimation; });
	FQueuedWidgetAnimationTransition& QueuedTransition = QueuedTransitionPtr ? *QueuedTransitionPtr : QueuedWidgetAnimationTransitions.AddDefaulted_GetRef();

	QueuedTransition.WidgetAnimation = InAnimation;
	QueuedTransition.TransitionMode = EQueuedWidgetAnimationMode::Forward;
	QueuedTransition.PlaybackSpeed = PlaybackSpeed;
	QueuedTransition.bRestoreState = bRestoreState;

	if (!AnimationTickManager)
	{
		AnimationTickManager = UUMGSequenceTickManager::Get(this);
		AnimationTickManager->AddWidget(this);
	}

	UpdateCanTick();
}

void UUserWidget::QueuePlayAnimationReverse(UWidgetAnimation* InAnimation, float PlaybackSpeed, bool bRestoreState)
{
	if (!InAnimation)
	{
		return;
	}

	FQueuedWidgetAnimationTransition* QueuedTransitionPtr = QueuedWidgetAnimationTransitions.FindByPredicate([&](const FQueuedWidgetAnimationTransition& QueuedTransition) { return QueuedTransition.WidgetAnimation == InAnimation; });
	FQueuedWidgetAnimationTransition& QueuedTransition = QueuedTransitionPtr ? *QueuedTransitionPtr : QueuedWidgetAnimationTransitions.AddDefaulted_GetRef();

	QueuedTransition.WidgetAnimation = InAnimation;
	QueuedTransition.TransitionMode = EQueuedWidgetAnimationMode::Reverse;
	QueuedTransition.PlaybackSpeed = PlaybackSpeed;
	QueuedTransition.bRestoreState = bRestoreState;

	if (!AnimationTickManager)
	{
		AnimationTickManager = UUMGSequenceTickManager::Get(this);
		AnimationTickManager->AddWidget(this);
	}

	UpdateCanTick();
}

void UUserWidget::QueueStopAnimation(const UWidgetAnimation* InAnimation)
{
	if (!InAnimation)
	{
		return;
	}

	FQueuedWidgetAnimationTransition* QueuedTransitionPtr = QueuedWidgetAnimationTransitions.FindByPredicate([&](const FQueuedWidgetAnimationTransition& QueuedTransition) { return QueuedTransition.WidgetAnimation == InAnimation; });
	FQueuedWidgetAnimationTransition& QueuedTransition = QueuedTransitionPtr ? *QueuedTransitionPtr : QueuedWidgetAnimationTransitions.AddDefaulted_GetRef();

	QueuedTransition.WidgetAnimation = const_cast<UWidgetAnimation*>(InAnimation);
	QueuedTransition.TransitionMode = EQueuedWidgetAnimationMode::Stop;

	UpdateCanTick();
}

void UUserWidget::QueueStopAllAnimations()
{
	for (FQueuedWidgetAnimationTransition& QueuedWidgetAnimationTransition : QueuedWidgetAnimationTransitions)
	{
		QueuedWidgetAnimationTransition.TransitionMode = EQueuedWidgetAnimationMode::Stop;
	}

	for (TSharedRef<FWidgetAnimationState> State : ActiveAnimations)
	{
		if (State->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
		{
			QueueStopAnimation(State->GetAnimation());
		}
	}

	UpdateCanTick();
}

float UUserWidget::QueuePauseAnimation(const UWidgetAnimation* InAnimation)
{
	if (InAnimation)
	{
		FQueuedWidgetAnimationTransition* QueuedTransitionPtr = QueuedWidgetAnimationTransitions.FindByPredicate([&](const FQueuedWidgetAnimationTransition& QueuedTransition) { return QueuedTransition.WidgetAnimation == InAnimation; });
		FQueuedWidgetAnimationTransition& QueuedTransition = QueuedTransitionPtr ? *QueuedTransitionPtr : QueuedWidgetAnimationTransitions.AddDefaulted_GetRef();

		QueuedTransition.WidgetAnimation = const_cast<UWidgetAnimation*>(InAnimation);
		QueuedTransition.TransitionMode = EQueuedWidgetAnimationMode::Pause;

		UpdateCanTick();

		if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
		{
			return (float)FoundState->GetCurrentTime().AsSeconds();
		}
	}

	return 0;
}

FWidgetAnimationHandle UUserWidget::PlayAnimation(UWidgetAnimation* InAnimation, float StartAtTime, int32 NumberOfLoops, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed, bool bRestoreState)
{
	SCOPED_NAMED_EVENT_TEXT("Widget::PlayAnimation", FColor::Emerald);

	if (TSharedPtr<FWidgetAnimationState> AnimationState = GetOrAddAnimationState(InAnimation))
	{
		FWidgetAnimationStatePlayParams PlayParams;
		PlayParams.StartAtTime = StartAtTime;
		PlayParams.NumLoopsToPlay = NumberOfLoops;
		PlayParams.PlayMode = PlayMode;
		PlayParams.PlaybackSpeed = PlaybackSpeed;
		PlayParams.bRestoreState = bRestoreState;

		AnimationState->Play(PlayParams);

		BroadcastAnimationStartedPlaying(*AnimationState);

		UpdateCanTick();

		return FWidgetAnimationHandle(AnimationState);
	}

	return FWidgetAnimationHandle();
}

FWidgetAnimationHandle UUserWidget::PlayAnimationTimeRange(UWidgetAnimation* InAnimation, float StartAtTime, float EndAtTime, int32 NumberOfLoops, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed, bool bRestoreState)
{
	SCOPED_NAMED_EVENT_TEXT("Widget::PlayAnimationTimeRange", FColor::Emerald);

	if (TSharedPtr<FWidgetAnimationState> AnimationState = GetOrAddAnimationState(InAnimation))
	{
		FWidgetAnimationStatePlayParams PlayParams;
		PlayParams.StartAtTime = StartAtTime;
		PlayParams.EndAtTime = EndAtTime;
		PlayParams.NumLoopsToPlay = NumberOfLoops;
		PlayParams.PlayMode = PlayMode;
		PlayParams.PlaybackSpeed = PlaybackSpeed;
		PlayParams.bRestoreState = bRestoreState;

		AnimationState->Play(PlayParams);

		BroadcastAnimationStartedPlaying(*AnimationState);

		UpdateCanTick();

		return FWidgetAnimationHandle(AnimationState);
	}

	return FWidgetAnimationHandle();
}

FWidgetAnimationHandle UUserWidget::PlayAnimationForward(UWidgetAnimation* InAnimation, float PlaybackSpeed, bool bRestoreState)
{
	TSharedPtr<FWidgetAnimationState> AnimationState = GetAnimationState(InAnimation);

	if (AnimationState && AnimationState->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
	{
		if (!AnimationState->IsPlayingForward())
		{
			AnimationState->Reverse();
		}

		return FWidgetAnimationHandle(AnimationState);
	}

	return PlayAnimation(InAnimation, 0.0f, 1, EUMGSequencePlayMode::Forward, PlaybackSpeed, bRestoreState);
}

FWidgetAnimationHandle UUserWidget::PlayAnimationReverse(UWidgetAnimation* InAnimation, float PlaybackSpeed, bool bRestoreState)
{
	TSharedPtr<FWidgetAnimationState> AnimationState = GetAnimationState(InAnimation);

	if (AnimationState && AnimationState->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
	{
		if (AnimationState->IsPlayingForward())
		{
			AnimationState->Reverse();
		}

		return FWidgetAnimationHandle(AnimationState);
	}

	return PlayAnimation(InAnimation, 0.0f, 1, EUMGSequencePlayMode::Reverse, PlaybackSpeed, bRestoreState);
}

void UUserWidget::StopAnimation(const UWidgetAnimation* InAnimation)
{
	if (InAnimation)
	{
		if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
		{
			FoundState->Stop();

			UpdateCanTick();
		}
	}
}

void UUserWidget::StopAllAnimations()
{
	bStoppingAllAnimations = true;

	for (TSharedRef<FWidgetAnimationState> State : ActiveAnimations)
	{
		if (State->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing)
		{
			State->Stop();
		}
	}

	bStoppingAllAnimations = false;

	UpdateCanTick();
}

float UUserWidget::PauseAnimation(const UWidgetAnimation* InAnimation)
{
	if (InAnimation)
	{
		// @todo UMG sequencer - Restart animations which have had Play called on them?
		if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
		{
			FoundState->Pause();
			return (float)FoundState->GetCurrentTime().AsSeconds();
		}
	}

	return 0;
}

float UUserWidget::GetAnimationCurrentTime(const UWidgetAnimation* InAnimation) const
{
	if (InAnimation)
	{
		if (TSharedPtr<const FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
		{
			return (float)FoundState->GetCurrentTime().AsSeconds();
		}
	}

	return 0;
}

void UUserWidget::SetAnimationCurrentTime(const UWidgetAnimation* InAnimation, float InTime)
{
	if (InAnimation)
	{
		if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
		{
			FoundState->SetCurrentTime(InTime);
		}
	}
}

bool UUserWidget::IsAnimationPlaying(const UWidgetAnimation* InAnimation) const
{
	if (InAnimation)
	{
		if (TSharedPtr<const FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
		{
			return FoundState->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;
		}
	}

	return false;
}

bool UUserWidget::IsAnyAnimationPlaying() const
{
	return ActiveAnimations.Num() > 0;
}

void UUserWidget::SetNumLoopsToPlay(const UWidgetAnimation* InAnimation, int32 InNumLoopsToPlay)
{
	if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
	{
		FoundState->SetNumLoopsToPlay(InNumLoopsToPlay);
	}
}

void UUserWidget::SetPlaybackSpeed(const UWidgetAnimation* InAnimation, float PlaybackSpeed)
{
	if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
	{
		FoundState->SetPlaybackSpeed(PlaybackSpeed);
	}
}

void UUserWidget::ReverseAnimation(const UWidgetAnimation* InAnimation)
{
	if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
	{
		FoundState->Reverse();
	}
}

void UUserWidget::BroadcastAnimationStartedPlaying(FWidgetAnimationState& State)
{
	OnAnimationStartedPlayingEvent.Broadcast(State);

	OnAnimationStarted(State.GetAnimation());

	BroadcastAnimationStateChange(State, EWidgetAnimationEvent::Started);

	if (UUMGSequencePlayer* LegacyPlayer = State.GetLegacyPlayer())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OnAnimationStartedPlaying(*LegacyPlayer);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

bool UUserWidget::IsAnimationPlayingForward(const UWidgetAnimation* InAnimation)
{
	if (InAnimation)
	{
		if (TSharedPtr<FWidgetAnimationState> FoundState = GetAnimationState(InAnimation))
		{
			return FoundState->IsPlayingForward();
		}
	}

	return false;
}

void UUserWidget::BroadcastAnimationFinishedPlaying(FWidgetAnimationState& State)
{
	const FName UserTag = State.GetUserTag();
	const UWidgetAnimation* Animation = State.GetAnimation();
	const EMovieScenePlayerStatus::Type PlayerStatus = State.GetPlaybackStatus();
	UUMGSequencePlayer* LegacyPlayer = State.GetLegacyPlayer();

	OnAnimationFinishedPlayingEvent.Broadcast(State);

	// WARNING: do not use State after this point. OnAnimationFinishedPlayingEvent may have triggered new animations
	// and reallocated the array of animation states.

	OnAnimationFinished(Animation);

	BroadcastAnimationStateChange(Animation, UserTag, EWidgetAnimationEvent::Finished);

	if (LegacyPlayer)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OnAnimationFinishedPlaying(*LegacyPlayer);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (PlayerStatus == EMovieScenePlayerStatus::Stopped && AnimationTickManager)
	{
		AnimationTickManager->AddLatentAction(
				FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UUserWidget::ClearStoppedAnimationStates));
	}

	UpdateCanTick();
}

void UUserWidget::BroadcastAnimationStateChange(const UUMGSequencePlayer& Player, EWidgetAnimationEvent AnimationEvent)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	BroadcastAnimationStateChange(Player.GetAnimation(), Player.GetUserTag(), AnimationEvent);

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UUserWidget::BroadcastAnimationStateChange(const FWidgetAnimationState& State, EWidgetAnimationEvent AnimationEvent)
{
	BroadcastAnimationStateChange(State.GetAnimation(), State.GetUserTag(), AnimationEvent);
}

void UUserWidget::BroadcastAnimationStateChange(const UWidgetAnimation* Animation, FName UserTag, EWidgetAnimationEvent AnimationEvent)
{
	// Make a temporary copy of the animation callbacks so that everyone gets a callback
	// even if they're removed as a result of other calls, we don't want order to matter here.
	TArray<FAnimationEventBinding> TempAnimationCallbacks = AnimationCallbacks;

	for (const FAnimationEventBinding& Binding : TempAnimationCallbacks)
	{
		if (Binding.Animation == Animation && Binding.AnimationEvent == AnimationEvent)
		{
			if (Binding.UserTag == NAME_None || Binding.UserTag == UserTag)
			{
				Binding.Delegate.ExecuteIfBound();
			}
		}
	}
}

void UUserWidget::PlaySound(USoundBase* SoundToPlay)
{
	if (SoundToPlay)
	{
		FSlateSound NewSound;
		NewSound.SetResourceObject(SoundToPlay);
		FSlateApplication::Get().PlaySound(NewSound);
	}
}

bool UUserWidget::SetDesiredFocusWidget(FName WidgetName)
{
	DesiredFocusWidget = FWidgetChild(this, WidgetName);
	return DesiredFocusWidget.GetWidget() != nullptr;
}

bool UUserWidget::SetDesiredFocusWidget(UWidget* Widget)
{
	// If the widget is invalid or not contained within our known widget tree, return false
	if (!Widget)
	{
		return false;
	}

	if (WidgetTree)
	{
		TArray<UWidget*> AllWidgets;
		WidgetTree->GetAllWidgets(AllWidgets);

		if (!AllWidgets.Contains(Widget))
		{
			return false;
		}
	}
	// Otherwise, we should still remember the name to allow to resolve later against a valid tree.
	return SetDesiredFocusWidget(Widget->GetFName());
}


FName UUserWidget::GetDesiredFocusWidgetName() const
{
	return DesiredFocusWidget.GetFName();
}

UWidget* UUserWidget::GetDesiredFocusWidget() const
{
	if (WidgetTree)
	{
		// Ensure that we are always resolving against the widget tree
		return const_cast<FWidgetChild&>(DesiredFocusWidget).Resolve(WidgetTree);
	}
	return nullptr;
}

UWidget* UUserWidget::GetWidgetHandle(TSharedRef<SWidget> InWidget)
{
	return WidgetTree->FindWidget(InWidget);
}

TSharedRef<SWidget> UUserWidget::RebuildWidget()
{
	check(!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject));
	
	// In the event this widget is replaced in memory by the blueprint compiler update
	// the widget won't be properly initialized, so we ensure it's initialized and initialize
	// it if it hasn't been.
	if ( !bInitialized )
	{
		Initialize();
	}

	// Setup the player context on sub user widgets, if we have a valid context
	if (PlayerContext.IsValid())
	{
		WidgetTree->ForEachWidget([&] (UWidget* Widget) {
			if ( UUserWidget* UserWidget = Cast<UUserWidget>(Widget) )
			{
				UserWidget->UpdatePlayerContextIfInvalid(PlayerContext);
			}
		});
	}

	// Add the first component to the root of the widget surface.
	TSharedRef<SWidget> UserRootWidget = WidgetTree->RootWidget ? WidgetTree->RootWidget->TakeWidget() : TSharedRef<SWidget>(SNew(SSpacer));

	return UserRootWidget;
}

void UUserWidget::OnWidgetRebuilt()
{
	// When a user widget is rebuilt we can safely initialize the navigation now since all the slate
	// widgets should be held onto by a smart pointer at this point.
	BuildNavigation();
	WidgetTree->ForEachWidget([&] (UWidget* Widget) {
		Widget->BuildNavigation();
	});

	if (!IsDesignTime())
	{
		// Notify the widget to run per-construct.
		NativePreConstruct();

		// Notify the widget that it has been constructed.
		NativeConstruct();
	}
#if WITH_EDITOR
	else if ( HasAnyDesignerFlags(EWidgetDesignFlags::ExecutePreConstruct) )
	{
		bool bCanCallPreConstruct = true;
		if (UWidgetBlueprintGeneratedClass* GeneratedBPClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
		{
			bCanCallPreConstruct = GeneratedBPClass->bCanCallPreConstruct;
		}

		if (bCanCallPreConstruct)
		{
			NativePreConstruct();
		}
	}
#endif
}

TSharedPtr<SWidget> UUserWidget::GetSlateWidgetFromName(const FName& Name) const
{
	UWidget* WidgetObject = GetWidgetFromName(Name);
	return WidgetObject ? WidgetObject->GetCachedWidget() : TSharedPtr<SWidget>();
}

UWidget* UUserWidget::GetWidgetFromName(const FName& Name) const
{
	return WidgetTree ? WidgetTree->FindWidget(Name) : nullptr;
}

void UUserWidget::GetSlotNames(TArray<FName>& SlotNames) const
{
	// Only do this if this widget is of a blueprint class
	if (const UWidgetBlueprintGeneratedClass* BGClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
	{
		SlotNames.Append(BGClass->InstanceNamedSlots);
	}
	else if (WidgetTree) // For non-blueprint widget blueprints we have to go through the widget tree to locate the named slots dynamically.
	{
		// TODO: This code is probably defunct now, that we always have a BPGC?
		
		WidgetTree->ForEachWidget([&SlotNames] (UWidget* Widget) {
			if ( Widget && Widget->IsA<UNamedSlot>() )
			{
				SlotNames.Add(Widget->GetFName());
			}
		});
	}
}

UWidget* UUserWidget::GetContentForSlot(FName SlotName) const
{
	for ( const FNamedSlotBinding& Binding : NamedSlotBindings )
	{
		if ( Binding.Name == SlotName )
		{
			return Binding.Content;
		}
	}

	return nullptr;
}

void UUserWidget::SetContentForSlot(FName SlotName, UWidget* Content)
{
	bool bFoundExistingSlot = false;

	bool bIsMissingSlot = false;
	// Dynamically insert the new widget into the hierarchy if it exists.
	if (WidgetTree)
	{
		ensureMsgf(!HasAnyFlags(RF_ClassDefaultObject), TEXT("The Widget CDO is not expected to ever have a valid widget tree."));

		if (UNamedSlot* NamedSlot = Cast<UNamedSlot>(WidgetTree->FindWidget(SlotName)))
		{
			NamedSlot->ClearChildren();

			if (Content)
			{
				NamedSlot->AddChild(Content);
			}
		}
		else
		{
			bIsMissingSlot = true;
		}
	}

	// Find the binding in the existing set and replace the content for that binding.
	for ( int32 BindingIndex = 0; BindingIndex < NamedSlotBindings.Num(); BindingIndex++ )
	{
		FNamedSlotBinding& Binding = NamedSlotBindings[BindingIndex];

		if ( Binding.Name == SlotName )
		{
			bFoundExistingSlot = true;

			if ( Content && !bIsMissingSlot)
			{
				Binding.Content = Content;
			}
			else
			{
				NamedSlotBindings.RemoveAt(BindingIndex);
			}

			break;
		}
	}

	if ( !bFoundExistingSlot && Content && !bIsMissingSlot)
	{
		// Add the new binding to the list of bindings.
		FNamedSlotBinding NewBinding;
		NewBinding.Name = SlotName;
		NewBinding.Content = Content;

		NamedSlotBindings.Add(NewBinding);
	}

}

UWidget* UUserWidget::GetRootWidget() const
{
	if ( WidgetTree )
	{
		return WidgetTree->RootWidget;
	}

	return nullptr;
}

void UUserWidget::AddToViewport(int32 ZOrder)
{
	if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
	{
		FGameViewportWidgetSlot ViewportSlot;
		if (bIsManagedByGameViewportSubsystem)
		{
			ViewportSlot = Subsystem->GetWidgetSlot(this);
		}
		ViewportSlot.ZOrder = ZOrder;
		Subsystem->AddWidget(this, ViewportSlot);
	}
}

bool UUserWidget::AddToPlayerScreen(int32 ZOrder)
{
	if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
	{
		if (ULocalPlayer* LocalPlayer = GetOwningLocalPlayer())
		{
			FGameViewportWidgetSlot ViewportSlot;
			if (bIsManagedByGameViewportSubsystem)
			{
				ViewportSlot = Subsystem->GetWidgetSlot(this);
			}
			ViewportSlot.ZOrder = ZOrder;
			Subsystem->AddWidgetForPlayer(this, GetOwningLocalPlayer(), ViewportSlot);
			return true;
		}
		else
		{
			FMessageLog("PIE").Error(LOCTEXT("AddToPlayerScreen_NoPlayer", "AddToPlayerScreen Failed.  No Owning Player!"));
		}
	}
	return false;
}

void UUserWidget::RemoveFromViewport()
{
	RemoveFromParent();
}

bool UUserWidget::GetIsVisible() const
{
	return IsInViewport();
}

void UUserWidget::SetVisibility(ESlateVisibility InVisibility)
{
	ESlateVisibility OldVisibility = GetVisibility();

	Super::SetVisibility(InVisibility);

	if (OldVisibility != GetVisibility())
	{
		OnNativeVisibilityChanged.Broadcast(InVisibility);
		OnVisibilityChanged.Broadcast(InVisibility);
	}
}

void UUserWidget::SetPlayerContext(const FLocalPlayerContext& InPlayerContext)
{
	// Pop the input component off of the current player controller, if we have one...
	StopProcessingInputScriptDelegates();
	StopListeningForPlayerControllerChanges();
	
	PlayerContext = InPlayerContext;
	CachedWorld.Reset();

	if (WidgetTree)
	{
		WidgetTree->ForEachWidget(
			[&InPlayerContext] (UWidget* Widget) 
			{
				if (UUserWidget* UserWidget = Cast<UUserWidget>(Widget))
				{
					UserWidget->SetPlayerContext(InPlayerContext);
				}
			});
	}
	
	// ... and start processing input again and push onto this new player controller's input stack if we can
	StartListeningForPlayerControllerChanges();
	StartProcessingInputScriptDelegates();
}

const FLocalPlayerContext& UUserWidget::GetPlayerContext() const
{
	return PlayerContext;
}

ULocalPlayer* UUserWidget::GetOwningLocalPlayer() const
{
	if (PlayerContext.IsValid())
	{
		return PlayerContext.GetLocalPlayer();
	}
	return nullptr;
}

void UUserWidget::SetOwningLocalPlayer(ULocalPlayer* LocalPlayer)
{
	if (LocalPlayer)
	{
		// Pop the input component off of the current player controller, if we have one...
		StopProcessingInputScriptDelegates();

		// Remove bindings from any callbacks for PC changes on the old local player
		StopListeningForPlayerControllerChanges();
		
		PlayerContext = FLocalPlayerContext(LocalPlayer, GetWorld());
		CachedWorld.Reset();

		// And start listening to the player controller changes again
		StartListeningForPlayerControllerChanges();

		// ... and start processing input again and push onto this new player controller's input stack if we can
		StartProcessingInputScriptDelegates();
	}
}

APlayerController* UUserWidget::GetOwningPlayer() const
{
	return PlayerContext.IsValid() ? PlayerContext.GetPlayerController() : nullptr;
}

void UUserWidget::SetOwningPlayer(APlayerController* LocalPlayerController)
{
	if (LocalPlayerController && LocalPlayerController->IsLocalController())
	{
		// Pop the input component off of the current player controller, if we have one...
		StopProcessingInputScriptDelegates();
		
		PlayerContext = FLocalPlayerContext(LocalPlayerController);
		CachedWorld.Reset();

		// ... and start processing input again and push onto this new player controller's input stack if we can
		StartProcessingInputScriptDelegates();
	}
}

APawn* UUserWidget::GetOwningPlayerPawn() const
{
	if (APlayerController* PC = GetOwningPlayer())
	{
		return PC->GetPawn();
	}

	return nullptr;
}

APlayerCameraManager* UUserWidget::GetOwningPlayerCameraManager() const
{
	if (APlayerController* PC = GetOwningPlayer())
	{
		return PC->PlayerCameraManager;
	}

	return nullptr;
}

void UUserWidget::SetPositionInViewport(FVector2D Position, bool bRemoveDPIScale)
{
	if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
	{
		if (bIsManagedByGameViewportSubsystem)
		{
			FGameViewportWidgetSlot ViewportSlot = Subsystem->GetWidgetSlot(this);
			ViewportSlot = UGameViewportSubsystem::SetWidgetSlotPosition(ViewportSlot, this, Position, bRemoveDPIScale);
			Subsystem->SetWidgetSlot(this, ViewportSlot);
		}
		else
		{
			FGameViewportWidgetSlot ViewportSlot = UGameViewportSubsystem::SetWidgetSlotPosition(FGameViewportWidgetSlot(), this, Position, bRemoveDPIScale);
			Subsystem->SetWidgetSlot(this, ViewportSlot);
		}
	}
}

void UUserWidget::SetDesiredSizeInViewport(FVector2D DesiredSize)
{
	if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
	{
		if (bIsManagedByGameViewportSubsystem)
		{
			FGameViewportWidgetSlot ViewportSlot = Subsystem->GetWidgetSlot(this);
			ViewportSlot = UGameViewportSubsystem::SetWidgetSlotDesiredSize(ViewportSlot, DesiredSize);
			Subsystem->SetWidgetSlot(this, ViewportSlot);
		}
		else
		{
			FGameViewportWidgetSlot ViewportSlot = UGameViewportSubsystem::SetWidgetSlotDesiredSize(FGameViewportWidgetSlot(), DesiredSize);
			Subsystem->SetWidgetSlot(this, ViewportSlot);
		}
	}
}

void UUserWidget::SetAnchorsInViewport(FAnchors Anchors)
{
	if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
	{
		if (bIsManagedByGameViewportSubsystem)
		{
			FGameViewportWidgetSlot ViewportSlot = Subsystem->GetWidgetSlot(this);
			if (ViewportSlot.Anchors != Anchors)
			{
				ViewportSlot.Anchors = Anchors;
				Subsystem->SetWidgetSlot(this, ViewportSlot);
			}
		}
		else
		{
			FGameViewportWidgetSlot ViewportSlot;
			ViewportSlot.Anchors = Anchors;
			Subsystem->SetWidgetSlot(this, ViewportSlot);
		}
	}
}

void UUserWidget::SetAlignmentInViewport(FVector2D Alignment)
{
	if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
	{
		if (bIsManagedByGameViewportSubsystem)
		{
			FGameViewportWidgetSlot ViewportSlot = Subsystem->GetWidgetSlot(this);
			if (ViewportSlot.Alignment != Alignment)
			{
				ViewportSlot.Alignment = Alignment;
				Subsystem->SetWidgetSlot(this, ViewportSlot);
			}
		}
		else
		{
			FGameViewportWidgetSlot ViewportSlot;
			ViewportSlot.Alignment = Alignment;
			Subsystem->SetWidgetSlot(this, ViewportSlot);
		}
	}
}

FMargin UUserWidget::GetFullScreenOffset() const
{
	if (bIsManagedByGameViewportSubsystem)
	{
		if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
		{
			return Subsystem->GetWidgetSlot(this).Offsets;
		}
	}
	return FGameViewportWidgetSlot().Offsets;
}

FAnchors UUserWidget::GetAnchorsInViewport() const
{
	if (bIsManagedByGameViewportSubsystem)
	{
		if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
		{
			return Subsystem->GetWidgetSlot(this).Anchors;
		}
	}
	return FGameViewportWidgetSlot().Anchors;
}

FVector2D UUserWidget::GetAlignmentInViewport() const
{
	if (bIsManagedByGameViewportSubsystem)
	{
		if (UGameViewportSubsystem* Subsystem = UGameViewportSubsystem::Get())
		{
			return Subsystem->GetWidgetSlot(this).Alignment;
		}
	}
	return FGameViewportWidgetSlot().Alignment;
}

void UUserWidget::RemoveObsoleteBindings(const TArray<FName>& NamedSlots)
{
	for (int32 BindingIndex = 0; BindingIndex < NamedSlotBindings.Num(); BindingIndex++)
	{
		const FNamedSlotBinding& Binding = NamedSlotBindings[BindingIndex];

		if (!NamedSlots.Contains(Binding.Name))
		{
			NamedSlotBindings.RemoveAt(BindingIndex);
			BindingIndex--;
		}
	}
}

#if WITH_EDITOR

const FText UUserWidget::GetPaletteCategory()
{
	return PaletteCategory;
}

void UUserWidget::SetDesignerFlags(EWidgetDesignFlags NewFlags)
{
	UWidget::SetDesignerFlags(NewFlags);

	if (WidgetTree)
	{
		if (WidgetTree->RootWidget)
		{
			WidgetTree->RootWidget->SetDesignerFlags(NewFlags);
		}
	}
}

void UUserWidget::OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs)
{
	Super::OnDesignerChanged(EventArgs);

	if ( ensure(WidgetTree) )
	{
		WidgetTree->ForEachWidget([&EventArgs] (UWidget* Widget) {
			Widget->OnDesignerChanged(EventArgs);
		});
	}
}

void UUserWidget::ValidateBlueprint(const UWidgetTree& BlueprintWidgetTree, IWidgetCompilerLog& CompileLog) const
{
	ValidateCompiledDefaults(CompileLog);
	ValidateCompiledWidgetTree(BlueprintWidgetTree, CompileLog);
	BlueprintWidgetTree.ForEachWidget(
		[&CompileLog] (UWidget* Widget)
		{
			Widget->ValidateCompiledDefaults(CompileLog);
		});
}

void UUserWidget::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	static FName DesiredFocusWidgetPropertyName(GET_MEMBER_NAME_CHECKED(UUserWidget, DesiredFocusWidget));
	if (PropertyName == DesiredFocusWidgetPropertyName)
	{
		if (UWidgetBlueprintGeneratedClass* BGClass = GetWidgetTreeOwningClass())
		{
			if (UUserWidget* UserWidgetCDO = BGClass->GetDefaultObject<UUserWidget>())
			{
				// We cannot use the Widget Ptr as we need to find the widget with the same name in the CDO
				UserWidgetCDO->SetDesiredFocusWidget(DesiredFocusWidget.GetFName());
			}
		}
	}

	if ( PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive )
	{
		TSharedPtr<SWidget> SafeWidget = GetCachedWidget();
		if ( SafeWidget.IsValid() )
		{
			// Re-Run execute PreConstruct when we get a post edit property change, to do something
			// akin to running Sync Properties, so users don't have to recompile to see updates.
			NativePreConstruct();
		}
	}
}

void UUserWidget::AssignGUIDToBindings()
{
	if (UWidgetBlueprintGeneratedClass* BGClass = GetWidgetTreeOwningClass())
	{
		for (int32 BindingIndex = 0; BindingIndex < NamedSlotBindings.Num(); BindingIndex++)
		{
			FNamedSlotBinding& Binding = NamedSlotBindings[BindingIndex];
			if (BGClass->NamedSlotsWithID.Contains(Binding.Name))
			{
				Binding.Guid = BGClass->NamedSlotsWithID[Binding.Name];
			}
		}
	}
}

void UUserWidget::UpdateBindingForSlot(FName SlotName)
{
	if (UWidgetBlueprintGeneratedClass* BGClass = GetWidgetTreeOwningClass())
	{
		if (BGClass->NamedSlotsWithID.Contains(SlotName))
		{
			for (FNamedSlotBinding& Binding : NamedSlotBindings)
			{
				if (BGClass->NamedSlotsWithID[SlotName] == Binding.Guid && !BGClass->NamedSlotsWithID.Contains(Binding.Name))
				{
					Binding.Name = SlotName;
				}
			}
		}
	}
}
#endif

void UUserWidget::OnAnimationStarted_Implementation(const UWidgetAnimation* Animation)
{

}

void UUserWidget::OnAnimationFinished_Implementation(const UWidgetAnimation* Animation)
{

}

void UUserWidget::BindToAnimationStarted(UWidgetAnimation* InAnimation, FWidgetAnimationDynamicEvent InDelegate)
{
	FAnimationEventBinding Binding;
	Binding.Animation = InAnimation;
	Binding.Delegate = InDelegate;
	Binding.AnimationEvent = EWidgetAnimationEvent::Started;

	AnimationCallbacks.Add(Binding);
}

void UUserWidget::UnbindFromAnimationStarted(UWidgetAnimation* InAnimation, FWidgetAnimationDynamicEvent InDelegate)
{
	AnimationCallbacks.RemoveAll([InAnimation, &InDelegate](FAnimationEventBinding& InBinding) {
		return InBinding.Animation == InAnimation && InBinding.Delegate == InDelegate && InBinding.AnimationEvent == EWidgetAnimationEvent::Started;
	});
}

void UUserWidget::UnbindAllFromAnimationStarted(UWidgetAnimation* InAnimation)
{
	AnimationCallbacks.RemoveAll([InAnimation](FAnimationEventBinding& InBinding) {
		return InBinding.Animation == InAnimation && InBinding.AnimationEvent == EWidgetAnimationEvent::Started;
	});
}

void UUserWidget::UnbindAllFromAnimationFinished(UWidgetAnimation* InAnimation)
{
	AnimationCallbacks.RemoveAll([InAnimation](FAnimationEventBinding& InBinding) {
		return InBinding.Animation == InAnimation && InBinding.AnimationEvent == EWidgetAnimationEvent::Finished;
	});
}

void UUserWidget::BindToAnimationFinished(UWidgetAnimation* InAnimation, FWidgetAnimationDynamicEvent InDelegate)
{
	FAnimationEventBinding Binding;
	Binding.Animation = InAnimation;
	Binding.Delegate = InDelegate;
	Binding.AnimationEvent = EWidgetAnimationEvent::Finished;

	AnimationCallbacks.Add(Binding);
}

void UUserWidget::UnbindFromAnimationFinished(UWidgetAnimation* InAnimation, FWidgetAnimationDynamicEvent InDelegate)
{
	AnimationCallbacks.RemoveAll([InAnimation, &InDelegate](FAnimationEventBinding& InBinding) {
		return InBinding.Animation == InAnimation && InBinding.Delegate == InDelegate && InBinding.AnimationEvent == EWidgetAnimationEvent::Finished;
	});
}

void UUserWidget::BindToAnimationEvent(UWidgetAnimation* InAnimation, FWidgetAnimationDynamicEvent InDelegate, EWidgetAnimationEvent AnimationEvent, FName UserTag)
{
	FAnimationEventBinding Binding;
	Binding.Animation = InAnimation;
	Binding.Delegate = InDelegate;
	Binding.AnimationEvent = AnimationEvent;
	Binding.UserTag = UserTag;

	AnimationCallbacks.Add(Binding);
}

// Native handling for SObjectWidget

void UUserWidget::NativeOnInitialized()
{
	CreateInputComponent();
	
	if (UWidgetBlueprintGeneratedClass* BPClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
	{
		BPClass->ForEachExtension([this](UWidgetBlueprintGeneratedClassExtension* Extension)
			{
				Extension->Initialize(this);
			});
	}

	// Extension can add other extensions. Use index loop to initialize them all.
	for (int32 Index = 0; Index < Extensions.Num(); ++Index)
	{
		UUserWidgetExtension* Extension = Extensions[Index];
		check(Extension);
		Extension->Initialize();
	}

	OnInitialized();
}

void UUserWidget::NativePreConstruct()
{
	LLM_SCOPE_BYTAG(UI_UMG);
	const bool bIsDesignTime = IsDesignTime();
	if (UWidgetBlueprintGeneratedClass* BPClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
	{
		BPClass->ForEachExtension([this, bIsDesignTime](UWidgetBlueprintGeneratedClassExtension* Extension)
			{
				Extension->PreConstruct(this, bIsDesignTime);
			});
	}

	bAreExtensionsPreConstructed = true;
	// Extension can add other extensions. Use index loop to initialize them all.
	TArray<UUserWidgetExtension*, TInlineAllocator<32>> LocalExtensions;
	LocalExtensions.Append(Extensions);
	for (UUserWidgetExtension* Extension : LocalExtensions)
	{
		check(Extension);
		Extension->PreConstruct(bIsDesignTime);
	}

	DesiredFocusWidget.Resolve(WidgetTree);

	PreConstruct(bIsDesignTime);
}

void UUserWidget::NativeConstruct()
{
	LLM_SCOPE_BYTAG(UI_UMG);

	// Upon construction, allow for our input delegates to be called
	StartProcessingInputScriptDelegates();
	StartListeningForPlayerControllerChanges();
	
	if (UWidgetBlueprintGeneratedClass* BPClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
	{
		BPClass->ForEachExtension([this](UWidgetBlueprintGeneratedClassExtension* Extension)
			{
				Extension->Construct(this);
			});
	}

	// Extension can add other extensions.
	//check(bAreExtensionsConstructed == false);
	bAreExtensionsConstructed = true;
	if (Extensions.Num() > 0)
	{
		TArray<UUserWidgetExtension*, TInlineAllocator<32>> LocalExtensions;
		LocalExtensions.Append(Extensions);
		for (UUserWidgetExtension* Extension : LocalExtensions)
		{
			check(Extension);
			Extension->Construct();
		}
	}

	Construct();
	UpdateCanTick();
}

void UUserWidget::NativeDestruct()
{
	// On NativeDestruct we want to stop processing any input delegates so that
	// when this widget isn't visible they don't fire. But, the widget might be pushed
	// back onto the viewport again, so we don't want to completely unbind all delegates yet. 
	StopProcessingInputScriptDelegates();
	StopListeningForPlayerControllerChanges();
	
	OnNativeDestruct.Broadcast(this);

	Destruct();

	// Extension can remove other extensions.
	bAreExtensionsConstructed = false; // To prevent calling Destruct on the same extension if it's removed by another extension.
	bAreExtensionsPreConstructed = false;
	if (Extensions.Num() > 0)
	{
		TArray<UUserWidgetExtension*, TInlineAllocator<32>> LocalExtensions;
		LocalExtensions.Append(Extensions);
		for (UUserWidgetExtension* Extension : LocalExtensions)
		{
			check(Extension);
			Extension->Destruct();
		}
	}

	if (UWidgetBlueprintGeneratedClass* BPClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
	{
		BPClass->ForEachExtension([this](UWidgetBlueprintGeneratedClassExtension* Extension)
			{
				Extension->Destruct(this);
			});
	}
}

void UUserWidget::CreateInputComponent()
{
	// We cannot bind to any input events during design time.
	if (IsDesignTime())
	{
		return;
	}
	
	// Just an editor-time sanity check for this behavior.
	// bAutomaticallyRegisterInputOnConstruction should be true if this widget or any of its
	// BP parent classes have input nodes in their graphs which we want to set up
	// automatic bindings for.
#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		const UUserWidget* DefaultWidget = CastChecked<UUserWidget>(GetClass()->GetDefaultObject());
		ensure(bAutomaticallyRegisterInputOnConstruction == DefaultWidget->bAutomaticallyRegisterInputOnConstruction);
	}
#endif	// #if WITH_EDITOR

	// If bAutomaticallyRegisterInputOnConstruction is false, then there are no input delegates in this widget
	// that need to be processed. There is no need to create an input component
	if (!bAutomaticallyRegisterInputOnConstruction)
	{
		return;
	}
	
	if (const APlayerController* Controller = GetOwningPlayer())
	{
		// Use the existing PC's input class, or fallback to the project default. We should use the existing class
		// instead of just the default one because if you have a plugin that has a PC with a different default input
		// class then this would fail
		const UClass* InputClass = Controller->InputComponent ? Controller->InputComponent->GetClass() : UInputSettings::GetDefaultInputComponentClass();
		InputComponent = NewObject<UInputComponent>( this, InputClass, NAME_None, RF_Transient );

		// TODO: The input consumption and priority behaviors of the component should be driven
		// not by these properties, but by something like the Z-order of the widget. This way,
		// you could create some "UI Input Actions" and have them routed in a more deterministic
		// way. UE-306592
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InputComponent->bBlockInput = bStopAction;
		InputComponent->Priority = Priority;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		UInputDelegateBinding::BindInputDelegates(GetClass(), InputComponent, this);
	}
	else
	{
		FMessageLog("PIE").Info(FText::Format(LOCTEXT("NoInputListeningWithoutPlayerController", "Unable to listen to input actions without a player controller in {0}."), FText::FromName(GetClass()->GetFName())));
	}
}

void UUserWidget::StartProcessingInputScriptDelegates()
{
	// The input component should always be valid if bAutomaticallyRegisterInputOnConstruction is true,
	// because it would have been created in UUserWidget::CreateInputComponent
	if (!InputComponent)
	{
		return;
	}

	// Again, we should only ever have a valid input component if we have a valid player controller.
	APlayerController* Controller = GetOwningPlayer();
	if (!ensure(Controller))
	{
		return;
	}

	// Actually push the input component to the stack. This is what will make the input systems
	// actually call the delegates bound in this graph if the input is activated
	Controller->PushInputComponent(InputComponent);
}

void UUserWidget::StopProcessingInputScriptDelegates()
{
	// The input component should always be valid if bAutomaticallyRegisterInputOnConstruction is true,
	// because it would have been created in UUserWidget::CreateInputComponent
	if (!InputComponent)
	{
		return;
	}

	// Again, we should only ever have a valid input component if we have a valid player controller.
	APlayerController* Controller = GetOwningPlayer();
	if (!Controller)
	{
		return;
	}

	// Pop this component off of the input stack, which will stop the input delegates in the graph
	// from being called.
	Controller->PopInputComponent(InputComponent);
}

void UUserWidget::DestroyInputComponent()
{	
	if (!InputComponent)
	{
		return;
	}

	// Ensure that the component is removed from the owning controller
	if (APlayerController* Controller = GetOwningPlayer())
	{
		Controller->PopInputComponent(InputComponent);
	}

	// Unbind all the action delegates which were bound to this widget
	for (int32 ExistingIndex = InputComponent->GetNumActionBindings() - 1; ExistingIndex >= 0; --ExistingIndex)
	{
		InputComponent->RemoveActionBinding(ExistingIndex);
	}
	InputComponent->ClearActionBindings();

	// Destroy the input component
	InputComponent->MarkAsGarbage();
	InputComponent = nullptr;
}

void UUserWidget::StartListeningForPlayerControllerChanges()
{
	if (ULocalPlayer* LP = GetOwningLocalPlayer())
	{
		LP->OnPlayerControllerChanged().AddUObject(this, &UUserWidget::HandleOwningLocalPlayerControllerChanged);
	}
}

void UUserWidget::StopListeningForPlayerControllerChanges()
{
	if (ULocalPlayer* LP = GetOwningLocalPlayer())
	{
		LP->OnPlayerControllerChanged().RemoveAll(this);
	}
}

void UUserWidget::HandleOwningLocalPlayerControllerChanged(APlayerController* NewPlayerController)
{
	// If the controller is the same, then we don't need to do anything.
	if (NewPlayerController == GetOwningPlayer())
	{
		return;
	}

	// SetOwningPlayer will stop listening to input delegates on the old PC, and then start again on the new PC
	SetOwningPlayer(NewPlayerController);
}

void UUserWidget::NativeTick(const FGeometry& MyGeometry, float InDeltaTime)
{ 
	// If this ensure is hit it is likely UpdateCanTick as not called somewhere
	if(ensureMsgf(TickFrequency != EWidgetTickFrequency::Never, TEXT("SObjectWidget and UUserWidget have mismatching tick states or UUserWidget::NativeTick was called manually (Never do this)")))
	{
		GInitRunaway();

		// Extension can be added while ticking another extension.
		//This loop does guarantee that they will all be updated this frame, if it's the case,  but it will not crash.
		for (int32 Index = 0; Index < Extensions.Num(); ++Index)
		{
			Extensions[Index]->Tick(MyGeometry, InDeltaTime);
		}

#if WITH_EDITOR
		const bool bTickAnimations = !IsDesignTime();
#else
		const bool bTickAnimations = true;
#endif
		if (bTickAnimations)
		{

			if (AnimationTickManager)
			{
				AnimationTickManager->OnWidgetTicked(this);
			}

			UWorld* World = GetWorld();
			if (World)
			{
				// Update any latent actions we have for this actor
				World->GetLatentActionManager().ProcessLatentActions(this, InDeltaTime);
			}
		}

		if (bHasScriptImplementedTick)
		{
			Tick(MyGeometry, InDeltaTime);
		}
	}
}

void UUserWidget::TickActionsAndAnimation(float InDeltaTime)
{
	// Don't tick the animation if inside of a PostLoad
	if (FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return;
	}

	if (!ensureMsgf(!bIsTickingAnimations, TEXT("Re-entrant animation ticking detected!")))
	{
		return;
	}

	TGuardValue TickingGuard(bIsTickingAnimations, true);

	ExecuteQueuedAnimationTransitions();

	// Update active widget animations. None will be removed here, but new
	// ones can be added during the tick, if one animation ends and triggers
	// starting another animation. So iterate with an index to keep going 
	// when the array size increases.
	for (int32 Index = 0; Index < ActiveAnimations.Num(); ++Index)
	{
		TSharedRef<FWidgetAnimationState> State = ActiveAnimations[Index];
		State->Tick(InDeltaTime);
	}
}

void UUserWidget::FlushAnimations()
{
	UUMGSequenceTickManager::Get(this)->ForceFlush();
}

void UUserWidget::CancelLatentActions()
{
	UWorld* World = GetWorld();
	if (World)
	{
		World->GetLatentActionManager().RemoveActionsForObject(this);
		World->GetTimerManager().ClearAllTimersForObject(this);
		UpdateCanTick();
	}
}

void UUserWidget::StopAnimationsAndLatentActions()
{
	StopAllAnimations();
	CancelLatentActions();
}

void UUserWidget::ListenForInputAction( FName ActionName, TEnumAsByte< EInputEvent > EventType, bool bConsume, FOnInputAction Callback )
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ( !InputComponent )
	{
		InitializeInputComponent();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if ( InputComponent )
	{
		FInputActionBinding NewBinding( ActionName, EventType.GetValue() );
		NewBinding.bConsumeInput = bConsume;
		NewBinding.ActionDelegate.GetDelegateForManualSet().BindUObject( this, &ThisClass::OnInputAction, Callback );

		InputComponent->AddActionBinding( NewBinding );
	}
}

void UUserWidget::StopListeningForInputAction( FName ActionName, TEnumAsByte< EInputEvent > EventType )
{
	if ( InputComponent )
	{
		for ( int32 ExistingIndex = InputComponent->GetNumActionBindings() - 1; ExistingIndex >= 0; --ExistingIndex )
		{
			const FInputActionBinding& ExistingBind = InputComponent->GetActionBinding( ExistingIndex );
			if ( ExistingBind.GetActionName() == ActionName && ExistingBind.KeyEvent == EventType )
			{
				InputComponent->RemoveActionBinding( ExistingIndex );
			}
		}
	}
}

void UUserWidget::StopListeningForAllInputActions()
{
	if ( InputComponent )
	{
		for ( int32 ExistingIndex = InputComponent->GetNumActionBindings() - 1; ExistingIndex >= 0; --ExistingIndex )
		{
			InputComponent->RemoveActionBinding( ExistingIndex );
		}

		UnregisterInputComponent();

		InputComponent->ClearActionBindings();
		InputComponent->MarkAsGarbage();
		InputComponent = nullptr;
	}
}

bool UUserWidget::IsListeningForInputAction( FName ActionName ) const
{
	bool bResult = false;
	if ( InputComponent )
	{
		for ( int32 ExistingIndex = InputComponent->GetNumActionBindings() - 1; ExistingIndex >= 0; --ExistingIndex )
		{
			const FInputActionBinding& ExistingBind = InputComponent->GetActionBinding( ExistingIndex );
			if ( ExistingBind.GetActionName() == ActionName )
			{
				bResult = true;
				break;
			}
		}
	}

	return bResult;
}

void UUserWidget::RegisterInputComponent()
{
	if ( InputComponent )
	{
		if ( APlayerController* Controller = GetOwningPlayer() )
		{
			Controller->PushInputComponent(InputComponent);
		}
	}
}

void UUserWidget::UnregisterInputComponent()
{
	if ( InputComponent )
	{
		if ( APlayerController* Controller = GetOwningPlayer() )
		{
			Controller->PopInputComponent(InputComponent);
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UUserWidget::SetInputActionPriority( int32 NewPriority )
{
	if ( InputComponent )
	{
		Priority = NewPriority;
		InputComponent->Priority = Priority;
	}
}

int32 UUserWidget::GetInputActionPriority() const
{
	return Priority;
}

void UUserWidget::SetInputActionBlocking( bool bShouldBlock )
{
	if ( InputComponent )
	{
		bStopAction = bShouldBlock;
		InputComponent->bBlockInput = bStopAction;
	}
}

bool UUserWidget::IsInputActionBlocking() const
{
	return bStopAction;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UUserWidget::OnInputAction( FOnInputAction Callback )
{
	if ( GetIsEnabled() )
	{
		Callback.ExecuteIfBound();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UUserWidget::InitializeInputComponent()
{
	if ( APlayerController* Controller = GetOwningPlayer() )
	{
		// Use the existing PC's input class, or fallback to the project default. We should use the existing class
		// instead of just the default one because if you have a plugin that has a PC with a different default input
		// class then this would fail
		UClass* InputClass = Controller->InputComponent ? Controller->InputComponent->GetClass() : UInputSettings::GetDefaultInputComponentClass();
		InputComponent = NewObject< UInputComponent >( this, InputClass, NAME_None, RF_Transient );

		InputComponent->bBlockInput = bStopAction;
		InputComponent->Priority = Priority;

		Controller->PushInputComponent( InputComponent );
	}
	else
	{
		FMessageLog("PIE").Info(FText::Format(LOCTEXT("NoInputListeningWithoutPlayerController", "Unable to listen to input actions without a player controller in {0}."), FText::FromName(GetClass()->GetFName())));
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UUserWidget::UpdateCanTick() 
{
	TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();
	UWorld* World = GetWorld();

	if(SafeGCWidget.IsValid() && World)
	{
		// Default to never tick, only recompute for auto
		bool bCanTick = false;
		if (TickFrequency == EWidgetTickFrequency::Auto)
		{
			// Note: WidgetBPClass can be NULL in a cooked build.
			UWidgetBlueprintGeneratedClass* WidgetBPClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass());
			bCanTick |= !WidgetBPClass || WidgetBPClass->ClassRequiresNativeTick();
			bCanTick |= bHasScriptImplementedTick;
			bCanTick |= World->GetLatentActionManager().GetNumActionsForObject(this) != 0;
			bCanTick |= ActiveAnimations.Num() > 0;
			bCanTick |= QueuedWidgetAnimationTransitions.Num() > 0;

			if (!bCanTick && bAreExtensionsConstructed)
			{
				for(UUserWidgetExtension* Extension : Extensions)
				{
					if (Extension->RequiresTick())
					{
						bCanTick = true;
						break;
					}
				}
			}
		}

		SafeGCWidget->SetCanTick(bCanTick);
	}
}

int32 UUserWidget::NativePaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	if ( bHasScriptImplementedPaint )
	{
		FPaintContext Context(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		OnPaint( Context );

		return FMath::Max(LayerId, Context.MaxLayer);
	}

	return LayerId;
}

void UUserWidget::SetMinimumDesiredSize(FVector2D InMinimumDesiredSize)
{
	if (MinimumDesiredSize != InMinimumDesiredSize)
	{
		MinimumDesiredSize = InMinimumDesiredSize;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

bool UUserWidget::NativeIsInteractable() const
{
	return IsInteractable();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UUserWidget::NativeSupportsKeyboardFocus() const
{
	return bIsFocusable;
}

bool UUserWidget::IsFocusable() const
{
	return bIsFocusable;
}

void UUserWidget::SetIsFocusable(bool InIsFocusable)
{
	bIsFocusable = InIsFocusable;
	Invalidate(EInvalidateWidgetReason::Paint);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

FReply UUserWidget::NativeOnFocusReceived( const FGeometry& InGeometry, const FFocusEvent& InFocusEvent )
{
	FReply Reply = OnFocusReceived( InGeometry, InFocusEvent ).NativeReply;

	// Forward focus if Desired Focus is set
	if (UWidget* WidgetToFocus = GetDesiredFocusWidget())
	{
		if (TSharedPtr<SWidget> SlateWidget = WidgetToFocus->GetCachedWidget())
		{
			FWidgetPath Path;
			// Ensure that the widget is actually hosted/ the Widget tree is not null
			if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(SlateWidget.ToSharedRef(), Path) && Path.IsValid() && Path.ContainsWidget(GetCachedWidget().Get()))
			{
				// Execute full slate focus tunneling using the path
				FSlateApplication::Get().SetUserFocus(InFocusEvent.GetUser(), Path, InFocusEvent.GetCause());
				return FReply::Handled();
			}
		}
	}

	return Reply;
}

void UUserWidget::NativeOnFocusLost( const FFocusEvent& InFocusEvent )
{
	OnFocusLost( InFocusEvent );
}

void UUserWidget::NativeOnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	TSharedPtr<SObjectWidget> SafeGCWidget = MyGCWidget.Pin();
	if ( SafeGCWidget.IsValid() )
	{
		const bool bDecendantNewlyFocused = NewWidgetPath.ContainsWidget(SafeGCWidget.Get());
		if ( bDecendantNewlyFocused )
		{
			const bool bDecendantPreviouslyFocused = PreviousFocusPath.ContainsWidget(SafeGCWidget.Get());
			if ( !bDecendantPreviouslyFocused )
			{
				NativeOnAddedToFocusPath( InFocusEvent );
			}
		}
		else
		{
			NativeOnRemovedFromFocusPath( InFocusEvent );
		}
	}
}

void UUserWidget::NativeOnAddedToFocusPath(const FFocusEvent& InFocusEvent)
{
	OnAddedToFocusPath(InFocusEvent);
}

void UUserWidget::NativeOnRemovedFromFocusPath(const FFocusEvent& InFocusEvent)
{
	OnRemovedFromFocusPath(InFocusEvent);
}

FNavigationReply UUserWidget::NativeOnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent, const FNavigationReply& InDefaultReply)
{
	// No Blueprint Support At This Time

	return InDefaultReply;
}

FReply UUserWidget::NativeOnKeyChar( const FGeometry& InGeometry, const FCharacterEvent& InCharEvent )
{
	return OnKeyChar( InGeometry, InCharEvent ).NativeReply;
}

FReply UUserWidget::NativeOnPreviewKeyDown( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent )
{
	return OnPreviewKeyDown( InGeometry, InKeyEvent ).NativeReply;
}

FReply UUserWidget::NativeOnKeyDown( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent )
{
	return OnKeyDown( InGeometry, InKeyEvent ).NativeReply;
}

FReply UUserWidget::NativeOnKeyUp( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent )
{
	return OnKeyUp( InGeometry, InKeyEvent ).NativeReply;
}

FReply UUserWidget::NativeOnAnalogValueChanged( const FGeometry& InGeometry, const FAnalogInputEvent& InAnalogEvent )
{
	return OnAnalogValueChanged( InGeometry, InAnalogEvent ).NativeReply;
}

FReply UUserWidget::NativeOnMouseButtonDown( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	return OnMouseButtonDown( InGeometry, InMouseEvent ).NativeReply;
}

FReply UUserWidget::NativeOnPreviewMouseButtonDown( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	return OnPreviewMouseButtonDown( InGeometry, InMouseEvent ).NativeReply;
}

FReply UUserWidget::NativeOnMouseButtonUp( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	return OnMouseButtonUp(InGeometry, InMouseEvent).NativeReply;
}

FReply UUserWidget::NativeOnMouseMove( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	return OnMouseMove( InGeometry, InMouseEvent ).NativeReply;
}

void UUserWidget::NativeOnMouseEnter( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	OnMouseEnter( InGeometry, InMouseEvent );
}

void UUserWidget::NativeOnMouseLeave( const FPointerEvent& InMouseEvent )
{
	OnMouseLeave( InMouseEvent );
}

FReply UUserWidget::NativeOnMouseWheel( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	return OnMouseWheel( InGeometry, InMouseEvent ).NativeReply;
}

FReply UUserWidget::NativeOnMouseButtonDoubleClick( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent )
{
	return OnMouseButtonDoubleClick( InGeometry, InMouseEvent ).NativeReply;
}

void UUserWidget::NativeOnDragDetected( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, UDragDropOperation*& OutOperation )
{
	OnDragDetected( InGeometry, InMouseEvent, OutOperation);
}

void UUserWidget::NativeOnDragEnter( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation )
{
	OnDragEnter( InGeometry, InDragDropEvent, InOperation );
}

void UUserWidget::NativeOnDragLeave( const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation )
{
	OnDragLeave( InDragDropEvent, InOperation );
}

bool UUserWidget::NativeOnDragOver( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation )
{
	return OnDragOver( InGeometry, InDragDropEvent, InOperation );
}

bool UUserWidget::NativeOnDrop( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation )
{
	return OnDrop( InGeometry, InDragDropEvent, InOperation );
}

void UUserWidget::NativeOnDragCancelled( const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation )
{
	OnDragCancelled( InDragDropEvent, InOperation );
}

FReply UUserWidget::NativeOnTouchGesture( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent )
{
	return OnTouchGesture( InGeometry, InGestureEvent ).NativeReply;
}

FReply UUserWidget::NativeOnTouchStarted( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent )
{
	return OnTouchStarted( InGeometry, InGestureEvent ).NativeReply;
}

FReply UUserWidget::NativeOnTouchMoved( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent )
{
	return OnTouchMoved( InGeometry, InGestureEvent ).NativeReply;
}

FReply UUserWidget::NativeOnTouchEnded( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent )
{
	return OnTouchEnded( InGeometry, InGestureEvent ).NativeReply;
}

FReply UUserWidget::NativeOnMotionDetected( const FGeometry& InGeometry, const FMotionEvent& InMotionEvent )
{
	return OnMotionDetected( InGeometry, InMotionEvent ).NativeReply;
}

FReply UUserWidget::NativeOnTouchForceChanged(const FGeometry& InGeometry, const FPointerEvent& InTouchEvent)
{
	return OnTouchForceChanged(InGeometry, InTouchEvent).NativeReply;
}

FReply UUserWidget::NativeOnTouchFirstMove(const FGeometry& InGeometry, const FPointerEvent& InTouchEvent)
{
	return OnTouchFirstMove(InGeometry, InTouchEvent).NativeReply;
}

FCursorReply UUserWidget::NativeOnCursorQuery( const FGeometry& InGeometry, const FPointerEvent& InCursorEvent )
{
	return (bOverride_Cursor)
		? FCursorReply::Cursor(GetCursor())
		: FCursorReply::Unhandled();
}

FNavigationReply UUserWidget::NativeOnNavigation(const FGeometry& InGeometry, const FNavigationEvent& InNavigationEvent)
{
	return FNavigationReply::Escape();
}
	
void UUserWidget::NativeOnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	OnMouseCaptureLost();
}

bool UUserWidget::IsAsset() const
{
	// This stops widget archetypes from showing up in the content browser
	return false;
}

void UUserWidget::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	if (WidgetTree)
	{
		WidgetTree->SetFlags(RF_Transient);
	}

	// Remove bindings that are no longer contained in the class.
	if ( UWidgetBlueprintGeneratedClass* BGClass = Cast<UWidgetBlueprintGeneratedClass>(GetClass()))
	{
		RemoveObsoleteBindings(BGClass->NamedSlots);
	}

	// Prevent null extensions from getting serialized
	for (auto It = Extensions.CreateIterator(); It; ++It)
	{
		if (*It == nullptr)
		{
			It.RemoveCurrent();
		}
	}

	Super::PreSave(ObjectSaveContext);
}

void UUserWidget::PostLoad()
{
	Super::PostLoad();

	// Remove null extensions that have been serialized in our widget
	for (auto It = Extensions.CreateIterator(); It; ++It)
	{
		if (*It == nullptr)
		{
			It.RemoveCurrent();
		}
	}

#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UUserWidget* DefaultWidget = Cast<UUserWidget>(GetClass()->GetDefaultObject());
		bHasScriptImplementedTick = DefaultWidget->bHasScriptImplementedTick;
		bHasScriptImplementedPaint = DefaultWidget->bHasScriptImplementedPaint;
		bAutomaticallyRegisterInputOnConstruction = DefaultWidget->bAutomaticallyRegisterInputOnConstruction;
	}
#endif
}

/////////////////////////////////////////////////////

UUserWidget* UUserWidget::CreateWidgetInstance(UWidget& OwningWidget, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName)
{
	UUserWidget* ParentUserWidget = Cast<UUserWidget>(&OwningWidget);
	if (!ParentUserWidget && OwningWidget.GetOuter())
	{
		// If we were given a UWidget, the nearest parent UserWidget is the outer of the UWidget's WidgetTree outer
		ParentUserWidget = Cast<UUserWidget>(OwningWidget.GetOuter()->GetOuter());
	}

	if (ensure(ParentUserWidget && ParentUserWidget->WidgetTree))
	{
		UUserWidget* NewWidget = CreateInstanceInternal(ParentUserWidget->WidgetTree, UserWidgetClass, WidgetName, ParentUserWidget->GetWorld(), ParentUserWidget->GetOwningLocalPlayer());
#if WITH_EDITOR
		if (NewWidget)
		{
			NewWidget->SetDesignerFlags(OwningWidget.GetDesignerFlags());
		}
#endif
		return NewWidget;
	}

	return nullptr;
}

UUserWidget* UUserWidget::CreateWidgetInstance(UWidgetTree& OwningWidgetTree, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName)
{
	// If the widget tree we're owned by is outered to a UUserWidget great, initialize it like any old widget.
	if (UUserWidget* OwningUserWidget = Cast<UUserWidget>(OwningWidgetTree.GetOuter()))
	{
		return CreateWidgetInstance(*OwningUserWidget, UserWidgetClass, WidgetName);
	}

	return CreateInstanceInternal(&OwningWidgetTree, UserWidgetClass, WidgetName, nullptr, nullptr);
}

UUserWidget* UUserWidget::CreateWidgetInstance(APlayerController& OwnerPC, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName)
{
	if (!OwnerPC.IsLocalPlayerController())
		{
		const FText FormatPattern = LOCTEXT("NotLocalPlayer", "Only Local Player Controllers can be assigned to widgets. {PlayerController} is not a Local Player Controller.");
		FFormatNamedArguments FormatPatternArgs;
		FormatPatternArgs.Add(TEXT("PlayerController"), FText::FromName(OwnerPC.GetFName()));
		FMessageLog("PIE").Error(FText::Format(FormatPattern, FormatPatternArgs));
		}
	else if (!OwnerPC.Player)
	{
		const FText FormatPattern = LOCTEXT("NoPlayer", "CreateWidget cannot be used on Player Controller with no attached player. {PlayerController} has no Player attached.");
		FFormatNamedArguments FormatPatternArgs;
		FormatPatternArgs.Add(TEXT("PlayerController"), FText::FromName(OwnerPC.GetFName()));
		FMessageLog("PIE").Error(FText::Format(FormatPattern, FormatPatternArgs));
	}
	else if (UWorld* World = OwnerPC.GetWorld())
	{
		UGameInstance* GameInstance = World->GetGameInstance();
		UObject* Outer = GameInstance ? StaticCast<UObject*>(GameInstance) : StaticCast<UObject*>(World);
		return CreateInstanceInternal(Outer, UserWidgetClass, WidgetName, World, CastChecked<ULocalPlayer>(OwnerPC.Player));
	}
	return nullptr;
}

UUserWidget* UUserWidget::CreateWidgetInstance(UGameInstance& GameInstance, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName)
{
	return CreateInstanceInternal(&GameInstance, UserWidgetClass, WidgetName, GameInstance.GetWorld(), GameInstance.GetFirstGamePlayer());
}

UUserWidget* UUserWidget::CreateWidgetInstance(UWorld& World, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName)
{
	if (UGameInstance* GameInstance = World.GetGameInstance())
	{
		return CreateWidgetInstance(*GameInstance, UserWidgetClass, WidgetName);
	}
	return CreateInstanceInternal(&World, UserWidgetClass, WidgetName, &World, World.GetFirstLocalPlayerFromController());
}

UUserWidget* UUserWidget::CreateInstanceInternal(UObject* Outer, TSubclassOf<UUserWidget> UserWidgetClass, FName InstanceName, UWorld* World, ULocalPlayer* LocalPlayer)
{
	LLM_SCOPE_BYTAG(UI_UMG);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Only do this on a non-shipping or test build.
	if (!CreateWidgetHelpers::ValidateUserWidgetClass(UserWidgetClass))
	{
		return nullptr;
	}
#else
	if (!UserWidgetClass)
	{
		UE_LOGF(LogUMG, Error, "CreateWidget called with a null class.");
		return nullptr;
	}
#endif

#if !UE_BUILD_SHIPPING
	// Check if the world is being torn down before we create a widget for it.
	if (World)
	{
		// Look for indications that widgets are being created for a dead and dying world.
		ensureMsgf(!World->bIsTearingDown, TEXT("Widget Class %s - Attempting to be created while tearing down the world '%s'"), *UserWidgetClass->GetName(), *World->GetName());
	}
#endif

	if (!Outer)
	{
		FMessageLog("PIE").Error(FText::Format(LOCTEXT("OuterNull", "Unable to create the widget {0}, no outer provided."), FText::FromName(UserWidgetClass->GetFName())));
		return nullptr;
	}
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Outer->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UserWidgetClass, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(InstanceName, UserWidgetClass->GetFName(), Outer->GetPackage()->GetFName());

	UUserWidget* NewWidget = NewObject<UUserWidget>(Outer, UserWidgetClass, InstanceName, RF_Transactional);
	
	if (LocalPlayer)
	{
		NewWidget->SetPlayerContext(FLocalPlayerContext(LocalPlayer, World));
	}

	NewWidget->Initialize();

	return NewWidget;
}

void UUserWidget::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UUserWidget* TypedThis = CastChecked<UUserWidget>(InThis);

	for (auto It = TypedThis->ActiveAnimations.CreateIterator(); It; ++It)
	{
		TSharedRef<FWidgetAnimationState> State(*It);
		State->AddReferencedObjects(Collector);
	}
}

void UUserWidget::ClearStoppedAnimationStates()
{
	ensureMsgf(!bIsTickingAnimations, TEXT("When active animations are ticked, animations can only be added, not removed"));

	for (auto It = ActiveAnimations.CreateIterator(); It; ++It)
	{
		TSharedRef<FWidgetAnimationState> State(*It);
		if (!State->IsValid())
		{
			It.RemoveCurrent();
		}
		else if ((State->GetPlaybackStatus() == EMovieScenePlayerStatus::Stopped) && !State->IsStopping())
		{
			State->TearDown();
			It.RemoveCurrent();
		}
	}
}

void UUserWidget::UpdatePlayerContextIfInvalid(const FLocalPlayerContext& ParentPlayerContext)
{
	if (PlayerContext.IsValid())
	{
		if (WidgetTree)
		{
			WidgetTree->ForEachWidget(
				[this] (UWidget* Widget)
				{
					if (UUserWidget* UserWidget = Cast<UUserWidget>(Widget))
					{
						UserWidget->UpdatePlayerContextIfInvalid(PlayerContext);
					}
				});
		}
	}
	else
	{
		SetPlayerContext(ParentPlayerContext);
	}
}

void UUserWidget::OnLatentActionsChanged(UObject* ObjectWhichChanged, ELatentActionChangeType ChangeType)
{
	if (UUserWidget* WidgetThatChanged = Cast<UUserWidget>(ObjectWhichChanged))
	{
		TSharedPtr<SObjectWidget> SafeGCWidget = WidgetThatChanged->MyGCWidget.Pin();
		if (SafeGCWidget.IsValid())
		{
			bool bCanTick = SafeGCWidget->GetCanTick();

			WidgetThatChanged->UpdateCanTick();

			if (SafeGCWidget->GetCanTick() && !bCanTick)
			{
				// If the widget can now tick, recache the volatility of the widget.
				WidgetThatChanged->Invalidate(EInvalidateWidgetReason::LayoutAndVolatility);
			}
		}
	}
}

UUserWidgetExtension* UUserWidget::GetExtension(TSubclassOf<UUserWidgetExtension> InExtensionType) const
{
	for (UUserWidgetExtension* Extension : Extensions)
	{
		if (ensure(Extension))
		{
			if (Extension->IsA(InExtensionType))
			{
				return Extension;
			}
		}
	}
	return nullptr;
}

TArray<UUserWidgetExtension*> UUserWidget::GetExtensions(TSubclassOf<UUserWidgetExtension> InExtensionType) const
{
	TArray<UUserWidgetExtension*> Result;
	for (UUserWidgetExtension* Extension : Extensions)
	{
		if (Extension->IsA(InExtensionType))
		{
			Result.Add(Extension);
		}
	}
	return Result;
}

UUserWidgetExtension* UUserWidget::AddExtension(TSubclassOf<UUserWidgetExtension> InExtensionType)
{
	UUserWidgetExtension* Extension = NewObject<UUserWidgetExtension>(this, InExtensionType);
	Extensions.Add(Extension);
	if (bInitialized)
	{
		Extension->Initialize();
	}

	if (bAreExtensionsPreConstructed)
	{
		const bool bIsDesignTime = IsDesignTime();
		Extension->PreConstruct(bIsDesignTime);
	}
	
	if (bAreExtensionsConstructed)
	{
		Extension->Construct();
		if (Extension->RequiresTick())
		{
			UpdateCanTick();
		}
	}
	return Extension;
}

UUserWidgetExtension* UUserWidget::GetOrAddExtension(TSubclassOf<UUserWidgetExtension> InExtensionType)
{
	if (UUserWidgetExtension* ExistingExtension = GetExtension(InExtensionType))
	{
		return ExistingExtension;
	}
	return AddExtension(InExtensionType);
}

void UUserWidget::RemoveExtension(UUserWidgetExtension* InExtension)
{
	if (InExtension)
	{
		if (Extensions.RemoveSingleSwap(InExtension))
		{
			if (bAreExtensionsConstructed)
			{
				bool bUpdateTick = InExtension->RequiresTick();
				InExtension->Destruct();
				if (bUpdateTick)
				{
					UpdateCanTick();
				}
			}
		}
	}
}

void UUserWidget::RemoveExtensions(TSubclassOf<UUserWidgetExtension> InExtensionType)
{

	TArray<UUserWidgetExtension*, TInlineAllocator<32>> LocalExtensions;
	for (int32 Index = Extensions.Num() - 1; Index >= 0; --Index)
	{
		UUserWidgetExtension* Extension = Extensions[Index];
		if (Extension->IsA(InExtensionType))
		{
			LocalExtensions.Add(Extension);
			Extensions.RemoveAtSwap(Index);

		}
	}

	if (bAreExtensionsConstructed)
	{
		bool bUpdateTick = false;
		for (UUserWidgetExtension* Extension : LocalExtensions)
		{
			bUpdateTick = bUpdateTick || Extension->RequiresTick();
			Extension->Destruct();
		}
		if (bUpdateTick)
		{
			UpdateCanTick();
		}
	}
}


/////////////////////////////////////////////////////

bool CreateWidgetHelpers::ValidateUserWidgetClass(const UClass* UserWidgetClass)
{
	if (UserWidgetClass == nullptr)
	{
		FMessageLog("PIE").Error(LOCTEXT("WidgetClassNull", "CreateWidget called with a null class."));
		return false;
	}

	if (!UserWidgetClass->IsChildOf(UUserWidget::StaticClass()))
	{
		const FText FormatPattern = LOCTEXT("NotUserWidget", "CreateWidget can only be used on UUserWidget children. {UserWidgetClass} is not a UUserWidget.");
		FFormatNamedArguments FormatPatternArgs;
		FormatPatternArgs.Add(TEXT("UserWidgetClass"), FText::FromName(UserWidgetClass->GetFName()));
		FMessageLog("PIE").Error(FText::Format(FormatPattern, FormatPatternArgs));
		return false;
	}

	if (UserWidgetClass->HasAnyClassFlags(CLASS_Abstract | CLASS_NewerVersionExists | CLASS_Deprecated))
	{
		const FText FormatPattern = LOCTEXT("NotValidClass", "Abstract, Deprecated or Replaced classes are not allowed to be used to construct a user widget. {UserWidgetClass} is one of these.");
		FFormatNamedArguments FormatPatternArgs;
		FormatPatternArgs.Add(TEXT("UserWidgetClass"), FText::FromName(UserWidgetClass->GetFName()));
		FMessageLog("PIE").Error(FText::Format(FormatPattern, FormatPatternArgs));
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

