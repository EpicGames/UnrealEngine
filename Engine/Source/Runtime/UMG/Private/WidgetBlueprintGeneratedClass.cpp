// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Blueprint/WidgetNavigation.h"
#include "Blueprint/WidgetTree.h"
#include "MovieScene.h"
#include "Animation/WidgetAnimation.h"
#include "Serialization/TextReferenceCollector.h"
#include "Engine/UserInterfaceSettings.h"
#include "Extensions/WidgetBlueprintGeneratedClassExtension.h"
#include "UMGPrivate.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/ObjectResource.h"
#include "UObject/LinkerLoad.h"
#include "Engine/StreamableManager.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBlueprintGeneratedClass)


#if WITH_EDITOR
#include "Engine/Blueprint.h"
#endif

#define LOCTEXT_NAMESPACE "UMG"

FAutoConsoleCommand GDumpTemplateSizesCommand(
	TEXT("Widget.DumpTemplateSizes"),
	TEXT("Dump the sizes of all widget class templates in memory"),
	FConsoleCommandDelegate::CreateStatic([]()
	{
		struct FClassAndSize
		{
			FString ClassName;
			int32 TemplateSize = 0;
		};

		TArray<FClassAndSize> TemplateSizes;

		for (TObjectIterator<UWidgetBlueprintGeneratedClass> WidgetClassIt; WidgetClassIt; ++WidgetClassIt)
		{
			UWidgetBlueprintGeneratedClass* WidgetClass = *WidgetClassIt;

			if (WidgetClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				continue;
			}

#if WITH_EDITOR
			if (Cast<UBlueprint>(WidgetClass->ClassGeneratedBy)->SkeletonGeneratedClass == WidgetClass)
			{
				continue;
			}
#endif

			FClassAndSize Entry;
			Entry.ClassName = WidgetClass->GetName();

			if (UUserWidget* TemplateWidget = WidgetClass->GetDefaultObject<UUserWidget>())
			{
				int32 TemplateSize = WidgetClass->GetStructureSize();
				if (const UWidgetTree* TemplateWidgetTree = WidgetClass->GetWidgetTreeArchetype())
				{
					TemplateWidgetTree->ForEachWidget([&TemplateSize](UWidget* Widget) {
						TemplateSize += Widget->GetClass()->GetStructureSize();
					});
				}

				Entry.TemplateSize = TemplateSize;
			}

			TemplateSizes.Add(Entry);
		}

		TemplateSizes.StableSort([](const FClassAndSize& A, const FClassAndSize& B) {
			return A.TemplateSize > B.TemplateSize;
		});

		uint32 TotalSizeBytes = 0;
		UE_LOGF(LogUMG, Display, "%-60ls %-15ls", TEXT("Template Class"), TEXT("Size (bytes)"));
		for (const FClassAndSize& Entry : TemplateSizes)
		{
			TotalSizeBytes += Entry.TemplateSize;
			if (Entry.TemplateSize > 0)
			{
				UE_LOGF(LogUMG, Display, "%-60ls %-15d", *Entry.ClassName, Entry.TemplateSize);
			}
			else
			{
				UE_LOGF(LogUMG, Display, "%-60ls %-15ls", *Entry.ClassName, TEXT("0 - (No Template)"));
			}
		}

		UE_LOGF(LogUMG, Display, "Total size of templates %.3f MB", TotalSizeBytes/(1024.f*1024.f));
	}), ECVF_Cheat);

#if WITH_EDITOR

int32 TemplatePreviewInEditor = 0;
static FAutoConsoleVariableRef CVarTemplatePreviewInEditor(TEXT("Widget.TemplatePreviewInEditor"), TemplatePreviewInEditor, TEXT("Should a dynamic template be generated at runtime for the editor for widgets?  Useful for debugging templates."), ECVF_Default);

PRAGMA_DISABLE_DEPRECATION_WARNINGS;
FWidgetBlueprintGeneratedClassDelegates::FGetAssetTags FWidgetBlueprintGeneratedClassDelegates::GetAssetTags;
PRAGMA_ENABLE_DEPRECATION_WARNINGS;
FWidgetBlueprintGeneratedClassDelegates::FGetAssetTagsWithContext FWidgetBlueprintGeneratedClassDelegates::GetAssetTagsWithContext;
FWidgetBlueprintGeneratedClassDelegates::FCollectSaveOverrides FWidgetBlueprintGeneratedClassDelegates::CollectSaveOverrides;

#endif

#if WITH_EDITORONLY_DATA
namespace
{
	void CollectWidgetBlueprintGeneratedClassTextReferences(UObject* Object, FArchive& Ar)
	{
		// In an editor build, both UWidgetBlueprint and UWidgetBlueprintGeneratedClass reference an identical WidgetTree.
		// So we ignore the UWidgetBlueprintGeneratedClass when looking for persistent text references since it will be overwritten by the UWidgetBlueprint version.
	}
}
#endif

/////////////////////////////////////////////////////
// UWidgetBlueprintGeneratedClass

UWidgetBlueprintGeneratedClass::UWidgetBlueprintGeneratedClass()
{
	bCanCallInitializedWithoutPlayerContext = false;
#if WITH_EDITORONLY_DATA
	{
		static const FAutoRegisterTextReferenceCollectorCallback AutomaticRegistrationOfTextReferenceCollector(UWidgetBlueprintGeneratedClass::StaticClass(), &CollectWidgetBlueprintGeneratedClassTextReferences);
	}
	bCanCallPreConstruct = true;
#endif
}

void UWidgetBlueprintGeneratedClass::InitializeBindingsStatic(UUserWidget* UserWidget, const TArrayView<const FDelegateRuntimeBinding> InBindings, const TMap<FName, FObjectPropertyBase*>& InPropertyMap)
{
	check(!UserWidget->IsTemplate());

	// Note: It's not safe to assume here that the UserWidget class type is a UWidgetBlueprintGeneratedClass!
	// - @see InitializeWidgetStatic()

	// For each property binding that we're given, find the corresponding field, and setup the delegate binding on the widget.
	for (const FDelegateRuntimeBinding& Binding : InBindings)
	{
		if (FObjectPropertyBase*const* PropPtr = InPropertyMap.Find(*Binding.ObjectName))
		{
			const FObjectPropertyBase* WidgetProperty = *PropPtr;
			check(WidgetProperty);

			if (UWidget* Widget = Cast<UWidget>(WidgetProperty->GetObjectPropertyValue_InContainer(UserWidget)))
			{
				FDelegateProperty* DelegateProperty = FindFProperty<FDelegateProperty>(Widget->GetClass(), FName(*(Binding.PropertyName.ToString() + TEXT("Delegate"))));
				if (!DelegateProperty)
				{
					DelegateProperty = FindFProperty<FDelegateProperty>(Widget->GetClass(), Binding.PropertyName);
				}

				if (DelegateProperty)
				{
					bool bSourcePathBound = false;

					if (Binding.SourcePath.IsValid())
					{
						bSourcePathBound = Widget->AddBinding(DelegateProperty, UserWidget, Binding.SourcePath);
					}

					// If no native binder is found then the only possibility is that the binding is for
					// a delegate that doesn't match the known native binders available and so we
					// fallback to just attempting to bind to the function directly.
					if (bSourcePathBound == false)
					{
						FScriptDelegate* ScriptDelegate = DelegateProperty->GetPropertyValuePtr_InContainer(Widget);
						if (ScriptDelegate)
						{
							ScriptDelegate->BindUFunction(UserWidget, Binding.FunctionName);
						}
					}
				}
			}
		}
	}
}

void UWidgetBlueprintGeneratedClass::InitializeWidgetStatic(UUserWidget* UserWidget
	, const UClass* InClass
	, UWidgetTree* InWidgetTree
	, const UClass* InWidgetTreeWidgetClass
	, const TArrayView<UWidgetAnimation*> InAnimations
	, const TArrayView<const FDelegateRuntimeBinding> InBindings)
{
	check(InClass);

	if ( UserWidget->IsTemplate() )
	{
		return;
	}

#if UE_HAS_WIDGET_GENERATED_BY_CLASS
	TWeakObjectPtr<UClass> WidgetGeneratedByClass = MakeWeakObjectPtr(const_cast<UClass*>(InClass));
	UserWidget->WidgetGeneratedByClass = WidgetGeneratedByClass;
#endif

	UWidgetTree* CreatedWidgetTree = UserWidget->WidgetTree;

	// Normally the ClonedTree should be null - we do in the case of design time with the widget, actually
	// clone the widget tree directly from the WidgetBlueprint so that the rebuilt preview matches the newest
	// widget tree, without a full blueprint compile being required.  In that case, the WidgetTree on the UserWidget
	// will have already been initialized to some value.  When that's the case, we'll avoid duplicating it from the class
	// similar to how we use to use the DesignerWidgetTree.
	if ( CreatedWidgetTree == nullptr )
	{
		TMap<FName, UWidget*> NamedSlotContentToMerge;
		if (const UWidgetBlueprintGeneratedClass* WidgetsActualClass = Cast<UWidgetBlueprintGeneratedClass>(UserWidget->GetClass()))
		{
			WidgetsActualClass->GetNamedSlotArchetypeContent([&NamedSlotContentToMerge](FName SlotName, UWidget* Content)
			{
				NamedSlotContentToMerge.Add(SlotName, Content);
			});
		}
		
		UserWidget->DuplicateAndInitializeFromWidgetTree(InWidgetTree, NamedSlotContentToMerge);
		CreatedWidgetTree = UserWidget->WidgetTree;
	}

#if WITH_EDITOR
	UserWidget->WidgetGeneratedBy = InClass->ClassGeneratedBy;
#endif

	if (CreatedWidgetTree)
	{
		UClass* WidgetBlueprintClass = UserWidget->GetClass();

		TMap<FName, FObjectPropertyBase*> ObjectPropertiesMap;
		for (TFieldIterator<FObjectPropertyBase>It(WidgetBlueprintClass, EFieldIterationFlags::Default); It; ++It)
		{
			check(*It);
			ensureMsgf(!ObjectPropertiesMap.Contains(It->GetFName()), TEXT("There are properties with the same names: '%s'"), *It->GetName());
			ObjectPropertiesMap.Add(It->GetFName(), *It);
		}

		BindAnimationsStatic(UserWidget, InAnimations, ObjectPropertiesMap);

		// Support navigation rules set on dynamically created user widgets by resolving the navigation rules against itself
		// if this widget is pre-placed in another widget, the navigation rules will be resolved afterward against the outer widget below in CreatedWidgetTree->ForEachWidget
		if (UserWidget->Navigation)
		{
			UserWidget->Navigation->ResolveRules(UserWidget, CreatedWidgetTree);
		}

		CreatedWidgetTree->ForEachWidget([&](UWidget* Widget) {
			// Not fatal if NULL, but shouldn't happen
			if (!ensure(Widget != nullptr))
			{
				return;
			}

#if UE_HAS_WIDGET_GENERATED_BY_CLASS
			Widget->WidgetGeneratedByClass = WidgetGeneratedByClass;
#endif

#if WITH_EDITOR
			Widget->WidgetGeneratedBy = InClass->ClassGeneratedBy;
#endif

			// Find property with the same name as the template and assign the new widget to it.
			if (FObjectPropertyBase** PropPtr = ObjectPropertiesMap.Find(Widget->GetFName()))
			{
				FObjectPropertyBase* Prop = *PropPtr;
				check(Prop);
				Prop->SetObjectPropertyValue_InContainer(UserWidget, Widget);
				UObject* Value = Prop->GetObjectPropertyValue_InContainer(UserWidget);
				check(Value == Widget);
			}

			// Initialize Navigation Data
			if (Widget->Navigation)
			{
				Widget->Navigation->ResolveRules(UserWidget, CreatedWidgetTree);
			}

#if WITH_EDITOR
			Widget->ConnectEditorData();
#endif
		});

		InitializeBindingsStatic(UserWidget, InBindings, ObjectPropertiesMap);

		// Bind any delegates on widgets
		if (!UserWidget->IsDesignTime())
		{
			UBlueprintGeneratedClass::BindDynamicDelegates(InClass, UserWidget);
		}
	}
}

void UWidgetBlueprintGeneratedClass::BindAnimationsStatic(UUserWidget* Instance, const TArrayView<UWidgetAnimation*> InAnimations, const TMap<FName, FObjectPropertyBase*>& InPropertyMap)
{
	// Note: It's not safe to assume here that the UserWidget class type is a UWidgetBlueprintGeneratedClass!
	// - @see InitializeWidgetStatic()

	for (UWidgetAnimation* Animation : InAnimations)
	{
		if (Animation->GetMovieScene())
		{
			// Find property with the same name as the animation and assign the animation to it.
			if (FObjectPropertyBase*const* PropPtr = InPropertyMap.Find(Animation->GetMovieScene()->GetFName()))
			{
				check(*PropPtr);
				(*PropPtr)->SetObjectPropertyValue_InContainer(Instance, Animation);
			}
		}
	}
}

#if WITH_EDITOR
void UWidgetBlueprintGeneratedClass::SetClassRequiresNativeTick(bool InClassRequiresNativeTick)
{
	bClassRequiresNativeTick = InClassRequiresNativeTick;
}
#endif

void UWidgetBlueprintGeneratedClass::InitializeWidget(UUserWidget* UserWidget) const
{
	TArray<UWidgetAnimation*, FConcurrentLinearArrayAllocator> AllAnims;
	TArray<FDelegateRuntimeBinding, FConcurrentLinearArrayAllocator> AllBindings;

	// Iterate all generated classes in the widget's parent class hierarchy and include animations and bindings
	// found on each one.
	UClass* SuperClass = UserWidget->GetClass();
	while (UWidgetBlueprintGeneratedClass* WBPGC = Cast<UWidgetBlueprintGeneratedClass>(SuperClass))
	{
		AllAnims.Append(WBPGC->Animations);
		AllBindings.Append(WBPGC->Bindings);

		SuperClass = SuperClass->GetSuperClass();
	}

	UWidgetTree* PrimaryWidgetTree = WidgetTree;
	UWidgetBlueprintGeneratedClass* PrimaryWidgetTreeClass = FindWidgetTreeOwningClass();
	if (PrimaryWidgetTreeClass)
	{
		PrimaryWidgetTree = PrimaryWidgetTreeClass->WidgetTree;
	}

	InitializeWidgetStatic(UserWidget, this, PrimaryWidgetTree, PrimaryWidgetTreeClass, AllAnims, AllBindings);
}

void UWidgetBlueprintGeneratedClass::PostLoad()
{
	Super::PostLoad();

	if (WidgetTree)
	{
		// We don't want any of these flags to carry over from the WidgetBlueprint
		WidgetTree->ClearFlags(RF_Public | RF_ArchetypeObject | RF_DefaultSubObject);

#if !WITH_EDITOR
		WidgetTree->AddToCluster(this, true);
#endif
	}

#if WITH_EDITOR
	if ( GetLinkerUEVersion() < VER_UE4_RENAME_WIDGET_VISIBILITY )
	{
		static const FName Visiblity("Visiblity");
		static const FName Visibility("Visibility");

		for ( FDelegateRuntimeBinding& Binding : Bindings )
		{
			if ( Binding.PropertyName == Visiblity )
			{
				Binding.PropertyName = Visibility;
			}
		}
	}
#endif
}

void UWidgetBlueprintGeneratedClass::PurgeClass(bool bRecompilingOnLoad)
{
	Super::PurgeClass(bRecompilingOnLoad);

	const ERenameFlags RenFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty;

	// Remove the old widdget tree.
	if ( WidgetTree )
	{
        // Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
		FLinkerLoad::InvalidateExport(WidgetTree);
		WidgetTree->Rename(nullptr, GetTransientPackage(), RenFlags);
		WidgetTree = nullptr;
	}

	// Remove all animations.
	for ( UWidgetAnimation* Animation : Animations )
	{
        // Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
		FLinkerLoad::InvalidateExport(Animation);
		Animation->Rename(nullptr, GetTransientPackage(), RenFlags);
	}

	Animations.Empty();
	Bindings.Empty();
}

bool UWidgetBlueprintGeneratedClass::NeedsLoadForServer() const
{
	const UUserInterfaceSettings* UISettings = GetDefault<UUserInterfaceSettings>();
	return UISettings->bLoadWidgetsOnDedicatedServer;
}

void UWidgetBlueprintGeneratedClass::SetWidgetTreeArchetype(UWidgetTree* InWidgetTree)
{
	WidgetTree = InWidgetTree;

	if (WidgetTree)
	{
		// We don't want any of these flags to carry over from the WidgetBlueprint
		WidgetTree->ClearFlags(RF_Public | RF_ArchetypeObject | RF_DefaultSubObject | RF_Transient);
	}
}

void UWidgetBlueprintGeneratedClass::GetNamedSlotArchetypeContent(TFunctionRef<void(FName /*SlotName*/, UWidget* /*Content*/)> Predicate) const
{
	for (const FName& SlotName : NamedSlots)
	{
		const UWidgetBlueprintGeneratedClass* BPGClass = this;
		while (BPGClass)
		{
			UWidgetTree* Tree = BPGClass->GetWidgetTreeArchetype();
			if (UWidget* ContentInSlot = Tree->GetContentForSlot(SlotName))
			{
				// Report the content in the slot, and break so we can test the next slot.
				Predicate(SlotName, ContentInSlot);
				break;
			}

			BPGClass = Cast<UWidgetBlueprintGeneratedClass>(BPGClass->GetSuperClass());
		}
	}
}

void UWidgetBlueprintGeneratedClass::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		// We've made it so the actual set of named slots we expose is AvailableNamedSlots, we need to copy
		// the initial set we load though to ensure we don't get compile time load errors.
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WidgetInheritedNamedSlots)
		{
			AvailableNamedSlots = NamedSlots;
			InstanceNamedSlots = NamedSlots;
		}
	}
}

UWidgetBlueprintGeneratedClass* UWidgetBlueprintGeneratedClass::FindWidgetTreeOwningClass() const
{
	UWidgetBlueprintGeneratedClass* RootBGClass = const_cast<UWidgetBlueprintGeneratedClass*>(this);
	UWidgetBlueprintGeneratedClass* BGClass = RootBGClass;

	while (BGClass)
	{
		//TODO NickD: This conditional post load shouldn't be needed any more once the Fast Widget creation path is the only path!
		// Force post load on the generated class so all subobjects are done (specifically the widget tree).
		BGClass->ConditionalPostLoad();

		const bool bNoRootWidget = (nullptr == BGClass->WidgetTree) || (nullptr == BGClass->WidgetTree->RootWidget);

		if (bNoRootWidget)
		{
			UWidgetBlueprintGeneratedClass* SuperBGClass = Cast<UWidgetBlueprintGeneratedClass>(BGClass->GetSuperClass());
			if (SuperBGClass)
			{
				BGClass = SuperBGClass;
				continue;
			}
			else
			{
				// If we reach a super class that isn't a UWidgetBlueprintGeneratedClass, return the root class.
				return RootBGClass;
			}
		}

		return BGClass;
	}

	return nullptr;
}

UWidgetBlueprintGeneratedClassExtension* UWidgetBlueprintGeneratedClass::GetExtension(TSubclassOf<UWidgetBlueprintGeneratedClassExtension> InExtensionType, bool bIncludeSuper)
{
	for (UWidgetBlueprintGeneratedClassExtension* Extension : Extensions)
	{
		if (Extension->IsA(InExtensionType))
		{
			return Extension;
		}
	}
	if (bIncludeSuper)
	{
		if (UWidgetBlueprintGeneratedClass* ParentClass = Cast<UWidgetBlueprintGeneratedClass>(GetSuperClass()))
		{
			return ParentClass->GetExtension(InExtensionType);
		}
	}
	return nullptr;
}

TArray<UWidgetBlueprintGeneratedClassExtension*> UWidgetBlueprintGeneratedClass::GetExtensions(TSubclassOf<UWidgetBlueprintGeneratedClassExtension> InExtensionType, bool bIncludeSuper)
{
	TArray<UWidgetBlueprintGeneratedClassExtension*> Result;
	GetExtensions(Result, InExtensionType, bIncludeSuper);
	return Result;
}

void UWidgetBlueprintGeneratedClass::GetExtensions(TArray<UWidgetBlueprintGeneratedClassExtension*>& OutExtensions, TSubclassOf<UWidgetBlueprintGeneratedClassExtension> InExtensionType, bool bIncludeSuper)
{
	for (UWidgetBlueprintGeneratedClassExtension* Extension : Extensions)
	{
		if (Extension->IsA(InExtensionType))
		{
			OutExtensions.Add(Extension);
		}
	}
	if (bIncludeSuper)
	{
		if (UWidgetBlueprintGeneratedClass* ParentClass = Cast<UWidgetBlueprintGeneratedClass>(GetSuperClass()))
		{
			ParentClass->GetExtensions(OutExtensions, InExtensionType, bIncludeSuper);
		}
	}
}

#if WITH_EDITOR

void UWidgetBlueprintGeneratedClass::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	TArray<UObject::FAssetRegistryTag> DeprecatedFunctionTags;
	FWidgetBlueprintGeneratedClassDelegates::GetAssetTags.Broadcast(this, DeprecatedFunctionTags);
	for (UObject::FAssetRegistryTag& Tag : DeprecatedFunctionTags)
	{
		Context.AddTag(MoveTemp(Tag));
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	FWidgetBlueprintGeneratedClassDelegates::GetAssetTagsWithContext.Broadcast(this, Context);
}

void UWidgetBlueprintGeneratedClass::CollectSaveOverrides(FObjectCollectSaveOverridesContext SaveContext)
{
	Super::CollectSaveOverrides(SaveContext);
	FWidgetBlueprintGeneratedClassDelegates::CollectSaveOverrides.Broadcast(this, SaveContext);
}
#endif

#undef LOCTEXT_NAMESPACE

