// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/WidgetBlueprintLibrary.h"
#include "Materials/MaterialInterface.h"
#include "Slate/SGameLayerManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "Engine/Texture2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Fonts/SlateFontInfo.h"
#include "Engine/Font.h"
#include "Engine/GameViewportClient.h"
#include "Brushes/SlateNoResource.h"
#include "Rendering/DrawElements.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/UMGCoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/UMGDragDropOp.h"
#include "Slate/SlateBrushAsset.h"
#include "EngineGlobals.h"
#include "Blueprint/WidgetLayoutLibrary.h"
#include "Engine/Engine.h"
#include "Engine/GameEngine.h"
#include "Widgets/Layout/SWindowTitleBarArea.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetBlueprintLibrary)

//For PIE error messages

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UWidgetBlueprintLibrary

UWidgetBlueprintLibrary::UWidgetBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

UUserWidget* UWidgetBlueprintLibrary::Create(UObject* WorldContextObject, TSubclassOf<UUserWidget> WidgetType, APlayerController* OwningPlayer)
{
	if (WidgetType == nullptr)
	{
		return nullptr;
	}

	if (OwningPlayer)
	{
		return CreateWidget(OwningPlayer, WidgetType);
	}
	else if (APlayerController* ImpliedOwningPlayer = Cast<APlayerController>(WorldContextObject))
	{
		return CreateWidget(ImpliedOwningPlayer, WidgetType);
	}
	else if (UUserWidget* OwningWidget = Cast<UUserWidget>(WorldContextObject))
	{
		return CreateWidget(OwningWidget, WidgetType);
	}
	else if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return CreateWidget(World, WidgetType);
	}

	return nullptr;
}

UDragDropOperation* UWidgetBlueprintLibrary::CreateDragDropOperation(TSubclassOf<UDragDropOperation> Operation)
{
	UDragDropOperation* DragDropOperation = nullptr;
	if ( Operation )
	{
		DragDropOperation = NewObject<UDragDropOperation>(GetTransientPackage(), Operation);
	}
	else
	{
		DragDropOperation = NewObject<UDragDropOperation>();
	}
	return DragDropOperation;
}

void UWidgetBlueprintLibrary::SetInputMode_UIOnlyEx(APlayerController* PlayerController, UWidget* InWidgetToFocus, EMouseLockMode InMouseLockMode, const bool bFlushInput /* = false */)
{
	if (PlayerController != nullptr)
	{
		FInputModeUIOnly InputMode;
		InputMode.SetLockMouseToViewportBehavior(InMouseLockMode);

		if (InWidgetToFocus != nullptr)
		{
			InputMode.SetWidgetToFocus(InWidgetToFocus->TakeWidget());
		}
		PlayerController->SetInputMode(InputMode);

		if (bFlushInput)
		{
			PlayerController->FlushPressedKeys();
		}
	}
	#if WITH_EDITOR 
	else
	{ 
		FMessageLog("PIE").Error(LOCTEXT("UMG WidgetBlueprint Library: SetInputMode_UIOnly", "SetInputMode_UIOnly expects a valid player controller as 'PlayerController' target"));
	}
	#endif // WITH_EDITOR
}

void UWidgetBlueprintLibrary::SetInputMode_GameAndUIEx(APlayerController* PlayerController, UWidget* InWidgetToFocus, EMouseLockMode InMouseLockMode, bool bHideCursorDuringCapture, const bool bFlushInput /* = false */)
{
	if (PlayerController != nullptr)
	{
		FInputModeGameAndUI InputMode;
		InputMode.SetLockMouseToViewportBehavior(InMouseLockMode);
		InputMode.SetHideCursorDuringCapture(bHideCursorDuringCapture);

		if (InWidgetToFocus != nullptr)
		{
			InputMode.SetWidgetToFocus(InWidgetToFocus->TakeWidget());
		}
		PlayerController->SetInputMode(InputMode);

		if (bFlushInput)
		{
			PlayerController->FlushPressedKeys();
		}
	}
	#if WITH_EDITOR 
	else
	{
		FMessageLog("PIE").Error(LOCTEXT("UMG WidgetBlueprint Library: SetInputMode_GameAndUI", "SetInputMode_GameAndUI expects a valid player controller as 'PlayerController' target"));	
	}
	#endif // WITH_EDITOR
	
}

void UWidgetBlueprintLibrary::SetInputMode_GameOnly(APlayerController* PlayerController, const bool bFlushInput /* = false */)
{
	if (PlayerController != nullptr)
	{
		FInputModeGameOnly InputMode;
		PlayerController->SetInputMode(InputMode);
		
		if (bFlushInput)
		{
			PlayerController->FlushPressedKeys();
		}
	}
	#if WITH_EDITOR 
	else
	{
		FMessageLog("PIE").Error(LOCTEXT("UMG WidgetBlueprint Library: SetInputMode_GameOnly", "SetInputMode_GameOnly expects a valid player controller as 'PlayerController' target"));
	}
	#endif // WITH_EDITOR
}

void UWidgetBlueprintLibrary::SetFocusToGameViewport()
{
	FSlateApplication::Get().SetAllUserFocusToGameViewport();
}

void UWidgetBlueprintLibrary::DrawBox(FPaintContext& Context, FVector2D Position, FVector2D Size, USlateBrushAsset* Brush, FLinearColor Tint)
{
	Context.MaxLayer++;

	if ( Brush )
	{
		FSlateDrawElement::MakeBox(
			Context.OutDrawElements,
			Context.MaxLayer,
			Context.AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(Position)),
			&Brush->Brush,
			ESlateDrawEffect::None,
			Tint);
	}
}

void UWidgetBlueprintLibrary::DrawSpline(FPaintContext& Context, FVector2D Start, FVector2D StartDir, FVector2D End, FVector2D EndDir, FLinearColor Tint, float Thickness)
{
	Context.MaxLayer++;

	FSlateDrawElement::MakeSpline(
		Context.OutDrawElements,
		Context.MaxLayer,
		Context.AllottedGeometry.ToPaintGeometry(),
		Start,
		StartDir,
		End,
		EndDir,
		Thickness,
		ESlateDrawEffect::None,
		Tint);
}

void UWidgetBlueprintLibrary::DrawLine(FPaintContext& Context, FVector2D PositionA, FVector2D PositionB, FLinearColor Tint, bool bAntiAlias, float Thickness)
{
	Context.MaxLayer++;

	TArray<FVector2f> Points;
	Points.Add(UE::Slate::CastToVector2f(PositionA));
	Points.Add(UE::Slate::CastToVector2f(PositionB));

	if ((PositionA - PositionB).SquaredLength() > KINDA_SMALL_NUMBER)
	{
		FSlateDrawElement::MakeLines(
			Context.OutDrawElements,
			Context.MaxLayer,
			Context.AllottedGeometry.ToPaintGeometry(),
			Points,
			ESlateDrawEffect::None,
			Tint,
			bAntiAlias,
			Thickness);
	}
}

void UWidgetBlueprintLibrary::DrawLines(FPaintContext& Context, const TArray<FVector2D>& Points, FLinearColor Tint, bool bAntiAlias, float Thickness)
{
	if (Points.Num() < 2)
	{
		return;
	}

	// We need to trim points that might be overlapping. We convert to Float at the same time
	TArray<FVector2f> ValidatedPoints;
	ValidatedPoints.Reserve(Points.Num());
	ValidatedPoints.Push(UE::Slate::CastToVector2f(Points[0]));
	FVector2D LastPoint = Points[0];
	for (int32 Index = 1; Index < Points.Num(); Index++)
	{
		FVector2D CurrentPoint = Points[Index];
		// If a the distance between two point is very small, do not add it.
		if ((CurrentPoint - LastPoint).SquaredLength() > KINDA_SMALL_NUMBER)
		{
			ValidatedPoints.Push(UE::Slate::CastToVector2f(Points[Index]));
			LastPoint = CurrentPoint;
		}
	}
	if (ValidatedPoints.Num() < 2)
	{
		return;
	}

	Context.MaxLayer++;

	FSlateDrawElement::MakeLines(
		Context.OutDrawElements,
		Context.MaxLayer,
		Context.AllottedGeometry.ToPaintGeometry(),
		MoveTemp(ValidatedPoints),
		ESlateDrawEffect::None,
		Tint,
		bAntiAlias,
		Thickness);
}

void UWidgetBlueprintLibrary::DrawText(FPaintContext& Context, const FString& InString, FVector2D Position, FLinearColor Tint)
{
	Context.MaxLayer++;

	//TODO UMG Create a font asset usable as a UFont or as a slate font asset.
	FSlateFontInfo FontInfo = FUMGCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText").Font;
	
	FSlateDrawElement::MakeText(
		Context.OutDrawElements,
		Context.MaxLayer,
		Context.AllottedGeometry.ToOffsetPaintGeometry(Position),
		InString,
		FontInfo,
		ESlateDrawEffect::None,
		Tint);
}

void UWidgetBlueprintLibrary::DrawTextFormatted(FPaintContext& Context, const FText& Text, FVector2D Position, UFont* Font, float FontSize, FName FontTypeFace, FLinearColor Tint)
{
	if ( Font )
	{
		Context.MaxLayer++;

		//TODO UMG Create a font asset usable as a UFont or as a slate font asset.
		FSlateFontInfo FontInfo(Font, FontSize, FontTypeFace);

		FSlateDrawElement::MakeText(
			Context.OutDrawElements,
			Context.MaxLayer,
			Context.AllottedGeometry.ToOffsetPaintGeometry(Position),
			Text,
			FontInfo,
			ESlateDrawEffect::None,
			Tint);
	}
}

FEventReply UWidgetBlueprintLibrary::Handled()
{
	FEventReply Reply;
	Reply.NativeReply = FReply::Handled();

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::Unhandled()
{
	FEventReply Reply;
	Reply.NativeReply = FReply::Unhandled();

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::CaptureMouse(FEventReply& Reply, UWidget* CapturingWidget)
{
	if ( CapturingWidget )
	{
		TSharedPtr<SWidget> CapturingSlateWidget = CapturingWidget->GetCachedWidget();
		if ( CapturingSlateWidget.IsValid() )
		{
			Reply.NativeReply = Reply.NativeReply.CaptureMouse(CapturingSlateWidget.ToSharedRef());
		}
	}

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::ReleaseMouseCapture(FEventReply& Reply)
{
	Reply.NativeReply = Reply.NativeReply.ReleaseMouseCapture();

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::LockMouse( UPARAM( ref ) FEventReply& Reply, UWidget* CapturingWidget )
{
	if ( CapturingWidget )
	{
		TSharedPtr< SWidget > SlateWidget = CapturingWidget->GetCachedWidget();
		if ( SlateWidget.IsValid() )
		{
			Reply.NativeReply = Reply.NativeReply.LockMouseToWidget( SlateWidget.ToSharedRef() );
		}
	}

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::UnlockMouse( UPARAM( ref ) FEventReply& Reply )
{
	Reply.NativeReply = Reply.NativeReply.ReleaseMouseLock();

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::SetUserFocus( UPARAM( ref ) FEventReply& Reply, UWidget* FocusWidget, bool bInAllUsers/* = false*/ )
{
	if (FocusWidget)
	{
		TSharedPtr<SWidget> CapturingSlateWidget = FocusWidget->GetCachedWidget();
		if (CapturingSlateWidget.IsValid())
		{
			Reply.NativeReply = Reply.NativeReply.SetUserFocus(CapturingSlateWidget.ToSharedRef(), EFocusCause::SetDirectly, bInAllUsers);
		}
	}

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::CaptureJoystick(FEventReply& Reply, UWidget* CapturingWidget, bool bInAllJoysticks/* = false*/)
{
	return SetUserFocus(Reply, CapturingWidget, bInAllJoysticks);
}

FEventReply UWidgetBlueprintLibrary::ClearUserFocus(FEventReply& Reply, bool bInAllUsers /*= false*/)
{
	Reply.NativeReply = Reply.NativeReply.ClearUserFocus(bInAllUsers);

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::ReleaseJoystickCapture(FEventReply& Reply, bool bInAllJoysticks /*= false*/)
{
	return ClearUserFocus(Reply, bInAllJoysticks);
}

FEventReply UWidgetBlueprintLibrary::SetMousePosition(FEventReply& Reply, FVector2D NewMousePosition)
{
	FIntPoint NewPoint((int32)NewMousePosition.X, (int32)NewMousePosition.Y);
	Reply.NativeReply = Reply.NativeReply.SetMousePos(NewPoint);

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::DetectDrag(FEventReply& Reply, UWidget* WidgetDetectingDrag, FKey DragKey)
{
	if ( WidgetDetectingDrag )
	{
		TSharedPtr<SWidget> SlateWidgetDetectingDrag = WidgetDetectingDrag->GetCachedWidget();
		if ( SlateWidgetDetectingDrag.IsValid() )
		{
			Reply.NativeReply = Reply.NativeReply.DetectDrag(SlateWidgetDetectingDrag.ToSharedRef(), DragKey);
		}
	}

	return Reply;
}

FEventReply UWidgetBlueprintLibrary::DetectDragIfPressed(const FPointerEvent& PointerEvent, UWidget* WidgetDetectingDrag, FKey DragKey)
{
	if ( PointerEvent.GetEffectingButton() == DragKey || PointerEvent.IsTouchEvent() )
	{
		FEventReply Reply = UWidgetBlueprintLibrary::Handled();
		return UWidgetBlueprintLibrary::DetectDrag(Reply, WidgetDetectingDrag, DragKey);
	}

	return UWidgetBlueprintLibrary::Unhandled();
}

FEventReply UWidgetBlueprintLibrary::EndDragDrop(FEventReply& Reply)
{
	Reply.NativeReply = Reply.NativeReply.EndDragDrop();

	return Reply;
}

bool UWidgetBlueprintLibrary::IsDragDropping()
{
	if ( FSlateApplication::Get().IsDragDropping() )
	{
		TSharedPtr<FDragDropOperation> SlateDragOp = FSlateApplication::Get().GetDragDroppingContent();
		if ( SlateDragOp.IsValid() && SlateDragOp->IsOfType<FUMGDragDropOp>() )
		{
			return true;
		}
	}

	return false;
}

UDragDropOperation* UWidgetBlueprintLibrary::GetDragDroppingContent()
{
	TSharedPtr<FDragDropOperation> SlateDragOp = FSlateApplication::Get().GetDragDroppingContent();
	if ( SlateDragOp.IsValid() && SlateDragOp->IsOfType<FUMGDragDropOp>() )
	{
		TSharedPtr<FUMGDragDropOp> UMGDragDropOp = StaticCastSharedPtr<FUMGDragDropOp>(SlateDragOp);
		return UMGDragDropOp->GetOperation();
	}

	return nullptr;
}

void UWidgetBlueprintLibrary::CancelDragDrop()
{
	FSlateApplication::Get().CancelDragDrop();
}

FSlateBrush UWidgetBlueprintLibrary::MakeBrushFromAsset(USlateBrushAsset* BrushAsset)
{
	if ( BrushAsset )
	{
		return BrushAsset->Brush;
	}

	return FSlateNoResource();
}

FSlateBrush UWidgetBlueprintLibrary::MakeBrushFromTexture(UTexture2D* Texture, int32 Width, int32 Height)
{
	if ( Texture )
	{
		FSlateBrush Brush;
		Brush.SetResourceObject(Texture);
		Width = (Width > 0) ? Width : Texture->GetSizeX();
		Height = (Height > 0) ? Height : Texture->GetSizeY();
		Brush.ImageSize = FVector2D(Width, Height);
		return Brush;
	}
	
	return FSlateNoResource();
}

FSlateBrush UWidgetBlueprintLibrary::MakeBrushFromMaterial(UMaterialInterface* Material, int32 Width, int32 Height)
{
	if ( Material )
	{
		FSlateBrush Brush;
		Brush.SetResourceObject(Material);
		Brush.ImageSize = FVector2D(Width, Height);

		return Brush;
	}

	return FSlateNoResource();
}

UObject* UWidgetBlueprintLibrary::GetBrushResource(const FSlateBrush& Brush)
{
	return Brush.GetResourceObject();
}

UTexture2D* UWidgetBlueprintLibrary::GetBrushResourceAsTexture2D(const FSlateBrush& Brush)
{
	return Cast<UTexture2D>(Brush.GetResourceObject());
}

UMaterialInterface* UWidgetBlueprintLibrary::GetBrushResourceAsMaterial(const FSlateBrush& Brush)
{
	return Cast<UMaterialInterface>(Brush.GetResourceObject());
}

void UWidgetBlueprintLibrary::SetBrushResourceToTexture(FSlateBrush& Brush, UTexture2D* Texture)
{
	Brush.SetResourceObject(Texture);
}

void UWidgetBlueprintLibrary::SetBrushResourceToMaterial(FSlateBrush& Brush, UMaterialInterface* Material)
{
	Brush.SetResourceObject(Material);
}

FSlateBrush UWidgetBlueprintLibrary::NoResourceBrush()
{
	return FSlateNoResource();
}

UMaterialInstanceDynamic* UWidgetBlueprintLibrary::GetDynamicMaterial(FSlateBrush& Brush)
{
	UObject* Resource = Brush.GetResourceObject();

	// If we already have a dynamic material, return it.
	if ( UMaterialInstanceDynamic* DynamicMaterial = Cast<UMaterialInstanceDynamic>(Resource) )
	{
		return DynamicMaterial;
	}
	// If the resource has a material interface we'll just update the brush to have a dynamic material.
	else if ( UMaterialInterface* Material = Cast<UMaterialInterface>(Resource) )
	{
		DynamicMaterial = UMaterialInstanceDynamic::Create(Material, nullptr);
		Brush.SetResourceObject(DynamicMaterial);

		return DynamicMaterial;
	}

	//TODO UMG can we do something for textures?  General purpose dynamic material for them?

	return nullptr;
}

void UWidgetBlueprintLibrary::DismissAllMenus()
{
	FSlateApplication::Get().DismissAllMenus();
}

void UWidgetBlueprintLibrary::GetAllWidgetsOfClass(const UObject* WorldContextObject, TArray<UUserWidget*>& FoundWidgets, TSubclassOf<UUserWidget> WidgetClass, bool TopLevelOnly)
{
	//Prevent possibility of an ever-growing array if user uses this in a loop
	FoundWidgets.Empty();
	
	if ( !WidgetClass || !WorldContextObject )
	{
		return;
	}
	 
	const UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if ( !World )
	{
		return;
	}

	for ( TObjectIterator<UUserWidget> Itr; Itr; ++Itr )
	{
		UUserWidget* LiveWidget = *Itr;

		// Skip any widget that's not in the current world context or that is not a child of the class specified.
		if (LiveWidget->GetWorld() != World || !LiveWidget->GetClass()->IsChildOf(WidgetClass))
		{
			continue;
		}
		
		if ( TopLevelOnly )
		{
			if ( LiveWidget->IsInViewport() )
			{
				FoundWidgets.Add(LiveWidget);
			}
		}
		else
		{
			FoundWidgets.Add(LiveWidget);
		}
	}
}

void UWidgetBlueprintLibrary::GetAllWidgetsWithInterface(const UObject* WorldContextObject, TArray<UUserWidget*>& FoundWidgets, TSubclassOf<UInterface> Interface, bool TopLevelOnly)
{
	//Prevent possibility of an ever-growing array if user uses this in a loop
	FoundWidgets.Empty();

	if (!Interface || !WorldContextObject)
	{
		return;
	}
	 
	const UWorld* const World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return;
	}

	for (TObjectIterator<UUserWidget> Itr; Itr; ++Itr)
	{
		UUserWidget* LiveWidget = *Itr;

		// Skip any widget that's not in the current world context or that is not a child of the class specified.
		if (LiveWidget->GetWorld() != World || !LiveWidget->GetClass()->ImplementsInterface(Interface))
		{
			continue;
		}

		if (TopLevelOnly)
		{
			if (LiveWidget->IsInViewport())
			{
				FoundWidgets.Add(LiveWidget);
			}
		}
		else
		{
			FoundWidgets.Add(LiveWidget);
		}
	}
}

FInputEvent UWidgetBlueprintLibrary::GetInputEventFromKeyEvent(const FKeyEvent& Event)
{
	return Event;
}

FKeyEvent UWidgetBlueprintLibrary::GetKeyEventFromAnalogInputEvent(const FAnalogInputEvent& Event)
{
	return Event;
}

FInputEvent UWidgetBlueprintLibrary::GetInputEventFromCharacterEvent(const FCharacterEvent& Event)
{
	return Event;
}

FInputEvent UWidgetBlueprintLibrary::GetInputEventFromPointerEvent(const FPointerEvent& Event)
{
	return Event;
}

FInputEvent UWidgetBlueprintLibrary::GetInputEventFromNavigationEvent(const FNavigationEvent& Event)
{
	return Event;
}

void UWidgetBlueprintLibrary::GetSafeZonePadding(const UObject* WorldContextObject, FVector4& SafePadding, FVector2D& SafePaddingScale, FVector4& SpillOverPadding)
{
	FVector2D ViewportSize = UWidgetLayoutLibrary::GetViewportSize(WorldContextObject);

	FMargin PaddingSize;
	FSlateApplication::Get().GetSafeZoneSize(PaddingSize, ViewportSize);
	SafePadding.X = PaddingSize.Left;
	SafePadding.Y = PaddingSize.Top;
	SafePadding.Z = PaddingSize.Right;
	SafePadding.W = PaddingSize.Bottom;
	FVector2D padding(FMath::Max(SafePadding.Z, SafePadding.X), FMath::Max(SafePadding.W, SafePadding.Y));
	SafePaddingScale = padding / ViewportSize;
	SpillOverPadding = SafePadding;
}

void UWidgetBlueprintLibrary::SetColorVisionDeficiencyType(EColorVisionDeficiency Type, float Severity, bool CorrectDeficiency, bool ShowCorrectionWithDeficiency)
{
	int32 AdjustedSeverity = FMath::TruncToInt32(FMath::Clamp(Severity, 0.f, 1.f) * 10);
	FSlateApplicationBase::Get().GetRenderer()->SetColorVisionDeficiencyType(Type, AdjustedSeverity, CorrectDeficiency, ShowCorrectionWithDeficiency);
}


bool UWidgetBlueprintLibrary::SetHardwareCursor(const UObject* WorldContextObject, EMouseCursor::Type CursorShape, FName CursorName, FVector2D HotSpot)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if ( World && World->IsGameWorld() )
	{
		if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
		{
			return ViewportClient->SetHardwareCursor(CursorShape, CursorName, HotSpot);
		}
	}

	return false;
}

void UWidgetBlueprintLibrary::SetWindowTitleBarState(UWidget* TitleBarContent, EWindowTitleBarMode Mode, bool bTitleBarDragEnabled, bool bWindowButtonsVisible, bool bTitleBarVisible)
{
	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
	if (GameEngine != nullptr && GameEngine->GameViewport)
	{
		TSharedPtr<IGameLayerManager> LayerManager = GameEngine->GameViewport->GetGameLayerManager();
		if (LayerManager.IsValid())
		{
			LayerManager->SetWindowTitleBarState(TitleBarContent ? TitleBarContent->GetCachedWidget() : nullptr, Mode, bTitleBarDragEnabled, bWindowButtonsVisible, bTitleBarVisible);
		}
	}
}

void UWidgetBlueprintLibrary::RestorePreviousWindowTitleBarState()
{
	UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
	if (GameEngine != nullptr && GameEngine->GameViewport)
	{
		TSharedPtr<IGameLayerManager> LayerManager = GameEngine->GameViewport->GetGameLayerManager();
		if (LayerManager.IsValid())
		{
			LayerManager->RestorePreviousWindowTitleBarState();
		}
	}
}

static UWidgetBlueprintLibrary::FOnGameWindowCloseButtonClickedDelegate OnGameWindowCloseButtonClicked;

static void OnGameWindowCloseButtonClickedSimpleDelegate()
{
	if (OnGameWindowCloseButtonClicked.IsBound())
	{
		OnGameWindowCloseButtonClicked.Execute();
	}
	else
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		TSharedPtr<SWindow> GameViewportWindow = GameEngine->GameViewportWindow.Pin();
		if (GameViewportWindow.IsValid())
		{
			GameViewportWindow->RequestDestroyWindow();
		}
	}
}

void UWidgetBlueprintLibrary::SetWindowTitleBarOnCloseClickedDelegate(UWidgetBlueprintLibrary::FOnGameWindowCloseButtonClickedDelegate Delegate)
{
	OnGameWindowCloseButtonClicked = Delegate;
	SWindowTitleBarArea::SetOnCloseButtonClickedDelegate(FSimpleDelegate::CreateStatic(&OnGameWindowCloseButtonClickedSimpleDelegate));
}

void UWidgetBlueprintLibrary::SetWindowTitleBarCloseButtonActive(bool bActive)
{
	SWindowTitleBarArea::SetIsCloseButtonActive(bActive);
}

#undef LOCTEXT_NAMESPACE

