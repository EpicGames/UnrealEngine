// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/WidgetChild.h"

#include "CoreMinimal.h"
#include "Blueprint/WidgetTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetChild)


FWidgetChild::FWidgetChild()
{}

FWidgetChild::FWidgetChild(const class UUserWidget* Outer, FName InWidgetName)
	:WidgetName(InWidgetName)
{
	if (!WidgetName.IsNone() && Outer && Outer->WidgetTree)
	{
		WidgetPtr = Outer->WidgetTree->FindWidget(WidgetName);
	}
}

UWidget* FWidgetChild::Resolve(const UWidgetTree* WidgetTree)
{
	if (!WidgetName.IsNone() && WidgetTree)
	{
		WidgetPtr = WidgetTree->FindWidget(WidgetName);		
	}
	else
	{
		WidgetPtr = nullptr;
	}
	
	return WidgetPtr.Get();
}
