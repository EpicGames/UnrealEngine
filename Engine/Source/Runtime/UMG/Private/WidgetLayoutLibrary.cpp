// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/WidgetLayoutLibrary.h"
#include "UMGPrivate.h"
#include "EngineGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "GameFramework/PlayerController.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Components/Widget.h"
#include "Components/CanvasPanelSlot.h"
#include "Components/HorizontalBoxSlot.h"
#include "Components/VerticalBoxSlot.h"
#include "Components/ScrollBoxSlot.h"
#include "Components/UniformGridSlot.h"
#include "Components/GridSlot.h"
#include "Components/OverlaySlot.h"
#include "Components/BorderSlot.h"
#include "Components/SafeZoneSlot.h"
#include "Components/ScaleBoxSlot.h"
#include "Components/ScrollBoxSlot.h"
#include "Components/SizeBoxSlot.h"
#include "Components/StackBoxSlot.h"
#include "Components/WrapBoxSlot.h"
#include "Components/WidgetSwitcherSlot.h"
#include "Engine/UserInterfaceSettings.h"
#include "Slate/SlateBrushAsset.h"
#include "Blueprint/WidgetLayoutLibrary.h"
#include "Engine/UserInterfaceSettings.h"
#include "Engine/RendererSettings.h"
#include "Blueprint/SlateBlueprintLibrary.h"
#include "Slate/SceneViewport.h"
#include "Slate/SGameLayerManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/FrameValue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetLayoutLibrary)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// UWidgetLayoutLibrary

UWidgetLayoutLibrary::UWidgetLayoutLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

bool UWidgetLayoutLibrary::ProjectWorldLocationToWidgetPosition(const APlayerController* PlayerController, FVector WorldLocation, FVector2D& ViewportPosition, bool bPlayerViewportRelative)
{
	FVector ScreenPosition3D;
	const bool bSuccess = ProjectWorldLocationToWidgetPositionWithDistance(PlayerController, WorldLocation, ScreenPosition3D, bPlayerViewportRelative);
	ViewportPosition = FVector2D(ScreenPosition3D.X, ScreenPosition3D.Y);
	return bSuccess;
}

bool UWidgetLayoutLibrary::ProjectWorldLocationToWidgetPositionWithDistance(const APlayerController* PlayerController, FVector WorldLocation, FVector& ViewportPosition, bool bPlayerViewportRelative)
{	
	if ( PlayerController )
	{
		FVector PixelLocation;
		const bool bProjected = PlayerController->ProjectWorldLocationToScreenWithDistance(WorldLocation, PixelLocation, bPlayerViewportRelative);

		if ( bProjected )
		{
			// Round the pixel projected value to reduce jittering due to layout rounding,
			// I do this before I remove scaling, because scaling is going to be applied later
			// in the opposite direction, so as long as we round, before inverse scale, scale should
			// result in more or less the same value, especially after slate does layout rounding.
			FVector2D ScreenPosition(FMath::RoundToDouble(PixelLocation.X), FMath::RoundToDouble(PixelLocation.Y));

			FVector2D ViewportPosition2D;
			USlateBlueprintLibrary::ScreenToViewport(PlayerController, ScreenPosition, ViewportPosition2D);
			ViewportPosition.X = ViewportPosition2D.X;
			ViewportPosition.Y = ViewportPosition2D.Y;
			ViewportPosition.Z = PixelLocation.Z;

			return true;
		}
	}

	ViewportPosition = FVector::ZeroVector;

	return false;
}

float UWidgetLayoutLibrary::GetViewportScale(const UObject* WorldContextObject)
{
	static TFrameValue<float> ViewportScaleCache;

	if ( !ViewportScaleCache.IsSet() || WITH_EDITOR )
	{
		float ViewportScale = 1.0f;

		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if ( World && World->IsGameWorld() )
		{
			if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
			{
				FVector2D ViewportSize;
				ViewportClient->GetViewportSize(ViewportSize);
				ViewportScale = GetDefault<UUserInterfaceSettings>()->GetDPIScaleBasedOnSize(FIntPoint(FMath::TruncToInt32(ViewportSize.X), FMath::TruncToInt32(ViewportSize.Y)));
			}
		}

		ViewportScaleCache = ViewportScale;
	}

	return ViewportScaleCache.GetValue();
}

float UWidgetLayoutLibrary::GetViewportScale(const UGameViewportClient* ViewportClient)
{
	FVector2D ViewportSize;
	ViewportClient->GetViewportSize(ViewportSize);
	
	float UserResolutionScale = GetDefault<UUserInterfaceSettings>()->GetDPIScaleBasedOnSize(FIntPoint(FMath::TruncToInt32(ViewportSize.X), FMath::TruncToInt32(ViewportSize.Y)));

	// Normally we'd factor in native DPI Scale here too, but because the SGameLayerManager already
	// accounts for the native DPI scale, and extracts it from the calculations, the UMG/Slate portion of the
	// game can more or less assume the platform scale is 1.0 for DPI.
	//
	// This id done because UMG already provides a solution for scaling UI across devices, that if influenced
	// by the platform scale would produce undesirable results.

	return UserResolutionScale;
}

FVector2D UWidgetLayoutLibrary::GetMousePositionOnPlatform()
{
	if ( FSlateApplication::IsInitialized() )
	{
		return FSlateApplication::Get().GetCursorPos();
	}

	return FVector2D(0, 0);
}

FVector2D UWidgetLayoutLibrary::GetMousePositionOnViewport(const UObject* WorldContextObject)
{
	if ( FSlateApplication::IsInitialized() )
	{
		FVector2D MousePosition = FSlateApplication::Get().GetCursorPos();
		FGeometry ViewportGeometry = GetViewportWidgetGeometry(WorldContextObject);
		return ViewportGeometry.AbsoluteToLocal(MousePosition);
	}

	return FVector2D(0, 0);
}

bool UWidgetLayoutLibrary::GetMousePositionScaledByDPI(const APlayerController* Player, float& LocationX, float& LocationY)
{
	if ( Player && Player->GetMousePosition(LocationX, LocationY) )
	{
		float Scale = UWidgetLayoutLibrary::GetViewportScale(Player);
		LocationX = LocationX / Scale;
		LocationY = LocationY / Scale;

		return true;
	}

	return false;
}
bool UWidgetLayoutLibrary::GetMousePositionScaledByDPI(const APlayerController* Player, double& LocationX, double& LocationY)
{
	float X = (float)LocationX, Y = (float)LocationY;
	if(GetMousePositionScaledByDPI(Player, X, Y))
	{
		LocationX = X; LocationY = Y;
		return true;
	}
	return false;
}

FVector2D UWidgetLayoutLibrary::GetViewportSize(const UObject* WorldContextObject)
{
	static TFrameValue<FVector2D> ViewportSizeCache;

	if ( !ViewportSizeCache.IsSet() || WITH_EDITOR )
	{
		FVector2D ViewportSize(1, 1);

		UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
		if ( World && World->IsGameWorld() )
		{
			if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
			{
				ViewportClient->GetViewportSize(ViewportSize);
			}
		}

		ViewportSizeCache = ViewportSize;
	}

	return ViewportSizeCache.GetValue();
}

FGeometry UWidgetLayoutLibrary::GetViewportWidgetGeometry(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if ( World && World->IsGameWorld() )
	{
		if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
		{
			TSharedPtr<IGameLayerManager> LayerManager = ViewportClient->GetGameLayerManager();
			if ( LayerManager.IsValid() )
			{
				return LayerManager->GetViewportWidgetHostGeometry();
			}
		}
	}

	return FGeometry();
}

FGeometry UWidgetLayoutLibrary::GetPlayerScreenWidgetGeometry(APlayerController* PlayerController)
{
	UWorld* World = GEngine->GetWorldFromContextObject(PlayerController, EGetWorldErrorMode::LogAndReturnNull);
	if ( World && World->IsGameWorld() )
	{
		if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
		{
			TSharedPtr<IGameLayerManager> LayerManager = ViewportClient->GetGameLayerManager();
			if ( LayerManager.IsValid() )
			{
				return LayerManager->GetPlayerWidgetHostGeometry(PlayerController->GetLocalPlayer());
			}
		}
	}

	return FGeometry();
}

UBorderSlot* UWidgetLayoutLibrary::SlotAsBorderSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UBorderSlot>(Widget->Slot);
	}

	return nullptr;
}

UCanvasPanelSlot* UWidgetLayoutLibrary::SlotAsCanvasSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UCanvasPanelSlot>(Widget->Slot);
	}

	return nullptr;
}

UGridSlot* UWidgetLayoutLibrary::SlotAsGridSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UGridSlot>(Widget->Slot);
	}

	return nullptr;
}

UHorizontalBoxSlot* UWidgetLayoutLibrary::SlotAsHorizontalBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UHorizontalBoxSlot>(Widget->Slot);
	}

	return nullptr;
}

UOverlaySlot* UWidgetLayoutLibrary::SlotAsOverlaySlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UOverlaySlot>(Widget->Slot);
	}

	return nullptr;
}

UUniformGridSlot* UWidgetLayoutLibrary::SlotAsUniformGridSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UUniformGridSlot>(Widget->Slot);
	}

	return nullptr;
}

UVerticalBoxSlot* UWidgetLayoutLibrary::SlotAsVerticalBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UVerticalBoxSlot>(Widget->Slot);
	}

	return nullptr;
}

UScrollBoxSlot* UWidgetLayoutLibrary::SlotAsScrollBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UScrollBoxSlot>(Widget->Slot);
	}

	return nullptr;
}

USafeZoneSlot* UWidgetLayoutLibrary::SlotAsSafeBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<USafeZoneSlot>(Widget->Slot);
	}

	return nullptr;
}

UScaleBoxSlot* UWidgetLayoutLibrary::SlotAsScaleBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UScaleBoxSlot>(Widget->Slot);
	}

	return nullptr;
}

USizeBoxSlot* UWidgetLayoutLibrary::SlotAsSizeBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<USizeBoxSlot>(Widget->Slot);
	}

	return nullptr;
}

UStackBoxSlot* UWidgetLayoutLibrary::SlotAsStackBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UStackBoxSlot>(Widget->Slot);
	}

	return nullptr;
}

UWrapBoxSlot* UWidgetLayoutLibrary::SlotAsWrapBoxSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UWrapBoxSlot>(Widget->Slot);
	}

	return nullptr;
}

UWidgetSwitcherSlot* UWidgetLayoutLibrary::SlotAsWidgetSwitcherSlot(UWidget* Widget)
{
	if (Widget)
	{
		return Cast<UWidgetSwitcherSlot>(Widget->Slot);
	}

	return nullptr;
}

void UWidgetLayoutLibrary::RemoveAllWidgets(const UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if ( World && World->IsGameWorld() )
	{
		if ( UGameViewportClient* ViewportClient = World->GetGameViewport() )
		{
			ViewportClient->RemoveAllViewportWidgets();
		}
	}
}

FWidgetDesignTimeInfo UWidgetLayoutLibrary::GetDesignTimeInfo(const UWidget* Widget)
{
	FWidgetDesignTimeInfo Info;

	// Validate that the caller is in a design-time preview context.
	if (!Widget || !Widget->IsDesignTime())
	{
		UE_LOGF(LogUMG, Error, "GetDesignTimeInfo() called outside of a design-time preview. This function should only be called from PreConstruct in the UMG Designer.");
		return Info;
	}

#if WITH_EDITOR
	int32 PreviewWidth = 0;
	int32 PreviewHeight = 0;

	const FString ConfigSection = TEXT("UMGEditor.Designer");

	GConfig->GetInt(*ConfigSection, TEXT("PreviewWidth"), PreviewWidth, GEditorPerProjectIni);
	GConfig->GetInt(*ConfigSection, TEXT("PreviewHeight"), PreviewHeight, GEditorPerProjectIni);

	Info.PreviewSize = FVector2D(static_cast<double>(PreviewWidth), static_cast<double>(PreviewHeight));
	Info.bIsPortrait = PreviewHeight > PreviewWidth;
#endif

	return Info;
}

#undef LOCTEXT_NAMESPACE

