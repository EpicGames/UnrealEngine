// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/WidgetTree.h"

#include "Components/Visual.h"
#include "Components/Widget.h"
#include "Blueprint/UserWidget.h"
#include "UMGPrivate.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetTree)

/////////////////////////////////////////////////////
// UWidgetTree

UWidgetTree::UWidgetTree(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
bool UWidgetTree::IsCapturingAsRootObjectForTransaction() const 
{ 
	// Track changes for child UWidgets with this WidgetTree as the outer
	return true; 
}
#endif //WITH_EDITOR

UWorld* UWidgetTree::GetWorld() const
{
	// The outer of a widget tree should be a user widget
	if ( UUserWidget* OwningWidget = Cast<UUserWidget>(GetOuter()) )
	{
		return OwningWidget->GetWorld();
	}

	return nullptr;
}

UWidget* UWidgetTree::FindWidget(const FName& Name) const
{
	UWidget* FoundWidget = nullptr;

	ForEachWidgetUntil([&FoundWidget, &Name] (UWidget* Widget) {
		if ( Widget->GetFName() == Name )
		{
			FoundWidget = Widget;
			return false;
		}
		return true;
	});

	return FoundWidget;
}

UWidget* UWidgetTree::FindWidget(TSharedRef<SWidget> InWidget) const
{
	UWidget* FoundWidget = nullptr;

	ForEachWidgetUntil([&FoundWidget, &InWidget] (UWidget* Widget) {
		if ( Widget->GetCachedWrappedWidget() == InWidget )
		{
			FoundWidget = Widget;
			return false;
		}
		return true;
	});

	return FoundWidget;
}

UPanelWidget* UWidgetTree::FindWidgetParent(UWidget* Widget, int32& OutChildIndex)
{
	UPanelWidget* Parent = Widget->GetParent();
	if ( Parent != nullptr )
	{
		OutChildIndex = Parent->GetChildIndex(Widget);
	}
	else
	{
		OutChildIndex = 0;
	}

	return Parent;
}

UWidget* UWidgetTree::FindWidgetChild(UPanelWidget* ParentWidget, FName ChildWidgetName, int32& OutChildIndex)
{
	OutChildIndex = INDEX_NONE;
	UWidget* FoundChild = nullptr;

	if (ParentWidget)
	{
		const auto CheckWidgetFunc = [&FoundChild, ChildWidgetName](UWidget* Widget) 
			{
				if (!FoundChild && Widget->GetFName() == ChildWidgetName)
				{
					FoundChild = Widget;
				}
			};

		// Check all the children of the given ParentWidget, but only track the index at the top level
		for (int32 ChildIdx = 0; ChildIdx < ParentWidget->GetChildrenCount(); ++ChildIdx)
		{
			CheckWidgetFunc(ParentWidget->GetChildAt(ChildIdx));

			if (!FoundChild)
			{
				ForWidgetAndChildren(ParentWidget->GetChildAt(ChildIdx), CheckWidgetFunc);
			}

			if (FoundChild)
			{
				OutChildIndex = ChildIdx;
				break;
			}
		}
	}

	return FoundChild;
}

int32 UWidgetTree::FindChildIndex(const UPanelWidget* ParentWidget, const UWidget* ChildWidget)
{
	const UWidget* CurrentWidget = ChildWidget;
	while (CurrentWidget)
	{
		const UPanelWidget* NextParent = CurrentWidget->Slot ? CurrentWidget->Slot->Parent : nullptr;
		if (NextParent && NextParent == ParentWidget)
		{
			return NextParent->GetChildIndex(CurrentWidget);
		}

		CurrentWidget = NextParent;
	}

	return INDEX_NONE;
}

bool UWidgetTree::RemoveWidget(UWidget* InRemovedWidget)
{
	bool bRemoved = false;

	UPanelWidget* InRemovedWidgetParent = InRemovedWidget->GetParent();
	if ( InRemovedWidgetParent )
	{
		if ( InRemovedWidgetParent->RemoveChild(InRemovedWidget) )
		{
			bRemoved = true;
		}
	}
	// If the widget being removed is the root, null it out.
	else if ( InRemovedWidget == RootWidget )
	{
		RootWidget = nullptr;
		bRemoved = true;
	}
	else
	{
		for (const auto& KVP : NamedSlotBindings)
		{
			if (KVP.Value == InRemovedWidget)
			{
				bRemoved = true;
				SetContentForSlot(KVP.Key, nullptr);
				break;
			}
		}
	}

	return bRemoved;
}

bool UWidgetTree::TryMoveWidgetToNewTree(UWidget* Widget, UWidgetTree* DestinationTree)
{
	bool bWidgetMoved = false;

	// A Widget's Outer is always a WidgetTree
	UWidgetTree* OriginalTree = Widget ? Cast<UWidgetTree>(Widget->GetOuter()) : nullptr;

	if (DestinationTree && OriginalTree && OriginalTree != DestinationTree)
	{
		bWidgetMoved = Widget->Rename(*Widget->GetName(), DestinationTree, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	}

	return bWidgetMoved;
}

void UWidgetTree::GetAllWidgets(TArray<UWidget*>& Widgets) const
{
	ForEachWidget([&Widgets] (UWidget* Widget) {
		Widgets.Add(Widget);
	});
}

void UWidgetTree::GetChildWidgets(UWidget* Parent, TArray<UWidget*>& Widgets)
{
	ForWidgetAndChildren(Parent, [&Widgets] (UWidget* Widget) {
		Widgets.Add(Widget);
	});
}

void UWidgetTree::ForEachWidget(TFunctionRef<void(UWidget*)> Predicate) const
{
	// Start with the root widget.
	if ( RootWidget )
	{
		Predicate(RootWidget);

		ForWidgetAndChildren(RootWidget, Predicate);
	}

	// Next, check the top level named slots.  Once the UserWidget that owns this widget tree is initialized, these
	// named slot bindings will all be emptied out, and their relevant named slot content will be inserted into the
	// hierarchy, but before that happens, or during the design time template, we need to treat them as separate
	// top level widgets because they're not yet in the hierarchy.
	for (const auto& NamedSlotsEntry : NamedSlotBindings)
	{
		if (UWidget* NamedSlotContent = NamedSlotsEntry.Value)
		{
			Predicate(NamedSlotContent);
			ForWidgetAndChildren(NamedSlotContent, Predicate);
		}
	}
}

void UWidgetTree::ForEachWidgetAndDescendants(TFunctionRef<void(UWidget*)> Predicate) const
{
	ForEachWidget([this, &Predicate] (UWidget* Widget) {
		if ( Widget != RootWidget )
		{
			if (const UUserWidget* UserWidgetChild = Cast<UUserWidget>(Widget))
			{
				if ( UserWidgetChild->WidgetTree )
				{
					UserWidgetChild->WidgetTree->ForEachWidgetAndDescendants(Predicate);
					return;
				}
			}
		}

		Predicate(Widget);
	});
}

void UWidgetTree::ForWidgetAndChildren(UWidget* Widget, TFunctionRef<void(UWidget*)> Predicate)
{
	// Search for any named slot with content that we need to dive into.
	if ( INamedSlotInterface* NamedSlotHost = Cast<INamedSlotInterface>(Widget) )
	{
		TArray<FName> SlotNames;
		NamedSlotHost->GetSlotNames(SlotNames);

		for ( FName SlotName : SlotNames )
		{
			if ( UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName) )
			{
				Predicate(SlotContent);

				ForWidgetAndChildren(SlotContent, Predicate);
			}
		}
	}

	// Search standard children.
	if ( UPanelWidget* PanelParent = Cast<UPanelWidget>(Widget) )
	{
		for ( int32 ChildIndex = 0; ChildIndex < PanelParent->GetChildrenCount(); ChildIndex++ )
		{
			if ( UWidget* ChildWidget = PanelParent->GetChildAt(ChildIndex) )
			{
				Predicate(ChildWidget);

				ForWidgetAndChildren(ChildWidget, Predicate);
			}
		}
	}
}

void UWidgetTree::ForEachWidgetUntil(TFunctionRef<bool(UWidget*)> Predicate) const
{
	// Mimics UWidgetTree::ForWidget
	if ( RootWidget )
	{
		if (!Predicate(RootWidget))
		{
			return;
		}

		if (!ForEachWidgetAndChildrenUntil(RootWidget, Predicate))
		{
			return;
		}
	}

	for (const auto& NamedSlotsEntry : NamedSlotBindings)
	{
		if (UWidget* NamedSlotContent = NamedSlotsEntry.Value)
		{
			if(!Predicate(NamedSlotContent))
			{
				return;
			}
			
			if(!ForEachWidgetAndChildrenUntil(NamedSlotContent, Predicate))
			{
				return;
			}
		}
	}
}

bool UWidgetTree::ForEachWidgetAndChildrenUntil(UWidget* Widget, TFunctionRef<bool(UWidget*)> Predicate)
{
	// Mimics UWidgetTree::ForWidgetAndChildren
	if ( INamedSlotInterface* NamedSlotHost = Cast<INamedSlotInterface>(Widget) )
	{
		TArray<FName> SlotNames;
		NamedSlotHost->GetSlotNames(SlotNames);

		for ( FName SlotName : SlotNames )
		{
			if ( UWidget* SlotContent = NamedSlotHost->GetContentForSlot(SlotName) )
			{
				if (!Predicate(SlotContent))
				{
					return false;
				}

				if (!ForEachWidgetAndChildrenUntil(SlotContent, Predicate))
				{
					return false;
				}
			}
		}
	}

	// Search standard children.
	if ( UPanelWidget* PanelParent = Cast<UPanelWidget>(Widget) )
	{
		for ( int32 ChildIndex = 0; ChildIndex < PanelParent->GetChildrenCount(); ChildIndex++ )
		{
			if ( UWidget* ChildWidget = PanelParent->GetChildAt(ChildIndex) )
			{
				if(!Predicate(ChildWidget))
				{
					return false;
				}

				if (!ForEachWidgetAndChildrenUntil(ChildWidget, Predicate))
				{
					return false;
				}
			}
		}
	}

	return true;
}

// INamedSlotInterface
//----------------------------------------------------------------------------------------

void UWidgetTree::GetSlotNames(TArray<FName>& SlotNames) const
{
	// Widget trees don't know for sure how many slots the real widget tree has, they can only tell you about the
	// slots that have content in them.
	NamedSlotBindings.GetKeys(SlotNames);
}

UWidget* UWidgetTree::GetContentForSlot(FName SlotName) const
{
	return NamedSlotBindings.FindRef(SlotName);
}

void UWidgetTree::SetContentForSlot(FName SlotName, UWidget* Content)
{
	if (Content)
	{
		NamedSlotBindings.Add(SlotName, Content);	
	}
	else
	{
		NamedSlotBindings.Remove(SlotName);	
	}
}

//----------------------------------------------------------------------------------------

void UWidgetTree::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	if (!ObjectSaveContext.IsCooking()) // AllWidgets is not used in cooked builds
	{
#if WITH_EDITORONLY_DATA
		// Update AllWidgets array used during serialization to match the current state of the WidgetTree
		AllWidgets.Empty();
		GetAllWidgets(MutableView(AllWidgets));
#endif
	}

#if WITH_EDITOR && UE_BUILD_DEBUG
	ForEachWidgetAndDescendants([this](UWidget* InChildWidget) {
		// Each widget tree is expected to only contain direct children,
		// adding a check to see if any of them contain anything but direct children.
		// It's unclear if this is actually a problem, but it is unexpected based on
		// the design, so adding a check to see if that proves not to be the case.
		if (InChildWidget->GetOuter() != this)
		{
			UE_LOGF(LogUMG, Warning, "WidgetTree(%ls) Contains Foreign Child (%ls)", *GetPathName(), *InChildWidget->GetPathName());
		}
	});
#endif

	Super::PreSave(ObjectSaveContext);
}

void UWidgetTree::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	FixupOrphanedWidgetNames();
	AllWidgets.Empty();
#endif
}

#if WITH_EDITORONLY_DATA

void UWidgetTree::FixupOrphanedWidgetNames()
{
	// Build set of widgets actually referenced by panel slots
    TSet<UWidget*> Widgets;
    TSet<UWidget*> ReferencedWidgets;
	ForEachWidget([&](UWidget* Widget)
    {
		Widgets.Add(Widget);
        if (UPanelWidget* Panel = Cast<UPanelWidget>(Widget))
        {
            for (int32 i = 0; i < Panel->GetChildrenCount(); ++i)
            {
                if (UWidget* Child = Panel->GetChildAt(i))
                {
                    ReferencedWidgets.Add(Child);
                }
            }
        }
    });

    for (const auto& NamedSlotEntry : NamedSlotBindings)
    {
        if (UWidget* NamedSlotContent = NamedSlotEntry.Value)
        {
            ReferencedWidgets.Add(NamedSlotContent);
        }
    }

    TMap<FName, UWidget*> NameToWidgetMap;
    for(UWidget* Widget : Widgets)
    {
        FName Name = Widget->GetFName();
        if (UWidget** ExistingWidget = NameToWidgetMap.Find(Name))
        {
			// Rename whichever widget is not already referenced 
            UWidget* WidgetToRename = ReferencedWidgets.Contains(Widget) ? *ExistingWidget : Widget;
            const FName NewName = MakeUniqueObjectName(WidgetToRename->GetOuter(), WidgetToRename->GetClass(), Name);

			UE_LOGF(LogUMG, Warning, "WidgetTree(%ls) Contain duplicate FName (%ls). Referencing both '%ls' and '%ls'", *GetPathName(), *Name.ToString(), *((*ExistingWidget)->GetPathName()), *Widget->GetPathName());
			UE_LOGF(LogUMG, Warning, "Renaming '%ls' to '%ls'", *WidgetToRename->GetName(), *NewName.ToString());

			WidgetToRename->Rename(*NewName.ToString(), nullptr, REN_DoNotDirty);
			NameToWidgetMap[NewName] = Widget;
        }
        else
        {
            NameToWidgetMap.Add(Name, Widget);
        }
    }
}
#endif //WITH_EDITORONLY_DATA
