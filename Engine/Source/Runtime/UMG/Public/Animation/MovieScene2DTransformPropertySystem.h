// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"
#include "MovieScene2DTransformPropertySystem.generated.h"

UCLASS(MinimalAPI)
class UMovieScene2DTransformPropertySystem : public UMovieScenePropertySystem
{
public:

	GENERATED_BODY()

	UMG_API UMovieScene2DTransformPropertySystem(const FObjectInitializer& ObjInit);

	UMG_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
