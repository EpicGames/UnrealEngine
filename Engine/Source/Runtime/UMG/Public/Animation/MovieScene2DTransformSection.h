// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieScene2DTransformSection.generated.h"


struct FWidgetTransform;

enum class EMovieScene2DTransformChannel : uint32
{
	None			= 0x000,

	TranslationX 	= 0x001,
	TranslationY 	= 0x002,
	Translation 	= TranslationX | TranslationY,

	Rotation 		= 0x004,

	ScaleX 			= 0x008,
	ScaleY 			= 0x010,
	Scale 			= ScaleX | ScaleY,

	ShearX 			= 0x020,
	ShearY 			= 0x040,
	Shear           = ShearX | ShearY,

	AllTransform	= Translation | Rotation | Scale | Shear
};
ENUM_CLASS_FLAGS(EMovieScene2DTransformChannel)

USTRUCT()
struct FMovieScene2DTransformMask
{
	GENERATED_BODY()

	FMovieScene2DTransformMask()
		: Mask(0)
	{}

	FMovieScene2DTransformMask(EMovieScene2DTransformChannel Channel)
		: Mask((__underlying_type(EMovieScene2DTransformChannel))Channel)
	{}

	EMovieScene2DTransformChannel GetChannels() const
	{
		return (EMovieScene2DTransformChannel)Mask;
	}

	FVector2D GetTranslationFactor() const
	{
		EMovieScene2DTransformChannel Channels = GetChannels();
		return FVector2D(
			EnumHasAllFlags(Channels, EMovieScene2DTransformChannel::TranslationX) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieScene2DTransformChannel::TranslationY) ? 1.f : 0.f);
	}

	float GetRotationFactor() const
	{
		EMovieScene2DTransformChannel Channels = GetChannels();
		if (EnumHasAllFlags(Channels, EMovieScene2DTransformChannel::Rotation))
		{
			return 1.f;
		}
		return 0.f;
	}

	FVector2D GetScaleFactor() const
	{
		EMovieScene2DTransformChannel Channels = GetChannels();
		return FVector2D(
			EnumHasAllFlags(Channels, EMovieScene2DTransformChannel::ScaleX) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieScene2DTransformChannel::ScaleY) ? 1.f : 0.f);
	}

	FVector2D GetShearFactor() const
	{
		EMovieScene2DTransformChannel Channels = GetChannels();
		return FVector2D(
			EnumHasAllFlags(Channels, EMovieScene2DTransformChannel::ShearX) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieScene2DTransformChannel::ShearY) ? 1.f : 0.f);
	}

private:

	UPROPERTY()
	uint32 Mask;
};

/**
 * A transform section
 */
UCLASS(MinimalAPI)
class UMovieScene2DTransformSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:
		
	/**
	 * Access the mask that defines which channels this track should animate
	 */
	UMG_API FMovieScene2DTransformMask GetMask() const;

	/**
	 * Set the mask that defines which channels this track should animate
	 */
	UMG_API void SetMask(FMovieScene2DTransformMask NewMask);

	/**
	 * Get the mask by name
	 */
	UMG_API FMovieScene2DTransformMask GetMaskByName(const FName& InName) const;

	UMG_API FWidgetTransform GetCurrentValue(const UObject* Object) const;

protected:

	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:

	UPROPERTY()
	FMovieScene2DTransformMask TransformMask;

	/** Translation curves*/
	UPROPERTY()
	FMovieSceneFloatChannel Translation[2];
	
	/** Rotation curve */
	UPROPERTY()
	FMovieSceneFloatChannel Rotation;

	/** Scale curves */
	UPROPERTY()
	FMovieSceneFloatChannel Scale[2];

	/** Shear curve */
	UPROPERTY()
	FMovieSceneFloatChannel Shear[2];
};
