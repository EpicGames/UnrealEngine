// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieScene2DTransformTrack.generated.h"

struct FWidgetTransform;

/**
 * Handles manipulation of 2D transforms in a movie scene
 */
UCLASS( MinimalAPI )
class UMovieScene2DTransformTrack
	: public UMovieScenePropertyTrack
{
	GENERATED_BODY()

public:

	UMovieScene2DTransformTrack(const FObjectInitializer& ObjectInitializer);

	//~ UMovieSceneTrack interface
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
};
