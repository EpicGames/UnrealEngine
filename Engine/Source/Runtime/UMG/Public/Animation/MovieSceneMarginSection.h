// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "MovieSceneSection.h"
#include "Layout/Margin.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneMarginSection.generated.h"

/**
 * A section in a Margin track
 */
UCLASS(MinimalAPI)
class UMovieSceneMarginSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/** Red curve data */
	UPROPERTY()
	FMovieSceneFloatChannel TopCurve;

	/** Green curve data */
	UPROPERTY()
	FMovieSceneFloatChannel LeftCurve;

	/** Blue curve data */
	UPROPERTY()
	FMovieSceneFloatChannel RightCurve;

	/** Alpha curve data */
	UPROPERTY()
	FMovieSceneFloatChannel BottomCurve;

private:

	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
};
