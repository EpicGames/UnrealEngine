// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneMarginTrack.generated.h"

/**
 * Handles manipulation of FMargins in a movie scene
 */
UCLASS( MinimalAPI )
class UMovieSceneMarginTrack : public UMovieScenePropertyTrack
{
	GENERATED_BODY()

public:

	UMovieSceneMarginTrack(const FObjectInitializer& Init);
	
	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
};
