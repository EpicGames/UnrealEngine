// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "Slate/WidgetTransform.h"
#include "Animation/WidgetMaterialTrackUtilities.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePropertyTraits.h"
#include "EntitySystem/MovieScenePropertyMetaData.h"

#include "Containers/ArrayView.h"


namespace UE
{
namespace MovieScene
{

struct FWidgetMaterialPath
{
	FWidgetMaterialPath() = default;
	FWidgetMaterialPath(TArrayView<const FName> Names)
		: Path(Names.GetData(), Names.Num())
	{}

	TArray<FName, TInlineAllocator<2>> Path;
};

struct FIntermediateWidgetTransform
{
	double TranslationX;
	double TranslationY;
	double Rotation;
	double ScaleX;
	double ScaleY;
	double ShearX;
	double ShearY;
};
UMG_API void ConvertOperationalProperty(const FIntermediateWidgetTransform& In, FWidgetTransform& Out);
UMG_API void ConvertOperationalProperty(const FWidgetTransform& In, FIntermediateWidgetTransform& Out);

struct FIntermediateMargin
{
	double Left;
	double Top;
	double Right;
	double Bottom;
};
UMG_API void ConvertOperationalProperty(const FIntermediateMargin& In, FMargin& Out);
UMG_API void ConvertOperationalProperty(const FMargin& In, FIntermediateMargin& Out);

UMG_API void UnpackChannelsFromOperational(const FIntermediateMargin& In, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
UMG_API void UnpackChannelsFromOperational(const FIntermediateWidgetTransform& In, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);

using FMarginTraits = TPropertyTraits<FMargin, FIntermediateMargin>;
using FWidgetTransformPropertyTraits = TPropertyTraits<FWidgetTransform, FIntermediateWidgetTransform>;

struct FMovieSceneUMGComponentTypes
{
	UMG_API ~FMovieSceneUMGComponentTypes();

	TPropertyComponents<FMarginTraits> Margin;
	TPropertyComponents<FWidgetTransformPropertyTraits> WidgetTransform;

	TComponentTypeID<FWidgetMaterialPath> WidgetMaterialPath;
	TComponentTypeID<FWidgetMaterialHandle> WidgetMaterialHandle;

	TCustomPropertyRegistration<FMarginTraits, 1> CustomMarginAccessors;
	TCustomPropertyRegistration<FWidgetTransformPropertyTraits, 1> CustomWidgetTransformAccessors;

	static UMG_API void Destroy();

	static UMG_API FMovieSceneUMGComponentTypes* Get();

private:
	FMovieSceneUMGComponentTypes();
};


} // namespace MovieScene
} // namespace UE
