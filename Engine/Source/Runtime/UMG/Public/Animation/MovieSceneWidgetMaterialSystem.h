// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"

#include "Systems/MovieSceneMaterialSystem.h"
#include "Animation/MovieSceneUMGComponentTypes.h"

#include "MovieSceneWidgetMaterialSystem.generated.h"

class UWidget;
class UMovieScenePiecewiseDoubleBlenderSystem;

namespace UE::MovieScene
{

struct FWidgetMaterialKey
{
	FObjectKey Object;
	FWidgetMaterialHandle WidgetMaterialHandle;

	friend uint32 GetTypeHash(const FWidgetMaterialKey& In)
	{
		return GetTypeHash(In.Object) ^ GetTypeHash(In.WidgetMaterialHandle);
	}
	friend bool operator==(const FWidgetMaterialKey& A, const FWidgetMaterialKey& B)
	{
		return A.Object == B.Object && A.WidgetMaterialHandle == B.WidgetMaterialHandle;
	}
};

struct FWidgetMaterialAccessor
{
	using KeyType = FWidgetMaterialKey;

	UWidget* Widget;
	FWidgetMaterialHandle WidgetMaterialHandle;

	FWidgetMaterialAccessor(const FWidgetMaterialKey& InKey);
	FWidgetMaterialAccessor(UObject* InObject, FWidgetMaterialHandle InWidgetMaterialHandle);

	explicit operator bool() const;

	UMaterialInterface* GetMaterial() const;
	void SetMaterial(UMaterialInterface* InMaterial);
	UMaterialInstanceDynamic* CreateDynamicMaterial(UMaterialInterface* InMaterial);
	FString ToString() const;
};

using FPreAnimatedWidgetMaterialTraits          = TPreAnimatedMaterialTraits<FWidgetMaterialAccessor, UObject*, FWidgetMaterialHandle>;
using FPreAnimatedWidgetMaterialParameterTraits = TPreAnimatedMaterialParameterTraits<FWidgetMaterialAccessor, UObject*, FWidgetMaterialHandle>;

struct FPreAnimatedWidgetMaterialSwitcherStorage
	: public TPreAnimatedStateStorage<TPreAnimatedMaterialTraits<FWidgetMaterialAccessor, UObject*, FWidgetMaterialHandle>>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedWidgetMaterialSwitcherStorage> StorageID;
};

struct FPreAnimatedWidgetMaterialParameterStorage
	: public TPreAnimatedStateStorage<TPreAnimatedMaterialParameterTraits<FWidgetMaterialAccessor, UObject*, FWidgetMaterialHandle>>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedWidgetMaterialParameterStorage> StorageID;
};

} // namespace UE::MovieScene


UCLASS(MinimalAPI)
class UMovieSceneWidgetMaterialSystem
	: public UMovieSceneEntitySystem
	, public IMovieScenePreAnimatedStateSystemInterface
{
public:

	GENERATED_BODY()

	UMovieSceneWidgetMaterialSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) override;

private:

	UE::MovieScene::TMovieSceneMaterialSystem<UE::MovieScene::FWidgetMaterialAccessor, UObject*, FWidgetMaterialHandle> SystemImpl;
};
