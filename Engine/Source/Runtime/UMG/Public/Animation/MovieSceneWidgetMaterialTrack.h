// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Internationalization/Text.h"
#include "Sections/MovieSceneParameterSection.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneWidgetMaterialTrack.generated.h"

class UMovieSceneEntitySystemLinker;
class UMovieSceneSection;
class UObject;
struct FFrameNumber;
struct FMovieSceneEntityComponentFieldBuilder;
struct FMovieSceneEvaluationFieldEntityMetaData;
template <typename ElementType> class TRange;

/**
 * A material track which is specialized for materials which are owned by widget brushes.
 */
UCLASS(MinimalAPI)
class UMovieSceneWidgetMaterialTrack
	: public UMovieSceneMaterialTrack
	, public IMovieSceneEntityProvider
	, public IMovieSceneParameterSectionExtender
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieSceneTrack interface
	virtual FName GetTrackName() const override;	
	
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

	/*~ IMovieSceneEntityProvider */
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	/*~ IMovieSceneParameterSectionExtender */
	virtual void ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

public:

	/** Gets name of the brush property which has the material to animate. */
	const TArray<FName>& GetBrushPropertyNamePath() const { return BrushPropertyNamePath; }

	/** Sets the name of the brush property which has the material to animate. */
	UMG_API void SetBrushPropertyNamePath( TArray<FName> InBrushPropertyNamePath );

private:

	/** The name of the brush property which will be animated by this track. */
	UPROPERTY()
	TArray<FName> BrushPropertyNamePath;

	/** The name of this track, generated from the property name path. */
	UPROPERTY()
	FName TrackName;
};
