// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Blueprint/UserWidget.h"
#include "IMovieScenePlayer.h"
#include "Animation/UMGSequenceTickManager.h"
#include "Animation/WidgetAnimationHandle.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Misc/QualifiedFrameTime.h"
#include "UMGSequencePlayer.generated.h"

class UWidgetAnimation;

UCLASS(Transient, BlueprintType, MinimalAPI)
class UUMGSequencePlayer : public UObject, public IMovieScenePlayer
{
	GENERATED_BODY()

public:

	UUMGSequencePlayer(const FObjectInitializer& ObjectInitializer);

	UMG_API void InitSequencePlayer(FWidgetAnimationState& InState);

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void InitSequencePlayer(UWidgetAnimation& InAnimation, UUserWidget& InUserWidget);

	/** Updates the running movie */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void Tick(float DeltaTime);

	/** Begins playing or restarts an animation */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void Play(float StartAtTime, int32 InNumLoopsToPlay, EUMGSequencePlayMode::Type InPlayMode, float InPlaybackSpeed, bool bRestoreState);

	/** Begins playing or restarts an animation  and plays to the specified end time */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void PlayTo(float StartAtTime, float EndAtTime, int32 InNumLoopsToPlay, EUMGSequencePlayMode::Type InPlayMode, float InPlaybackSpeed, bool bRestoreState);

	/** Stops a running animation and resets time */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void Stop();

	/** Pauses a running animation */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void Pause();

	/** Reverses a running animation */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void Reverse();

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void SetCurrentTime(float InTime);

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API FQualifiedFrameTime GetCurrentTime() const;

	/** @return The current animation being played */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API const UWidgetAnimation* GetAnimation() const;

	/** @return */
	UFUNCTION(BlueprintCallable, Category="Animation", meta=(DeprecatedFunction))
	UMG_API FName GetUserTag() const;

	UFUNCTION(BlueprintCallable, Category = "Animation", meta=(DeprecatedFunction))
	UMG_API void SetUserTag(FName InUserTag);

	/** Sets the number of loops to play */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void SetNumLoopsToPlay(int32 InNumLoopsToPlay);

	/** Sets the animation playback rate */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void SetPlaybackSpeed(float PlaybackSpeed);

	/** Gets the current time position in the player (in seconds). */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API bool IsPlayingForward() const;

	/** Check whether this player is currently being stopped */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API bool IsStopping() const;

public:

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual UObject* AsUObject() override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual IMovieScenePlaybackClient* GetPlaybackClient() override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual FMovieSceneSpawnRegister& GetSpawnRegister() override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual UObject* GetPlaybackContext() const override;

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API virtual void InitializeRootInstance(TSharedRef<UE::MovieScene::FSharedPlaybackState> NewSharedPlaybackState) override;

public:

	UMG_API virtual void BeginDestroy() override;

	/** Disable this sequence player by removing any of its animation data from the entity manager */
	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void RemoveEvaluationData();

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API void TearDown();

	DECLARE_EVENT_OneParam(UUMGSequencePlayer, FOnSequenceFinishedPlaying, UUMGSequencePlayer&);

	UE_DEPRECATED(5.6, "Please use FWidgetAnimationHandle and FWidgetAnimationState")
	UMG_API FOnSequenceFinishedPlaying& OnSequenceFinishedPlaying();

public:

	// Internal API.
	void BroadcastSequenceFinishedPlaying();

private:

	FWidgetAnimationHandle WidgetAnimationHandle;

	FMovieSceneRootEvaluationTemplateInstance RootTemplateInstance;

	FOnSequenceFinishedPlaying OnSequenceFinishedPlayingEvent;
};

