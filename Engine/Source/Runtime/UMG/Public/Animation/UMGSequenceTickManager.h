// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "MovieSceneLatentActionManager.h"

#include "UMGSequenceTickManager.generated.h"

class FMovieSceneEntitySystemRunner;
class UMovieSceneEntitySystemLinker;
class UUserWidget;

USTRUCT()
struct FSequenceTickManagerWidgetData
{
	GENERATED_BODY()

	/** The widget was ticked. */
	bool bIsTicking = true;

	/** The widget's animation was ticked this frame. */
	bool bActionsAndAnimationTicked = false;
};


/**
 * An automatically created global object that will manage all widget animations.
 */
UCLASS(MinimalAPI)
class UUMGSequenceTickManager : public UObject
{
public:
	GENERATED_BODY()

	UMG_API UUMGSequenceTickManager(const FObjectInitializer& Init);

	UMG_API void Initialize(UObject* Owner);

	UMovieSceneEntitySystemLinker* GetLinker() { return Linker; }
	TSharedPtr<FMovieSceneEntitySystemRunner> GetRunner() { return Runner; }

	UMG_API void AddLatentAction(FMovieSceneSequenceLatentActionDelegate Delegate);
	UMG_API void ClearLatentActions(UObject* Object);
	UMG_API void RunLatentActions();

	static UMG_API UUMGSequenceTickManager* Get(UObject* PlaybackContext);

	UMG_API void ForceFlush();

	UMG_API void AddWidget(UUserWidget* InWidget);
	UMG_API void RemoveWidget(UUserWidget* InWidget);

	UMG_API void OnWidgetTicked(UUserWidget* InWidget);

private:
	UMG_API virtual void BeginDestroy() override;

	UMG_API void HandleSlatePostTick(float DeltaSeconds);
	UMG_API void TickWidgetAnimations(float DeltaSeconds);
	UMG_API bool TickWidgetAnimations(
			float DeltaSeconds, 
			UUserWidget* UserWidget, 
			FSequenceTickManagerWidgetData& UserWidgetData,
			bool bIsCurrentlyEvaluating);

private:

	UPROPERTY(transient)
	TMap<TWeakObjectPtr<UUserWidget>, FSequenceTickManagerWidgetData> WeakUserWidgetData;

	UPROPERTY(transient)
	TArray<TObjectPtr<UUserWidget>> PendingUserWidgets;

	UPROPERTY(transient)
	TObjectPtr<UMovieSceneEntitySystemLinker> Linker;

	TSharedPtr<FMovieSceneEntitySystemRunner> Runner;

	FDelegateHandle SlateApplicationPreTickHandle, SlateApplicationPostTickHandle;

	FMovieSceneLatentActionManager LatentActionManager;

	bool bIsTicking;
};
