// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "MovieSceneSequence.h"
#include "Animation/WidgetAnimationBinding.h"
#include "Animation/WidgetAnimationEvents.h"
#include "WidgetAnimation.generated.h"

class UMovieScene;
class UUserWidget;

#if WITH_EDITOR
struct FSlateBrush;
#endif


/**
 * 
 */
UCLASS(BlueprintType, MinimalAPI)
class UWidgetAnimation : public UMovieSceneSequence
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITOR
	/**
	 * Get a placeholder animation.
	 *
	 * @return Placeholder animation.
	 */
	static UMG_API UWidgetAnimation* GetNullAnimation();

	/** @return The friendly name of the animation */
	const FString& GetDisplayLabel() const
	{
		return DisplayLabel;
	}

	/** Sets the friendly name of the animation to display in the editor */
	UMG_API void SetDisplayLabel(const FString& InDisplayLabel);

	/** Returns the DisplayLabel if set, otherwise the object name */
	UMG_API virtual FText GetDisplayName() const override;
#endif

	/**
	 * Get the start time of this animation.
	 *
	 * @return Start time in seconds.
	 * @see GetEndTime
	 */
	UFUNCTION(BlueprintCallable, Category="Animation")
	UMG_API float GetStartTime() const;

	/**
	 * Get the end time of this animation.
	 *
	 * @return End time in seconds.
	 * @see GetStartTime
	 */
	UFUNCTION(BlueprintCallable, Category="Animation")
	UMG_API float GetEndTime() const;

	// These animation binding functions were added so that we could cleanly upgrade assets
	// from before animation sharing, they don't actually modify the animation, they just pipe
	// through to the UUserWidget.  If we didn't put the functions here, it would be much more
	// difficult to upgrade users who were taking advantage of the Many-To-1, blueprint having
	// many animations binding to the same delegate.

	UFUNCTION(BlueprintCallable, Category = Animation, meta=(BlueprintInternalUseOnly = "TRUE"))
	void BindToAnimationStarted(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate);

	UFUNCTION(BlueprintCallable, Category = Animation, meta = (BlueprintInternalUseOnly = "TRUE"))
	void UnbindFromAnimationStarted(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate);

	UFUNCTION(BlueprintCallable, Category = Animation, meta = (BlueprintInternalUseOnly = "TRUE"))
	void UnbindAllFromAnimationStarted(UUserWidget* Widget);

	UFUNCTION(BlueprintCallable, Category = Animation, meta = (BlueprintInternalUseOnly = "TRUE"))
	void BindToAnimationFinished(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate);

	UFUNCTION(BlueprintCallable, Category = Animation, meta = (BlueprintInternalUseOnly = "TRUE"))
	void UnbindFromAnimationFinished(UUserWidget* Widget, FWidgetAnimationDynamicEvent Delegate);

	UFUNCTION(BlueprintCallable, Category = Animation, meta = (BlueprintInternalUseOnly = "TRUE"))
	void UnbindAllFromAnimationFinished(UUserWidget* Widget);

public:

	// UMovieSceneAnimation overrides
	virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override;
	virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override;
	virtual UMovieScene* GetMovieScene() const override;
	virtual UObject* GetParentObject(UObject* Object) const override;
	virtual void UnbindPossessableObjects(const FGuid& ObjectId) override;
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* InContext) override {}
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* InContext) override {}
	virtual void LocateBoundObjects(const FGuid& ObjectId, const UE::UniversalObjectLocator::FResolveParams& ResolveParams, TSharedPtr<const FSharedPlaybackState> SharedPlaybackState, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	virtual UObject* CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID) override;

	
#if WITH_EDITOR
	virtual void IterateDynamicBindings(TFunction<void(const FGuid&, FMovieSceneDynamicBinding&)> InCallback) override;
	virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const override;
	virtual bool IsFilterSupportedImpl(const FString& InFilterName) const override;
	virtual const FSlateBrush* GetCustomBrushForBinding(FGuid BindingID) const override;

	DECLARE_MULTICAST_DELEGATE_OneParam(FFixupWidgetDynamicBindingsEvent, UWidgetAnimation*);
	static UMG_API FFixupWidgetDynamicBindingsEvent FixupWidgetDynamicBindingsEvent;
#endif
	// ~UMovieSceneAnimation overrides

	//~ Begin UObject Interface. 
	virtual bool IsPostLoadThreadSafe() const override;
	//~ End UObject Interface

	/** Get Animation bindings of the animation */
	const TArray<FWidgetAnimationBinding>& GetBindings() const { return AnimationBindings; }

	/** Remove Animation Binding */
	UMG_API void RemoveBinding(const UObject& PossessedObject);
	UMG_API void RemoveBinding(const FWidgetAnimationBinding& Binding);
	

	/** Whether to finish evaluation on stop */
	bool GetLegacyFinishOnStop() const { return bLegacyFinishOnStop; }

protected:

	/** Called after this object has been deserialized */
	virtual void PostLoad() override;

public:

	/** Pointer to the movie scene that controls this animation. */
	UPROPERTY()
	TObjectPtr<UMovieScene> MovieScene;

	/**  */
	UPROPERTY()
	TArray<FWidgetAnimationBinding> AnimationBindings;

private:

	/** Whether to finish evaluation on stop. This legacy value is to preserve existing asset behavior to NOT finish on stop since content was created with this bug. If this is removed, evaluation should always finish on stop. */
	UPROPERTY()
	bool bLegacyFinishOnStop;

	/** The friendly name for this animation displayed in the designer. */
	UPROPERTY()
	FString DisplayLabel;
};
