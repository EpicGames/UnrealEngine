// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MovieSceneDynamicBinding.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "WidgetAnimationBinding.generated.h"

class UUserWidget;
class UWidgetTree;
class UMovieSceneSequence;

namespace UE::MovieScene
{
	struct FSharedPlaybackState;
}


/**
 * A single object bound to a UMG sequence.
 */
USTRUCT()
struct FWidgetAnimationBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName WidgetName;

	UPROPERTY()
	FName SlotWidgetName;

	UPROPERTY()
	FGuid AnimationGuid;

	UPROPERTY()
	bool bIsRootWidget = false;

	UPROPERTY()
	FMovieSceneDynamicBinding DynamicBinding;

public:

	/**
	 * Locates a runtime object to animate from the provided tree of widgets.
	 * @return the runtime object to animate or null if not found
	 */
	UE_DEPRECATED(5.5, "Please use the version that takes a SharedPlaybackState and Sequence")
	UMG_API UObject* FindRuntimeObject(const UWidgetTree& WidgetTree, UUserWidget& UserWidget) const;

	UMG_API UObject* FindRuntimeObject(const UWidgetTree& WidgetTree, UUserWidget& UserWidget, const UMovieSceneSequence* Sequence, TSharedPtr<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const;

	/**
	 * Compares two widget animation bindings for equality.
	 *
	 * @param X The first binding to compare.
	 * @param Y The second binding to compare.
	 * @return true if the bindings are equal, false otherwise.
	 */
	friend bool operator==(const FWidgetAnimationBinding& X, const FWidgetAnimationBinding& Y)
	{
		return (X.WidgetName == Y.WidgetName) && (X.SlotWidgetName == Y.SlotWidgetName) && (X.AnimationGuid == Y.AnimationGuid) && (X.bIsRootWidget == Y.bIsRootWidget) && (X.DynamicBinding.Function == Y.DynamicBinding.Function);
	}
};
