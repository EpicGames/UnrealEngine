// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Blueprint/UserWidget.h"
#include "WidgetAnimationDelegateBinding.generated.h"

class UUserWidget;

USTRUCT()
struct FBlueprintWidgetAnimationDelegateBinding
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	EWidgetAnimationEvent Action;

	UPROPERTY()
	FName AnimationToBind;

	UPROPERTY()
	FName FunctionNameToBind;

	UPROPERTY()
	FName UserTag;

	FBlueprintWidgetAnimationDelegateBinding()
		: Action(EWidgetAnimationEvent::Started)
		, AnimationToBind(NAME_None)
		, FunctionNameToBind(NAME_None)
		, UserTag(NAME_None)
	{
	}
};

UCLASS(MinimalAPI)
class UWidgetAnimationDelegateBinding : public UDynamicBlueprintBinding
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FBlueprintWidgetAnimationDelegateBinding> WidgetAnimationDelegateBindings;

	UMG_API virtual void BindDynamicDelegates(UObject* InInstance) const override;
};
