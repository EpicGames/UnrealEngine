// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"

#include "WidgetAnimationEvents.generated.h"

UDELEGATE()
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnWidgetAnimationPlaybackStatusChanged);

UDELEGATE()
DECLARE_DYNAMIC_DELEGATE(FWidgetAnimationDynamicEvent);

UDELEGATE()
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWidgetAnimationDynamicEvents);
