// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SharedPointerFwd.h"

#include "WidgetAnimationHandle.generated.h"

#define UE_API UMG_API

class UUMGSequencePlayer;
class UUserWidget;
struct FWidgetAnimationState;

/**
 * Handle to an ongoing or finished widget animation.
 */
USTRUCT(BlueprintType)
struct FWidgetAnimationHandle
{
	GENERATED_BODY()

public:

	/** Creates an invalid handle. */
	UMG_API FWidgetAnimationHandle();

	/** Gets the animation state. */
	UMG_API FWidgetAnimationState* GetAnimationState() const;

	/** Gets the animation state. */
	UMG_API TSharedPtr<FWidgetAnimationState> PinAnimationState() const;

public:

	// For backwards compatibility in C++ code.
	UMG_API UUMGSequencePlayer* GetSequencePlayer() const;
	inline operator UUMGSequencePlayer*() const { return GetSequencePlayer(); }

public:

	/**
	 * Returns whether this handle is valid.
	 * A valid handle may still return a null animation state if the animation has finished playing.
	 */
	UMG_API bool IsValid() const;

	/** Gets the user tag associated with the running animation. */
	UMG_API FName GetUserTag() const;

	/** Sets the user tag associated with the running animation. */
	UMG_API void SetUserTag(FName InUserTag);

public:

	/** Type hash for using handles in maps and other containers. */
	friend uint32 GetTypeHash(const FWidgetAnimationHandle& Handle)
	{
		return GetTypeHash(Handle.WeakState);
	}

private:

	/** Creates a handle for the given running animation on the given widget. */
	UMG_API FWidgetAnimationHandle(TSharedPtr<FWidgetAnimationState> InState);

private:

	/** Weak pointer to the animation state. */
	TWeakPtr<FWidgetAnimationState> WeakState;

	// Only UUserWidget and FWidgetAnimationState can create handles.
	friend class UUserWidget;
	friend struct FWidgetAnimationState;
};

UCLASS(MinimalAPI)
class UWidgetAnimationHandleFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category = "UMG")
	static UE_API FName GetUserTag(const FWidgetAnimationHandle& Target);

	UFUNCTION(BlueprintCallable, Category = "UMG")
	static UE_API void SetUserTag(UPARAM(ref) FWidgetAnimationHandle& Target, FName InUserTag);
};

#undef UE_API
