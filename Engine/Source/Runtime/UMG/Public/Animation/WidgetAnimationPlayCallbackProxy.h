// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/WidgetAnimationHandle.h"
#include "Blueprint/UserWidget.h"

#include "WidgetAnimationPlayCallbackProxy.generated.h"

class UUMGSequencePlayer;
class UUserWidget;
class UWidgetAnimation;
struct FWidgetAnimationState;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FWidgetAnimationResult);

UCLASS(MinimalAPI)
class UWidgetAnimationPlayCallbackProxy : public UObject
{
	GENERATED_UCLASS_BODY()

	// Called when animation has been completed
	UPROPERTY(BlueprintAssignable)
	FWidgetAnimationResult Finished;

	UFUNCTION(BlueprintCallable , Category = "User Interface|Animation", meta = (
				BlueprintInternalUseOnly = "true",
				DisplayName = "Play Animation with Finished event (legacy)",
				ShortToolTip = "Play Animation and trigger event on Finished (legacy version using deprecated UMG Sequencer Player)",
				ToolTip="Play Animation on widget and trigger Finish event when the animation is done (legacy version using deprecated UMG Sequencer Player)."))
	static UWidgetAnimationPlayCallbackProxy* CreatePlayAnimationProxyObject(
			UUMGSequencePlayer*& Result,
			UUserWidget* Widget,
			UWidgetAnimation* InAnimation,
			float StartAtTime = 0.0f,
			int32 NumLoopsToPlay = 1,
			EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward,
			float PlaybackSpeed = 1.0f);

	UFUNCTION(BlueprintCallable , Category = "User Interface|Animation", meta = (
				BlueprintInternalUseOnly = "true",
				DisplayName = "Play Animation with Finished event",
				ShortToolTip = "Play Animation and trigger event on Finished",
				ToolTip="Play Animation on widget and trigger Finish event when the animation is done."))
	static UWidgetAnimationPlayCallbackProxy* NewPlayAnimationProxyObject(
			FWidgetAnimationHandle& Result,
			UUserWidget* Widget,
			UWidgetAnimation* InAnimation,
			float StartAtTime = 0.0f,
			int32 NumLoopsToPlay = 1,
			EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward,
			float PlaybackSpeed = 1.0f);

	UFUNCTION(BlueprintCallable, Category = "User Interface|Animation", meta = (
				BlueprintInternalUseOnly = "true",
				DisplayName = "Play Animation Time Range with Finished event (legacy)",
				ShortToolTip = "Play Animation Time Range and trigger event on Finished (legacy version using deprecated UMG Sequencer Player)",
				ToolTip = "Play Animation Time Range on widget and trigger Finish event when the animation is done (legacy version using deprecated UMG Sequencer Player)."))
	static UWidgetAnimationPlayCallbackProxy* CreatePlayAnimationTimeRangeProxyObject(
			UUMGSequencePlayer*& Result,
			UUserWidget* Widget,
			UWidgetAnimation* InAnimation,
			float StartAtTime = 0.0f,
			float EndAtTime = 0.0f,
			int32 NumLoopsToPlay = 1,
			EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward,
			float PlaybackSpeed = 1.0f);

	UFUNCTION(BlueprintCallable, Category = "User Interface|Animation", meta = (
				BlueprintInternalUseOnly = "true",
				DisplayName = "Play Animation Time Range with Finished event",
				ShortToolTip = "Play Animation Time Range and trigger event on Finished",
				ToolTip = "Play Animation Time Range on widget and trigger Finish event when the animation is done."))
	static UWidgetAnimationPlayCallbackProxy* NewPlayAnimationTimeRangeProxyObject(
			FWidgetAnimationHandle& Result,
			UUserWidget* Widget,
			UWidgetAnimation* InAnimation,
			float StartAtTime = 0.0f,
			float EndAtTime = 0.0f,
			int32 NumLoopsToPlay = 1,
			EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward,
			float PlaybackSpeed = 1.0f);

private:

	void ExecutePlayAnimation(UUserWidget* Widget, UWidgetAnimation* InAnimation, float StartAtTime, int32 NumLoopsToPlay, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed);
	void ExecutePlayAnimationTimeRange(UUserWidget* Widget, UWidgetAnimation* InAnimation, float StartAtTime, float EndAtTime, int32 NumLoopsToPlay, EUMGSequencePlayMode::Type PlayMode, float PlaybackSpeed);
	void OnSequenceFinished(FWidgetAnimationState& State);
	bool OnAnimationFinished(float DeltaTime);

	FWidgetAnimationHandle WidgetAnimationHandle;
	FDelegateHandle OnFinishedHandle;
};
