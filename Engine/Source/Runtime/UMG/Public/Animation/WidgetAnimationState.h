// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UMGSequencePlayMode.h"
#include "Animation/UMGSequenceTickManager.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieScenePlayback.h"
#include "Evaluation/MovieScenePlaybackManager.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieSceneFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

class UUserWidget;
class UWidgetAnimation;
class UUMGSequencePlayer;
struct FWidgetAnimationHandle;
struct FWidgetAnimationState;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnWidgetAnimationEvent, FWidgetAnimationState&);

struct FWidgetAnimationStatePlayParams
{
	/** The start time for the animation. */
	double StartAtTime = 0;
	/** The end time for the animation. */
	TOptional<double> EndAtTime;
	/** An optional start offset to apply to the animation. */
	TOptional<double> StartOffset;
	/** An optional end offset to apply to the animation. */
	TOptional<double> EndOffset;
	/** The number of loops to play before finishing the animation. */
	int32 NumLoopsToPlay = 1;
	/** The play-rate of the animation. */
	float PlaybackSpeed = 1.f;
	/** The play mode of the animation. */
	EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward;
	/** Whether to restore pre-animated state after the animation has finished. */
	bool bRestoreState = false;
};

/**
 * A class that runs an animation on a widget.
 */
struct FWidgetAnimationState : public TSharedFromThis<FWidgetAnimationState>
{
public:

	/** Builds a new widget animation state. */
	FWidgetAnimationState();

	/** Initializes the widget animation state. */
	void Initialize(UWidgetAnimation* InAnimation, UUserWidget* InUserWidget);

	/** Gets the user widget this state is animating. */
	UUserWidget* GetUserWidget() const { return WeakUserWidget.Get(); }

	/** Gets the current animation being played */
	const UWidgetAnimation* GetAnimation() const { return Animation; }

	/** Gets the playback state of the animation being played. */
	TSharedPtr<UE::MovieScene::FSharedPlaybackState> GetSharedPlaybackState() const { return WeakPlaybackState.Pin(); }

	/** Gets the user tag for the animation. */
	FName GetUserTag() const { return UserTag; }

	/** Sets the user tag for the animation. */
	void SetUserTag(FName InUserTag) { UserTag = InUserTag; }

	/** Returns whether this state is currently stopping. */
	bool IsStopping() const { return bIsStopping; }

	/** Returns whether this state is currently pending destruction. */
	bool IsPendingDelete() const { return bIsPendingDelete; }

	/** Allows registering a callback for when the animation has finished playing. */
	FOnWidgetAnimationEvent& GetOnWidgetAnimationFinished() { return OnWidgetAnimationFinishedEvent; }

public:

	/** Gets a legacy player object for backwards compatibility, lazily created it if needed. */
	UMG_API UUMGSequencePlayer* GetOrCreateLegacyPlayer();

	/** Gets a legacy player object for backwards compatibility, returning null if none exists yet. */
	UMG_API UUMGSequencePlayer* GetLegacyPlayer() const;

	/** Gets the widget animation handle for the animation being played. */
	UMG_API FWidgetAnimationHandle GetAnimationHandle() const;

	/** Gets whether the animation is playing forwards or backwards. */
	UMG_API bool IsPlayingForward() const;

	/** Gets the playback status of the animation. */
	UMG_API EMovieScenePlayerStatus::Type GetPlaybackStatus() const;

	/** Sets the playback status of the animation. */
	UMG_API void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus);

	/** Gets the current time of the animation. */
	UMG_API FQualifiedFrameTime GetCurrentTime() const;

	/** Sets the current time of the animation. */
	UMG_API void SetCurrentTime(float InTime);

	/** Sets the number of loops to play. */
	UMG_API void SetNumLoopsToPlay(int32 InNumLoopsToPlay);

	/** Sets the animation playback rate. */
	UMG_API void SetPlaybackSpeed(float PlaybackSpeed);

	/** Update the animation. */
	UMG_API void Tick(float InDeltaSeconds);

	/** Begins playing or restarts an animation. */
	UMG_API void Play(const FWidgetAnimationStatePlayParams& PlayParams);

	/** Stops a running animation and resets time. */
	UMG_API void Stop();

	/** Pauses a running animation. */
	UMG_API void Pause();

	/** Reverses a running animation. */
	UMG_API void Reverse();

	/** Disable this sequence player by removing any of its animation data from the entity manager. */
	UMG_API void RemoveEvaluationData();

	/** Tears down the animation. */
	UMG_API void TearDown();

	/** Returns whether this animation is valid. An uninitialized or torn-down state is not valid. */
	UMG_API bool IsValid() const;

	/** Collect objects for the GC. */
	UMG_API void AddReferencedObjects(FReferenceCollector& Collector);

private:

	bool NeedsLegacyPlayer() const;

	void FlushIfPrivateLinker();

	void OnBegunPlay();
	void OnStopped();

private:

	/** Animation being played */
	TObjectPtr<UWidgetAnimation> Animation;

	/** Legacy sequence player for backwards compatibility */
	TObjectPtr<UUMGSequencePlayer> LegacyPlayer;

	/** The user widget this sequence is animating */
	TWeakObjectPtr<UUserWidget> WeakUserWidget;

private:

	/** Shared playback state for the animation. */
	TWeakPtr<UE::MovieScene::FSharedPlaybackState> WeakPlaybackState;
	
	/** Private linker for blocking/synchronous running. */
	TObjectPtr<UMovieSceneEntitySystemLinker> PrivateLinker;

	/** Playback manager for the animation. */
	FMovieScenePlaybackManager PlaybackManager;

	/** The current playback mode. */
	EUMGSequencePlayMode::Type PlayMode;

	/**
	 * The 'state' tag the user may want to use to track what the animation is for.
	 *
	 * It's very common in UI to use the same animation for intro / outro, so this allows you 
	 * to tag what the animation is currently doing so that you can have some events just get 
	 * called back when the animation finishes the outtro, to say, remove the UI then.
	 */
	FName UserTag;

	/** Accumulated delta-time we were not able to evaluate with due to ongoing budgeted evaluation. */
	float BlockedDeltaTimeCompensation;

	/** Whether to restore pre-animated state. */
	bool bRestoreState : 1;

	/** Whether we are in the process of starting play. */
	bool bIsBeginningPlay : 1;

	/** Whether we are in the process of stopping. */
	bool bIsStopping : 1;

	/** Whether we have played, stopped, and now waiting to be deleted. */
	bool bIsPendingDelete : 1;

	/** Callback for when the animation has finished playing. */
	FOnWidgetAnimationEvent OnWidgetAnimationFinishedEvent;
};

