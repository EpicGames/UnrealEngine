// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMaterialInterface;
class UWidget;

/**
 * Handle to a widget material for easy getting and setting of the material.  Not designed to be stored
 */
class FWidgetMaterialHandle
{
public:
	FWidgetMaterialHandle()
		: TypeName(NAME_None)
		, Data(nullptr)
	{}

	FWidgetMaterialHandle(FName InTypeName, void* InData)
		: TypeName(InTypeName)
		, Data(InData)
	{}

	friend uint32 GetTypeHash(const FWidgetMaterialHandle& In)
	{
		return HashCombine(GetTypeHash(In.TypeName), GetTypeHash(In.Data));
	}
	friend bool operator==(const FWidgetMaterialHandle& A, const FWidgetMaterialHandle& B)
	{
		return A.TypeName == B.TypeName && A.Data == B.Data;
	}
	friend bool operator!=(const FWidgetMaterialHandle& A, const FWidgetMaterialHandle& B)
	{
		return !(A == B);
	}

	/** @return true if this handle points to valid data */
	bool IsValid() const { return TypeName != NAME_None && Data != nullptr; }

	/** Gets the currently bound material */
	UMG_API UMaterialInterface* GetMaterial() const;

	/** Sets the new bound material */
	UMG_API void SetMaterial(UMaterialInterface* InMaterial, UWidget* OwnerWidget);

private:
	/** Struct typename for that the data is pointing to */
	FName TypeName;
	/** Pointer to the struct data holding the material */
	void* Data;
};

struct FWidgetMaterialPropertyPath
{
	FWidgetMaterialPropertyPath(const TArray<FProperty*>& InPropertyPath, const FString& InDisplayName)
		: PropertyPath(InPropertyPath)
		, DisplayName(InDisplayName)
	{}

	TArray<FProperty*> PropertyPath;
	FString DisplayName;
};
namespace WidgetMaterialTrackUtilities
{
	/** Gets a material handle from a property on a widget by the properties FName path. */
	UMG_API FWidgetMaterialHandle GetMaterialHandle(UWidget* Widget, TArrayView<const FName> BrushPropertyNamePath);

	/** Gets the property paths on a widget which are slate brush properties, and who's slate brush has a valid material. */
	UMG_API void GetMaterialBrushPropertyPaths( UWidget* Widget, TArray<FWidgetMaterialPropertyPath>& MaterialBrushPropertyPaths );

	/** Converts a property name path into a single name which is appropriate for a track name. */
	UMG_API FName GetTrackNameFromPropertyNamePath( TArrayView<const FName> PropertyNamePath );
}
