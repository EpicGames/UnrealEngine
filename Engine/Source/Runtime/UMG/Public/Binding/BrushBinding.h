// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Styling/SlateBrush.h"
#include "Binding/PropertyBinding.h"
#include "BrushBinding.generated.h"

UCLASS(MinimalAPI)
class UBrushBinding : public UPropertyBinding
{
	GENERATED_BODY()

public:

	UMG_API UBrushBinding();

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const override;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const override;

	UFUNCTION()
	UMG_API FSlateBrush GetValue() const;

private:
	enum class EConversion : uint8
	{
		None,
		Texture,
		//Material,
	};

	mutable TOptional<EConversion> bConversion;
};
