// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Binding/PropertyBinding.h"
#include "CheckedStateBinding.generated.h"

enum class ECheckBoxState : uint8;

UCLASS(MinimalAPI)
class UCheckedStateBinding : public UPropertyBinding
{
	GENERATED_BODY()

public:

	UMG_API UCheckedStateBinding();

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const override;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const override;

	UFUNCTION()
	UMG_API ECheckBoxState GetValue() const;

private:
	enum class EConversion : uint8
	{
		None,
		Bool
	};

	mutable TOptional<EConversion> bConversion;
};
