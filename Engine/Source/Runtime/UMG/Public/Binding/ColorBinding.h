// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Styling/SlateColor.h"
#include "Binding/PropertyBinding.h"
#include "ColorBinding.generated.h"

UCLASS(MinimalAPI)
class UColorBinding : public UPropertyBinding
{
	GENERATED_BODY()

public:

	UMG_API UColorBinding();

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const override;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const override;

	UMG_API virtual void Bind(FProperty* Property, FScriptDelegate* Delegate) override;

	UFUNCTION()
	UMG_API FSlateColor GetSlateValue() const;

	UFUNCTION()
	UMG_API FLinearColor GetLinearValue() const;

private:
	mutable TOptional<bool> bNeedsConversion;
};
