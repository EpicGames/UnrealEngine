// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "PropertyPathHelpers.h"
#include "UObject/ObjectMacros.h"

#include "DynamicPropertyPath.generated.h"

class FProperty;
class UObject;

/** */
USTRUCT()
struct FDynamicPropertyPath : public FCachedPropertyPath
{
	GENERATED_USTRUCT_BODY()

public:

	/** */
	UMG_API FDynamicPropertyPath();

	/** */
	UMG_API FDynamicPropertyPath(const FString& Path);

	/** */
	UMG_API FDynamicPropertyPath(const TArray<FString>& PropertyChain);

	/** Get the value represented by this property path */
	template<typename T>
	bool GetValue(UObject* InContainer, T& OutValue) const
	{
		FProperty* OutProperty;
		return GetValue<T>(InContainer, OutValue, OutProperty);
	}

	/** Get the value and the leaf property represented by this property path */
	template<typename T>
	bool GetValue(UObject* InContainer, T& OutValue, FProperty*& OutProperty) const
	{
		return PropertyPathHelpers::GetPropertyValue(InContainer, *this, OutValue, OutProperty);
	}
};
