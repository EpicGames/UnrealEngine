// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Binding/PropertyBinding.h"
#include "Int32Binding.generated.h"

UCLASS(MinimalAPI)
class UInt32Binding : public UPropertyBinding
{
	GENERATED_BODY()

public:

	UMG_API UInt32Binding();

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const override;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const override;

	UFUNCTION()
	UMG_API int32 GetValue() const;
};
