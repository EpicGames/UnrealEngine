// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "GenericPlatform/ICursor.h"
#include "Binding/PropertyBinding.h"
#include "MouseCursorBinding.generated.h"

UCLASS(MinimalAPI)
class UMouseCursorBinding : public UPropertyBinding
{
	GENERATED_BODY()

public:

	UMG_API UMouseCursorBinding();

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const override;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const override;

	UFUNCTION()
	UMG_API EMouseCursor::Type GetValue() const;
};
