// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SlateGlobals.h"
#include "Binding/DynamicPropertyPath.h"

#include "PropertyBinding.generated.h"

DECLARE_CYCLE_STAT_EXTERN(TEXT("UMG Binding"), STAT_UMGBinding, STATGROUP_Slate,);

UCLASS(MinimalAPI)
class UPropertyBinding : public UObject
{
	GENERATED_BODY()

public:
	UMG_API UPropertyBinding();

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const;

	UMG_API virtual void Bind(FProperty* Property, FScriptDelegate* Delegate);

public:
	/** The source object to use as the initial container to resolve the Source Property Path on. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UObject> SourceObject;

	/** The property path to trace to resolve this binding on the Source Object */
	UPROPERTY()
	FDynamicPropertyPath SourcePath;

	/** Used to determine if a binding already exists on the object and if this binding can be safely removed. */
	UPROPERTY()
	FName DestinationProperty;
};
