// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/StaticArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "WidgetStateBitfield.generated.h"

/** 
 * Expensive to create wrapper for binary and enumerated widget states that supports
 * FName-based interaction and dynamic states populated in an IWidgetStateNameMapper
 * May be used to represent either a target widget state combination, or an actual widgets state
 * 
 * All this does is cache the integer results of dynamic FName state lookup from an IWidgetStateNameMapper
 * So it's prefertable to create these once and reuse. In particular, if broadcasting current state
 * determine the correct bitfield to modify the existing state ahead of time. Save that bitfield and use it
 * when performing state-modifications. Likewise when performing state tests, try to construct the relevant
 * bitfield ahead of time & store that to later be compared against  
 */
USTRUCT()
struct FWidgetStateBitfield
{
	GENERATED_BODY()

public:

	/** Default, will convert to false as a bool */
	UMG_API FWidgetStateBitfield();

	/** Interprets name as binary state on */
	UMG_API FWidgetStateBitfield(const FName InStateName);

	/** Interprets name value pair as enum state */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API FWidgetStateBitfield(const FName InStateName, const uint8 InValue);

public:

	/** 
	 * Negation of binary and enum states, maintains enum state usage, but marks states as not allowed.
	 */
	UMG_API FWidgetStateBitfield operator~() const;

	/** 
	 * Intersection of binary and enum states, with union on enum state usage so enum state mismatches can fail
	 * Intersections will remove not allowed enum states, allowing for failure on bool conversion due to empty used states.
	 * Approximation of bitwise '&'. Non-commutative, will use LHS enum state set if RHS does not use enum. Additionally,
	 * if the RHS has binary states & the LHS does not meet those, all enum states will be marked unused to ensure failure
	 */
	UMG_API FWidgetStateBitfield Intersect(const FWidgetStateBitfield& Rhs) const;

	/** 
	 * Union of binary and enum states, allows for a single enum state to have multiple values in a bitfield.
	 * Unions will grow enum state set if allowance is same, otherwise the allowed set is maintained.
	 * Approximation of bitwise '|'. Commutative.
	 */
	UMG_API FWidgetStateBitfield Union(const FWidgetStateBitfield& Rhs) const;

public:

	/** True if any binary or enum state is set, and no empty used enum states exist */
	UMG_API operator bool() const;

	/** True if any binary state is set */
	UMG_API bool HasBinaryStates() const;

	/** True if any enum state is used, does not indicate if state is actually set. Use 'HasEmptyUsedEnumStates' for that. */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API bool HasEnumStates() const;

	/** True if any enum used enum states are empty, usually indicating a failed '&' test */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API bool HasEmptyUsedEnumStates() const;

public:

	/**
	 * Note: No need for HasNoFlags. Simply call !HasAnyFlags.
	 */

	/** True if any state flag is met */
	UMG_API bool HasAnyFlags(const FWidgetStateBitfield& InBitfield) const;

	/** True if all state flags are met */
	UMG_API bool HasAllFlags(const FWidgetStateBitfield& InBitfield) const;

	/** True if any binary state flag is met */
	UMG_API bool HasAnyBinaryFlags(const FWidgetStateBitfield& InBitfield) const;

	/** True if all binary state flags are met */
	UMG_API bool HasAllBinaryFlags(const FWidgetStateBitfield& InBitfield) const;

	/** True if any enum state flag is met */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API bool HasAnyEnumFlags(const FWidgetStateBitfield& InBitfield) const;

	/** True if all enum state flags are met */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API bool HasAllEnumFlags(const FWidgetStateBitfield& InBitfield) const;

public:

	/** Set state to given value, just calls assignment operator for you */
	UMG_API void SetState(const FWidgetStateBitfield& InBitfield);

	/** Negate both binary and enum states */
	UMG_API void NegateStates();

	/**
	 * Set binary state to given value
	 *
	 * @param BinaryStateIndex Index for Binary state to assign value to
	 * @param BinaryStateValue Value to assign for given Binary state
	 */
	UMG_API void SetBinaryState(uint8 BinaryStateIndex, bool BinaryStateValue);

	/**
	 * Set Binary state to given value, using a bitfield to indicate states that should be changed
	 *
	 * @param BinaryStateBitfield state bitfield, all marked states will be set with given value
	 * @param BinaryStateValue Value to assign for given Binary state
	 */
	UMG_API void SetBinaryState(const FWidgetStateBitfield& BinaryStateBitfield, bool BinaryStateValue);

	/** Set Binary state to given value, slow version w/ FName map lookup */
	UMG_API void SetBinaryStateSlow(FName BinaryStateName, bool BinaryStateValue);

	/** Negate only binary states */
	UMG_API void NegateBinaryStates();

	/** 
	 * Set enum state to given value, clearing all existing values for that state  
	 * 
	 * @param EnumStateIndex Index for enum state to assign value to
	 * @param EnumStateValue Value to assign for given enum state
	 */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void SetEnumState(uint8 EnumStateIndex, uint8 EnumStateValue);

	/**
	 * Set enum state to given value, clearing all existing values for states in use
	 *
	 * @param EnumStateBitfield state bitfield, all used states will copy-overwrite current enum state
	 */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void SetEnumState(const FWidgetStateBitfield& EnumStateBitfield);

	/** Set enum state to given value, clearing all existing values for that state, slow version w/ FName map lookup */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void SetEnumStateSlow(FName EnumStateName, uint8 EnumStateValue);

	/**
	 * Clear all existing values for given states & mark the states unused
	 *
	 * @param EnumStateBitfield state bitfield, all used states will be cleared
	 */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void ClearEnumState(const FWidgetStateBitfield& EnumStateBitfield);

	/**
	 * Clear all existing values for given state & mark the state unused
	 *
	 * @param EnumStateIndex Index for enum state to mark as unused
	 */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void ClearEnumState(uint8 EnumStateIndex);

	/** Clear all existing values for given state & mark the state unused, slow version w/ map lookup */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void ClearEnumState(FName EnumStateName);

	/** Negate only enum states, does not change usage, but marks existing values as not allowed */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void NegateEnumStates();

private:

	/** Binary states captured by this state field */
	uint64 BinaryStates = 0;
};
