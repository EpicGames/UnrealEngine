// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Binding/States/WidgetStateBitfield.h"

#include "WidgetStateRegistration.generated.h"

class UWidget;
class UWidgetStateSettings;

/**
 * Derive from to add a new widget binary state
 * 
 * Technically these can be created in BP, but for now we don't want to encourage
 * that workflow as it involves requring overrides for the virtuals which is technical.
 */
UCLASS(Transient, MinimalAPI)
class UWidgetBinaryStateRegistration : public UObject
{
	GENERATED_BODY()

public:
	UWidgetBinaryStateRegistration() = default;
	virtual ~UWidgetBinaryStateRegistration() = default;

	/** Called once during WidgetStateSettings initialization to get this widget state's name */
	UMG_API virtual FName GetStateName() const;

	/** Called on widget registration to correctly initialize widget state based on the current widget */
	UMG_API virtual bool GetRegisteredWidgetState(const UWidget* InWidget) const;

protected:
	friend UWidgetStateSettings;

	/** Called to give CDO chance to initialize any static state bitfields that might be declared for convenience */
	UMG_API virtual void InitializeStaticBitfields() const;
};

UCLASS(Transient, MinimalAPI)
class UWidgetHoveredStateRegistration : public UWidgetBinaryStateRegistration
{
	GENERATED_BODY()

public:

	/** Post-load initialized bit corresponding to this binary state */
	static UMG_API FWidgetStateBitfield Bit;

	static const inline FName StateName = FName("Hovered");

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual FName GetStateName() const override;
	UMG_API virtual bool GetRegisteredWidgetState(const UWidget* InWidget) const override;
	//~ End UWidgetBinaryStateRegistration Interface

protected:
	friend UWidgetStateSettings;

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual void InitializeStaticBitfields() const override;
	//~ End UWidgetBinaryStateRegistration Interface
};

UCLASS(Transient, MinimalAPI)
class UWidgetPressedStateRegistration : public UWidgetBinaryStateRegistration
{
	GENERATED_BODY()

public:

	/** Post-load initialized bit corresponding to this binary state */
	static UMG_API FWidgetStateBitfield Bit;

	static const inline FName StateName = FName("Pressed");

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual FName GetStateName() const override;
	UMG_API virtual bool GetRegisteredWidgetState(const UWidget* InWidget) const override;
	//~ End UWidgetBinaryStateRegistration Interface

protected:
	friend UWidgetStateSettings;

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual void InitializeStaticBitfields() const override;
	//~ End UWidgetBinaryStateRegistration Interface
};

UCLASS(Transient, MinimalAPI)
class UWidgetDisabledStateRegistration : public UWidgetBinaryStateRegistration
{
	GENERATED_BODY()

public:

	/** Post-load initialized bit corresponding to this binary state */
	static UMG_API FWidgetStateBitfield Bit;

	static const inline FName StateName = FName("Disabled");

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual FName GetStateName() const override;
	UMG_API virtual bool GetRegisteredWidgetState(const UWidget* InWidget) const override;
	//~ End UWidgetBinaryStateRegistration Interface

protected:
	friend UWidgetStateSettings;

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual void InitializeStaticBitfields() const override;
	//~ End UWidgetBinaryStateRegistration Interface
};

UCLASS(Transient, MinimalAPI)
class UWidgetSelectedStateRegistration : public UWidgetBinaryStateRegistration
{
	GENERATED_BODY()

public:

	/** Post-load initialized bit corresponding to this binary state */
	static UMG_API FWidgetStateBitfield Bit;

	static const inline FName StateName = FName("Selected");

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual FName GetStateName() const override;
	UMG_API virtual bool GetRegisteredWidgetState(const UWidget* InWidget) const override;
	//~ End UWidgetBinaryStateRegistration Interface

protected:
	friend UWidgetStateSettings;

	//~ Begin UWidgetBinaryStateRegistration Interface.
	UMG_API virtual void InitializeStaticBitfields() const override;
	//~ End UWidgetBinaryStateRegistration Interface
};

/**
 * Derive from to add a new Enum binary state
 */
UCLASS(Transient, MinimalAPI)
class UWidgetEnumStateRegistration : public UObject
{
	GENERATED_BODY()

public:
	UWidgetEnumStateRegistration() = default;
	virtual ~UWidgetEnumStateRegistration() = default;

	/** Called once during WidgetStateSettings initialization to get this widget state's name */
	UMG_API virtual FName GetStateName() const;

	/** Called on widget registration to determine if this widget uses the given state */
	UMG_API virtual bool GetRegisteredWidgetUsesState(const UWidget* InWidget) const;

	/** Called on widget registration to correctly initialize widget state based on the current widget */
	UMG_API virtual uint8 GetRegisteredWidgetState(const UWidget* InWidget) const;

protected:
	friend UWidgetStateSettings;

	/** Called to give CDO chance to initialize any static state bitfields that might be declared for convenience */
	UMG_API virtual void InitializeStaticBitfields() const;
};
