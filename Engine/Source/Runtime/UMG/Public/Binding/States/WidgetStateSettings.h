// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"

#include "WidgetStateSettings.generated.h"

class UWidget;

struct FWidgetStateBitfield;
class UWidgetBinaryStateRegistration;
class UWidgetEnumStateRegistration;

/**
 * Settings used to map widget states to indexes.
 * Does not perform any input validation, will crash if given invalid searches
 * 
 * Note: Currently doesn't really have any meaningful settings, used more as global singleton. May change.
 */
UCLASS(config = Game, defaultconfig, MinimalAPI)
class UWidgetStateSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
		
	static const UWidgetStateSettings* Get() 
	{ 
		return GetDefault<UWidgetStateSettings>();
	}

	//~ Begin UObject Interface.
	UMG_API virtual void PostInitProperties() override;
	//~ End UObject Interface

public:

	/** Obtain a list of all known States. */
	UMG_API void GetAllStateNames(TArray<FName>& OutStateNames) const;

	/** Obtain a list of all known Binary States. */
	UMG_API void GetBinaryStateNames(TArray<FName>& OutBinaryStateNames) const;

	/** Obtain a list of all known Enum States. */
	UMG_API void GetEnumStateNames(TArray<FName>& OutEnumStateNames) const;

	/** Get Index corresponding to this binary state, which may dynamically change - but not within a game session. */
	UMG_API uint8 GetBinaryStateIndex(const FName InBinaryStateName) const;

	/** Get Index corresponding to this enum state, which may dynamically change - but not within a game session. */
	UMG_API uint8 GetEnumStateIndex(const FName InEnumStateName) const;

	/** Get Name corresponding to this binary state index, which may dynamically change - but not within a game session. */
	UMG_API FName GetBinaryStateName(const uint8 InBinaryStateIndex) const;

	/** Get Name corresponding to this enum state index, which may dynamically change - but not within a game session. */
	UMG_API FName GetEnumStateName(const uint8 InEnumStateIndex) const;

	/** Gets bitfield of widget based on current state and given on registration state initializers */
	UMG_API FWidgetStateBitfield GetInitialRegistrationBitfield(const UWidget* InWidget) const;

private:
	/** Map of Binary State Names to index (In order of addition) */
	TMap<FName, uint8, TInlineSetAllocator<32>> BinaryStateMap;

	/** Map of Enum State Names to index (In order of addition) */
	TMap<FName, uint8, TInlineSetAllocator<8>> EnumStateMap;

	/** Ordered list of Binary Widget States */
	TArray<FName, TInlineAllocator<32>> BinaryStates;

	/** Ordered list of Enum Widget States */
	TArray<FName, TInlineAllocator<8>> EnumStates;

	/** Ordered list of Binary Widget States */
	TArray<TObjectPtr<UWidgetBinaryStateRegistration>, TInlineAllocator<32>> BinaryStateRegistrationCDOs;

	/** Ordered list of Enum Widget States */
	TArray<TObjectPtr<UWidgetEnumStateRegistration>, TInlineAllocator<8>> EnumStateRegistrationCDOs;
};
