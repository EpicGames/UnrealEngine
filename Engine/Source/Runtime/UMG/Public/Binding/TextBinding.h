// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Binding/PropertyBinding.h"
#include "TextBinding.generated.h"

UCLASS(MinimalAPI)
class UTextBinding : public UPropertyBinding
{
	GENERATED_BODY()

public:

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const override;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const override;

	UMG_API virtual void Bind(FProperty* Property, FScriptDelegate* Delegate) override;

	UFUNCTION()
	UMG_API FText GetTextValue() const;

	UFUNCTION()
	UMG_API FString GetStringValue() const;

private:

	enum class EConversion : uint8
	{
		None,
		String,
		Words,
		Integer,
		Float
	};

	mutable TOptional<EConversion> NeedsConversion;
};
