// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Binding/PropertyBinding.h"

#include "VisibilityBinding.generated.h"

enum class ESlateVisibility : uint8;

UCLASS(MinimalAPI)
class UVisibilityBinding : public UPropertyBinding
{
	GENERATED_BODY()

public:

	UMG_API UVisibilityBinding();

	UMG_API virtual bool IsSupportedSource(FProperty* Property) const override;
	UMG_API virtual bool IsSupportedDestination(FProperty* Property) const override;

	UFUNCTION()
	UMG_API ESlateVisibility GetValue() const;
};
