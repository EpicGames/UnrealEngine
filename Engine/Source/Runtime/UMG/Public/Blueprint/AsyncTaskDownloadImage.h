// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Interfaces/IHttpRequest.h"
#include "Kismet/BlueprintAsyncActionBase.h"

#include "AsyncTaskDownloadImage.generated.h"

class UTexture2DDynamic;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FDownloadImageDelegate, UTexture2DDynamic*, Texture);

UCLASS(MinimalAPI)
class UAsyncTaskDownloadImage : public UBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION(BlueprintCallable, meta=( BlueprintInternalUseOnly="true" ))
	static UMG_API UAsyncTaskDownloadImage* DownloadImage(FString URL);

public:

	UPROPERTY(BlueprintAssignable)
	FDownloadImageDelegate OnSuccess;

	UPROPERTY(BlueprintAssignable)
	FDownloadImageDelegate OnFail;

public:

	UMG_API void Start(FString URL);

private:

	/** Handles image requests coming from the web */
	void HandleImageRequest(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);
};
