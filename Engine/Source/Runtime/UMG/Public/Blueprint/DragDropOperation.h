// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Input/Events.h"
#include "Widgets/Views/STableRow.h"
#include "DragDropOperation.generated.h"

class UDragDropOperation;


/**
 * A drop zone enum that can be passed through blueprint implementable events and sent through multicast delegates.
 */
UENUM(BlueprintType)
enum class EUMGItemDropZone : uint8
{
	AboveItem,
	OntoItem,
	BelowItem,
	None
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnDragDropMulticast, UDragDropOperation*, Operation);

/**
 * Controls where the drag widget visual will appear when dragged relative to the pointer performing
 * the drag operation.
 */
UENUM(BlueprintType)
enum class EDragPivot : uint8
{
	/**  */
	MouseDown,
	TopLeft,
	TopCenter,
	TopRight,
	CenterLeft,
	CenterCenter,
	CenterRight,
	BottomLeft,
	BottomCenter,
	BottomRight
};

/**
 * This class is the base drag drop operation for UMG, extend it to add additional data and add new functionality.
 */
UCLASS(BlueprintType, Blueprintable, meta=( DontUseGenericSpawnObject="True" ), MinimalAPI)
class UDragDropOperation : public UObject
{
	GENERATED_UCLASS_BODY()
	
public:

	static EUMGItemDropZone ConvertSlateDropZoneToUMG(TOptional<EItemDropZone> DropZone);

	/** A simple string tag you can optionally use to provide extra metadata about the operation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Drag and Drop", meta=( ExposeOnSpawn="true" ))
	FString Tag;
	
	/**
	 * The payload of the drag operation.  This can be any UObject that you want to pass along as dragged data.  If you 
	 * were building an inventory screen this would be the UObject representing the item being moved to another slot.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Drag and Drop", meta=( ExposeOnSpawn="true" ))
	TObjectPtr<UObject> Payload;
	
	/**
	 * The Drag Visual is the widget to display when dragging the item.  Normally people create a new widget to represent the 
	 * temporary drag.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Drag and Drop", meta=( ExposeOnSpawn="true", DisplayName="Drag Visual" ))
	TObjectPtr<class UWidget> DefaultDragVisual;

	/**
	 * Controls where the drag widget visual will appear when dragged relative to the pointer performing
	 * the drag operation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Drag and Drop", meta=( ExposeOnSpawn="true" ))
	EDragPivot Pivot;

	/** A percentage offset (-1..+1) from the Pivot location, the percentage is of the desired size of the dragged visual. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Drag and Drop", AdvancedDisplay, meta=( ExposeOnSpawn="true" ))
	FVector2D Offset;

	/**  */
	UPROPERTY(BlueprintAssignable)
	FOnDragDropMulticast OnDrop;

	/**  */
	UPROPERTY(BlueprintAssignable)
	FOnDragDropMulticast OnDragCancelled;

	/**  */
	UPROPERTY(BlueprintAssignable)
	FOnDragDropMulticast OnDragged;

	/**  */
	UFUNCTION(BlueprintNativeEvent, Category="Drag and Drop")
	UMG_API void Drop(const FPointerEvent& PointerEvent);

	/**  */
	UFUNCTION(BlueprintNativeEvent, Category="Drag and Drop")
	UMG_API void DragCancelled(const FPointerEvent& PointerEvent);

	/**  */
	UFUNCTION(BlueprintNativeEvent, Category="Drag and Drop")
	UMG_API void Dragged(const FPointerEvent& PointerEvent);
};
