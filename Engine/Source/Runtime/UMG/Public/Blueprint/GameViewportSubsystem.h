// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Layout/Margin.h"
#include "UObject/ObjectKey.h"
#include "Widgets/Layout/Anchors.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "GameViewportSubsystem.generated.h"

class ULevel;
class ULocalPlayer;
class UWidget;
class UWorld;
class SConstraintCanvas;

/**
 * The default value fills the entire screen / player region.
 */
USTRUCT(BlueprintType)
struct FGameViewportWidgetSlot
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "User Interface")
	FAnchors Anchors = FAnchors(0.f, 0.f, 1.f, 1.f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "User Interface")
	FMargin Offsets;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "User Interface")
	FVector2D Alignment = FVector2D(0.f, 0.f);

	/** The higher the number, the more on top this widget will be. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "User Interface")
	int32 ZOrder = 0;

	/**
	 * Remove the widget when the Widget's World is removed.
	 * @note The Widget is added to the GameViewportClient of the Widget's World. The GameViewportClient can migrate from World to World. The widget can stay visible if the owner of the widget also migrate.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "User Interface")
	bool bAutoRemoveOnWorldRemoved = true;
};


/**
 * 
 */
UCLASS(MinimalAPI)
class UGameViewportSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin Subsystem
	UMG_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UMG_API virtual void Deinitialize() override;
	//~ End Subsystem

	static UMG_API UGameViewportSubsystem* Get();

	UE_DEPRECATED(5.8, "Use the Get function which takes no arguments instead.")
	static UMG_API UGameViewportSubsystem* Get(UWorld* World);
	
public:
	/* @return true if the widget was added to the viewport using AddWidget or AddWidgetForPlayer. */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category = "User Interface")
	UMG_API bool IsWidgetAdded(const UWidget* Widget) const;

	/** Adds it to the game's viewport. */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface")
	UMG_API bool AddWidget(UWidget* Widget, FGameViewportWidgetSlot Slot);

	/**
	 * Adds the widget to the game's viewport in the section dedicated to the player. This is valuable in a split screen
	 * game where you need to only show a widget over a player's portion of the viewport.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface")
	UMG_API bool AddWidgetForPlayer(UWidget* Widget, ULocalPlayer* Player, FGameViewportWidgetSlot Slot);

	/** Removes the widget from the viewport. */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface")
	UMG_API void RemoveWidget(UWidget* Widget);

	/* The slot info from previously added widget or info that is store for later. */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface")
	UMG_API FGameViewportWidgetSlot GetWidgetSlot(const UWidget* Widget) const;

	/* Update the slot info of a previously added widget or Store the slot info for later use. */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface")
	UMG_API void SetWidgetSlot(UWidget* Widget, FGameViewportWidgetSlot Slot);

	/** */
	DECLARE_EVENT_TwoParams(UGameViewportSubsystem, FWidgetAddedEvent, UWidget*, ULocalPlayer*);
	FWidgetAddedEvent OnWidgetAdded;

	/** */
	DECLARE_EVENT_OneParam(UGameViewportSubsystem, FWidgetRemovedEvent, UWidget*);
	FWidgetRemovedEvent OnWidgetRemoved;

public:
	/**
	 * Helper function to set the position in the viewport for the Slot.
	 * @param Position The 2D position to set the widget to in the viewport.
	 * @param bRemoveDPIScale If you've already calculated inverse DPI, set this to false.
	 * Otherwise inverse DPI is applied to the position so that when the location is scaled
	 * by DPI, it ends up in the expected position.
	 */
	UFUNCTION(BlueprintCallable, Category = "GameViewportWidgetSlot")
	static UMG_API FGameViewportWidgetSlot SetWidgetSlotPosition(FGameViewportWidgetSlot Slot, const UWidget* Widget, FVector2D Position, bool bRemoveDPIScale);

	/**
	 * Helper function to set the desired size in the viewport for the Slot.
	 */
	UFUNCTION(BlueprintCallable, Category = "GameViewportWidgetSlot")
	static UMG_API FGameViewportWidgetSlot SetWidgetSlotDesiredSize(FGameViewportWidgetSlot Slot, FVector2D Size);

private:
	struct FSlotInfo
	{
		FGameViewportWidgetSlot Slot;
		TWeakObjectPtr<ULocalPlayer> LocalPlayer;
		TWeakPtr<SConstraintCanvas> FullScreenWidget;
		SConstraintCanvas::FSlot* FullScreenWidgetSlot = nullptr;
	};

private:
	bool AddToScreen(UWidget* Widget, ULocalPlayer* Player, FGameViewportWidgetSlot& Slot);
	void RemoveWidgetInternal(UWidget* Widget, const TWeakPtr<SConstraintCanvas>& FullScreenWidget, const TWeakObjectPtr<ULocalPlayer>& LocalPlayer);
	void HandleWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResoures);
	void HandleRemoveWorld(UWorld* InWorld);
	void HandleLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

	FMargin GetFullScreenOffsetForWidget(UWidget* Widget) const;
	TPair<FMargin, bool> GetOffsetAttribute(UWidget* Widget, const FGameViewportWidgetSlot& Slot) const;

private:
	using FViewportWidgetList = TMap<TObjectKey<UWidget>, FSlotInfo>;
	FViewportWidgetList ViewportWidgets;
};
