// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "IUserListEntry.generated.h"

class UUserWidget;
class UListViewBase;
class IObjectTableRow;
enum class EItemDropZone;
enum class EUMGItemDropZone : uint8;
/**
 * Required interface for any UUserWidget class to be usable as entry widget in a UListViewBase
 * Provides access to getters and events for changes to the status of the widget as an entry that represents an item in a list.
 * @see UListViewBase @see ITypedUMGListView
 *
 * Note: To be usable as an entry for UListView, UTileView, or UTreeView, implement IUserObjectListEntry instead
 */
UINTERFACE(BlueprintType, MinimalAPI)
class UUserListEntry : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IUserListEntry : public IInterface
{
	GENERATED_IINTERFACE_BODY()
public:
	/** Returns true if the item represented by this entry is currently selected in the owning list view. */
	UMG_API bool IsListItemSelected() const;
	
	/** Returns true if the item represented by this entry is currently expanded and showing its children. Tree view entries only. */
	UMG_API bool IsListItemExpanded() const;
	
	/** Returns the list view that contains this entry. */
	UMG_API UListViewBase* GetOwningListView() const;

	/**
	 * Advanced native-only option for specific rows to preclude themselves from any kind of selection.
	 * Intended primarily for category separators and the like.
	 * Note that this is only relevant when the row is in a list that allows selection in the first place.
	 */
	virtual bool IsListItemSelectable() const { return true; }

public:
	/** Functionality largely for "internal" use/plumbing - see SObjectTableRow's usage. You shouldn't have any cause to call these directly. */
	static UMG_API void ReleaseEntry(UUserWidget& ListEntryWidget);
	static UMG_API void UpdateItemSelection(UUserWidget& ListEntryWidget, bool bIsSelected);
	static UMG_API void UpdateItemExpansion(UUserWidget& ListEntryWidget, bool bIsExpanded);
	static UMG_API void UpdateEntryDropIndicator(UUserWidget& ListEntryWidget, TOptional<EItemDropZone> DropZone);
	static UMG_API void UpdateEntryDragOverState(UUserWidget& ListEntryWidget, bool bIsDraggingOver);
	static UMG_API void EndEntryDropOperation(UUserWidget& ListEntryWidget, bool bSuccess);
	static UMG_API void HandleEntryDropped(UUserWidget& ListEntryWidget, UDragDropOperation* Operation);
	static UMG_API void HandleEntryDragged(UUserWidget& ListEntryWidget, UDragDropOperation* Operation);

protected:
	/** 
	 * These follow the same pattern as the NativeOn[X] methods in UUserWidget - super calls are expected in order to route the event to BP.
	 * See the BP events below for descriptions.
	 */
	UMG_API virtual void NativeOnItemSelectionChanged(bool bIsSelected);
	UMG_API virtual void NativeOnItemExpansionChanged(bool bIsExpanded);
	UMG_API virtual void NativeOnEntryReleased();

	/** Called when the selection state of the item represented by this entry changes. */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Item Selection Changed"))
	UMG_API void BP_OnItemSelectionChanged(bool bIsSelected);

	/** Called when the expansion state of the item represented by this entry changes. Tree view entries only. */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Item Expansion Changed"))
	UMG_API void BP_OnItemExpansionChanged(bool bIsExpanded);

	/** Called when this entry is released from the owning table and no longer represents any list item */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Entry Released"))
	UMG_API void BP_OnEntryReleased();

	/** Called when the drop zone indicator is changed for this entry */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Update Entry Drop Indicator"))
	UMG_API void BP_OnUpdateEntryDropIndicator(EUMGItemDropZone DropZone);

	/** Called when the the entry is dragged over */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Entry Drag Over Changed"))
	UMG_API void BP_OnEntryDragOverChanged(bool bIsDraggingOver);

	/** Called when this entry accepts a drag and drop operation */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Entry Accept Drop"))
	UMG_API void BP_OnEndEntryDropOperation(bool bSuccess);

	/** Called when this entry is dropped */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Entry Dropped"))
	UMG_API void BP_OnEntryDropped(UDragDropOperation* Operation);

	/** Called when this entry is dragged */
	UFUNCTION(BlueprintImplementableEvent, Category = UserListEntry, meta = (DisplayName = "On Entry Dragged"))
	UMG_API void BP_OnEntryDragged(UDragDropOperation* Operation);
	
};

/** Static library to supply "for free" functionality to widgets that implement IUserListEntry */
UCLASS(MinimalAPI)
class UUserListEntryLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** 
	 * Returns true if the item represented by this entry is currently selected in the owning list view. 
	 * @param UserListEntry Note: Visually not transmitted, but this defaults to "self". No need to hook up if calling internally.
	 */
	UFUNCTION(BlueprintPure, Category = UserListEntry, meta = (DefaultToSelf = UserListEntry))
	static UMG_API bool IsListItemSelected(TScriptInterface<IUserListEntry> UserListEntry);

	/** 
	 * Returns true if the item represented by this entry is currently expanded and showing its children. Tree view entries only.
	 * @param UserListEntry Note: Visually not transmitted, but this defaults to "self". No need to hook up if calling internally.
	 */
	UFUNCTION(BlueprintPure, Category = UserListEntry, meta = (DefaultToSelf = UserListEntry))
	static UMG_API bool IsListItemExpanded(TScriptInterface<IUserListEntry> UserListEntry);

	/** 
	 * Returns the list view that contains this entry.
	 * @param UserListEntry Note: Visually not transmitted, but this defaults to "self". No need to hook up if calling internally.
	 */
	UFUNCTION(BlueprintPure, Category = UserListEntry, meta = (DefaultToSelf = UserListEntry))
	static UMG_API UListViewBase* GetOwningListView(TScriptInterface<IUserListEntry> UserListEntry);
};
