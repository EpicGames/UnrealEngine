// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IUserListEntry.h"
#include "IUserObjectListEntry.generated.h"

/**
 * Required interface for any UUserWidget class to be usable as entry widget in a stock UMG list view - ListView, TileView, and TreeView
 * Provides a change event and getter for the object item the entry is assigned to represent by the owning list view (in addition to functionality from IUserListEntry)
 */
UINTERFACE(MinimalAPI)
class UUserObjectListEntry : public UUserListEntry
{
	GENERATED_UINTERFACE_BODY()
};

class IUserObjectListEntry : public IUserListEntry
{
	GENERATED_IINTERFACE_BODY()

public:
	/** Returns the UObject item in the owning UListView that this entry currently represents */
	template <typename ItemObjectT = UObject>
	ItemObjectT* GetListItem() const
	{
		static_assert(TIsDerivedFrom<ItemObjectT, UObject>::IsDerived, "Items represented by an ObjectListEntry are always expected to be UObjects.");
		return Cast<ItemObjectT>(GetListItemObjectInternal());
	}

protected:
	/** Follows the same pattern as the NativeOn[X] methods in UUserWidget - super calls are expected in order to route the event to BP. */
	UMG_API virtual void NativeOnListItemObjectSet(UObject* ListItemObject);
	
	/** Called when this entry is assigned a new item object to represent by the owning list view */
	UFUNCTION(BlueprintImplementableEvent, Category = ObjectListEntry)
	UMG_API void OnListItemObjectSet(UObject* ListItemObject);

private:
	UMG_API UObject* GetListItemObjectInternal() const;
	
	template <typename> friend class SObjectTableRow;
	static UMG_API void SetListItemObject(UUserWidget& ListEntryWidget, UObject* ListItemObject);
};

/** Static library to supply "for free" functionality to widgets that implement IUserListEntry */
UCLASS(MinimalAPI)
class UUserObjectListEntryLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** 
	 * Returns the item in the owning list view that this entry is currently assigned to represent. 
	 * @param UserObjectListEntry Note: Visually not transmitted, but this defaults to "self". No need to hook up if calling internally.
	 */
	UFUNCTION(BlueprintPure, Category = UserObjectListEntry, meta = (DefaultToSelf = UserObjectListEntry))
	static UMG_API UObject* GetListItemObject(TScriptInterface<IUserObjectListEntry> UserObjectListEntry);

	/** Returns the index of the list item in the owning UListView */
	UFUNCTION(BlueprintPure, Category = UserObjectListEntry, meta = (DefaultToSelf = UserObjectListEntry))
	static UMG_API int32 GetListItemIndex(TScriptInterface<IUserObjectListEntry> UserObjectListEntry);

	/** Returns true if the widget list entry is the first item in the list view */
	UFUNCTION(BlueprintPure, Category = UserObjectListEntry, meta = (DefaultToSelf = UserObjectListEntry))
	static UMG_API bool IsFirstWidget(TScriptInterface<IUserObjectListEntry> UserObjectListEntry);

	/** Returns true if the widget list entry is the last item in the list view */
	UFUNCTION(BlueprintPure, Category = UserObjectListEntry, meta = (DefaultToSelf = UserObjectListEntry))
	static UMG_API bool IsLastWidget(TScriptInterface<IUserObjectListEntry> UserObjectListEntry);
};
