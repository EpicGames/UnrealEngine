// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Script.h"
#include "Layout/Geometry.h"
#include "Styling/SlateBrush.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SlateBlueprintLibrary.generated.h"

UCLASS(meta=(ScriptName="SlateLibrary"), MinimalAPI)
class USlateBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 *
	 * @return true if the provided location in absolute coordinates is within the bounds of this geometry.
	 */
	 UFUNCTION(BlueprintPure, Category="User Interface|Geometry")
	static UMG_API bool IsUnderLocation(const FGeometry& Geometry, const FVector2D& AbsoluteCoordinate);

	/**
	 * Transforms absolute coordinates into local coordinates
	 *
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 *
	 * @return Transforms AbsoluteCoordinate into the local space of this Geometry.
	 */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry", meta = (DisplayName = "Absolute to Local (Coordinates)", Keywords = "transform"))
	static UMG_API FVector2D AbsoluteToLocal(const FGeometry& Geometry, FVector2D AbsoluteCoordinate);

	/**
	 * Transforms local coordinates into absolute coordinates
	 *
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 *
	 * @return  Absolute coordinates
	 */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry", meta = (DisplayName = "Local to Absolute (Coordinates)", Keywords = "transform"))
	static UMG_API FVector2D LocalToAbsolute(const FGeometry& Geometry, FVector2D LocalCoordinate);

	/** Returns the local top/left of the geometry in local space. */
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry")
	static UMG_API FVector2D GetLocalTopLeft(const FGeometry& Geometry);

	/** Returns the size of the geometry in local space. */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry")
	static UMG_API FVector2D GetLocalSize(const FGeometry& Geometry);

	/** Returns the size of the geometry in absolute space. */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry")
	static UMG_API FVector2D GetAbsoluteSize(const FGeometry& Geometry);

	/** Transforms a scalar from absolute space to local space  */
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DisplayName = "Absolute to Local (Scalar)", Keywords = "transform"))
	static UMG_API float Scalar_AbsoluteToLocal(const FGeometry& Geometry, float AbsoluteScalar);

	/** Transforms a scalar from local space to absolute space  */
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DisplayName = "Local to Absolute (Scalar)", Keywords = "transform"))
	static UMG_API float Scalar_LocalToAbsolute(const FGeometry& Geometry, float LocalScalar);

	/** Transforms a vector from absolute space to local space  */
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DisplayName = "Absolute to Local (Vector)", Keywords = "transform"))
	static UMG_API FVector2D Vector_AbsoluteToLocal(const FGeometry& Geometry, FVector2D AbsoluteVector);

	/** Transforms a vector from local space to absolute space  */
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DisplayName = "Local to Absolute (Vector)", Keywords = "transform"))
	static UMG_API FVector2D Vector_LocalToAbsolute(const FGeometry& Geometry, FVector2D LocalVector);

	/**  */
	UE_DEPRECATED(5.6, "TransformScalarAbsoluteToLocal returns inverted results and has been replaced by Scalar_LocalToAbsolute")
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DeprecatedFunction, DeprecationMessage = "Returns inverted results. Replace with 'Local to Absolute (Scalar)'"))
	static UMG_API float TransformScalarAbsoluteToLocal(const FGeometry& Geometry, float AbsoluteScalar);

	/**  */
	UE_DEPRECATED(5.6, "TransformScalarLocalToAbsolute returns inverted results and has been replaced by Scalar_AbsoluteToLocal")
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DeprecatedFunction, DeprecationMessage = "Returns inverted results. Replace with 'Absolute To Local (Scalar)'"))
	static UMG_API float TransformScalarLocalToAbsolute(const FGeometry& Geometry, float LocalScalar);

	/**  */
	UE_DEPRECATED(5.6, "TransformVectorAbsoluteToLocal returns inverted results and has been replaced by Vector_LocalToAbsolute")
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DeprecatedFunction, DeprecationMessage = "Returns inverted results. Replace with 'Local to Absolute (Vector)'"))
	static UMG_API FVector2D TransformVectorAbsoluteToLocal(const FGeometry& Geometry, FVector2D AbsoluteVector);

	/**  */
	UE_DEPRECATED(5.6, "TransformVectorLocalToAbsolute returns inverted results and has been replaced by Vector_AbsoluteToLocal")
	UFUNCTION(BlueprintPure, Category = "User Interface|Geometry", meta = (DeprecatedFunction, DeprecationMessage = "Returns inverted results. Replace with 'Absolute To Local (Vector)'"))
	static UMG_API FVector2D TransformVectorLocalToAbsolute(const FGeometry& Geometry, FVector2D LocalVector);

	/** Returns whether brushes A and B are identical. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (SlateBrush)", CompactNodeTitle = "=="), Category = "SlateBrush")
	static UMG_API bool EqualEqual_SlateBrush(const FSlateBrush& A, const FSlateBrush& B);

	/**
	 * Translates local coordinate of the geometry provided into local viewport coordinates.
	 *
	 * @param PixelPosition The position in the game's viewport, usable for line traces and 
	 * other uses where you need a coordinate in the space of viewport resolution units.
	 * @param ViewportPosition The position in the space of other widgets in the viewport.  Like if you wanted
	 * to add another widget to the viewport at the same position in viewport space as this location, this is
	 * what you would use.
	 */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry", meta=( WorldContext="WorldContextObject" ))
	static UMG_API void LocalToViewport(const UObject* WorldContextObject, const FGeometry& Geometry, FVector2D LocalCoordinate, FVector2D& PixelPosition, FVector2D& ViewportPosition);

	/**
	 * Translates absolute coordinate in desktop space of the geometry provided into local viewport coordinates.
	 *
	 * @param PixelPosition The position in the game's viewport, usable for line traces and
	 * other uses where you need a coordinate in the space of viewport resolution units.
	 * @param ViewportPosition The position in the space of other widgets in the viewport.  Like if you wanted
	 * to add another widget to the viewport at the same position in viewport space as this location, this is
	 * what you would use.
	 */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry", meta=( WorldContext="WorldContextObject" ))
	static UMG_API void AbsoluteToViewport(const UObject* WorldContextObject, FVector2D AbsoluteDesktopCoordinate, FVector2D& PixelPosition, FVector2D& ViewportPosition);

	/**
	 * Translates a screen position in pixels into the local space of a widget with the given geometry. 
	 * If bIncludeWindowPosition is true, then this method will also remove the game window's position (useful when in windowed mode).
	 */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry", meta=( WorldContext="WorldContextObject", DisplayName="ScreenToLocal" ))
	static UMG_API void ScreenToWidgetLocal(const UObject* WorldContextObject, const FGeometry& Geometry, FVector2D ScreenPosition, FVector2D& LocalCoordinate, bool bIncludeWindowPosition = false);

	/**
	 * Translates a screen position in pixels into absolute application coordinates.
	 * If bIncludeWindowPosition is true, then this method will also remove the game window's position (useful when in windowed mode).
	 */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry", meta=( WorldContext="WorldContextObject", DisplayName="ScreenToAbsolute" ))
	static UMG_API void ScreenToWidgetAbsolute(const UObject* WorldContextObject, FVector2D ScreenPosition, FVector2D& AbsoluteCoordinate, bool bIncludeWindowPosition = false);

	/**
	 * Translates a screen position in pixels into the local space of the viewport widget.
	 */
	UFUNCTION(BlueprintPure, Category="User Interface|Geometry", meta=( WorldContext="WorldContextObject" ))
	static UMG_API void ScreenToViewport(const UObject* WorldContextObject, FVector2D ScreenPosition, FVector2D& ViewportPosition);
};
