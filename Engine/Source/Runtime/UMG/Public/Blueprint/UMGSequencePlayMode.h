// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "UMGSequencePlayMode.generated.h"

class UUserWidget;

/** Describes playback modes for UMG sequences. */
UENUM(BlueprintType)
namespace EUMGSequencePlayMode
{
	enum Type : int
	{
		/** Animation plays and loops from the beginning to the end. */
		Forward,
		/** Animation plays and loops from the end to the beginning. */
		Reverse,
		/** Animation plays from the beginning to the end and then from the end to the beginning. */
		PingPong,
	};
}

