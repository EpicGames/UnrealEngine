// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UMGSequencePlayMode.h"
#include "Blueprint/WidgetChild.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectSaveContext.h"
#include "Styling/SlateColor.h"
#include "Layout/Geometry.h"
#include "Input/CursorReply.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Layout/Margin.h"
#include "Components/SlateWrapperTypes.h"
#include "Components/Widget.h"
#include "Components/NamedSlotInterface.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "Engine/LocalPlayer.h"
#include "Widgets/Layout/Anchors.h"
#include "Logging/MessageLog.h"
#include "Stats/Stats.h"
#include "EngineStats.h"
#include "SlateGlobals.h"
#include "Animation/WidgetAnimationEvents.h"
#include "Animation/WidgetAnimationHandle.h"
#include "Animation/WidgetAnimationState.h"
#include "UserWidget.generated.h"

class Error;
class FSlateWindowElementList;
class UDragDropOperation;
class UTexture2D;
class UUMGSequencePlayer;
class UUMGSequenceTickManager;
class UWidgetAnimation;
class UWidgetBlueprintGeneratedClass;
class UWidgetTree;
class UNamedSlot;
class UUserWidget;
class UUserWidgetExtension;

/** Describes overall action driving this animation transition. */
enum class EQueuedWidgetAnimationMode : uint8
{
	/** Animation plays with given params. */
	Play,
	/** Animation plays with given params to given point. */
	PlayTo,
	/** Animation plays from current position forward. */
	Forward,
	/** Animation plays from current position reverse. */
	Reverse,
	/** Animation stops playing. */
	Stop,
	/** Animation stops playing. */
	Pause,
	/** Default state, should not be used. */
	None,
};

/** Describes whether the widget needs a legacy UMG sequence player to be created. */
enum class ERequiresLegacyPlayer : uint8
{
	/** Will create a legacy player if the widget class has native C++ children. */
	AutoDetect,
	/** Force creating a legacy player. */
	Yes,
	/** Declares that no legacy player is required. */
	No
};

/**
 * Struct that maintains state of currently queued animation transtions to be evaluated next frame.
 */
USTRUCT()
struct FQueuedWidgetAnimationTransition
{
	GENERATED_BODY()

	/** Animation with a queued transition */
	UPROPERTY(Transient)
	TObjectPtr<UWidgetAnimation> WidgetAnimation;

	/** Overall action driving this animation transition */
	EQueuedWidgetAnimationMode TransitionMode;

	/** The time in the animation from which to start playing, relative to the start position. For looped animations, this will only affect the first playback of the animation */
	TOptional<float> StartAtTime;

	/** The absolute time in the animation where to stop, this is only considered in the last loop. */
	TOptional<float> EndAtTime;

	/** The number of times to loop this animation (0 to loop indefinitely) */
	TOptional<int32> NumLoopsToPlay;

	/** Specifies the playback mode (Forward, Reverse) */
	TOptional<EUMGSequencePlayMode::Type> PlayMode;

	/** The speed at which the animation should play */
	TOptional<float> PlaybackSpeed;

	/** Restores widgets to their pre-animated state when the animation stops */
	TOptional<bool> bRestoreState;

	FQueuedWidgetAnimationTransition()
		: WidgetAnimation(nullptr)
		, TransitionMode(EQueuedWidgetAnimationMode::None)
	{}
};

/** Determines what strategy we use to determine when and if the widget ticks. */
UENUM()
enum class EWidgetTickFrequency : uint8
{
	/** This widget never ticks */
	Never = 0,

	/** 
	 * This widget will tick if a blueprint tick function is implemented, any latent actions are found or animations need to play
	 * If the widget inherits from something other than UserWidget it will also tick so that native C++ or inherited ticks function
	 * To disable native ticking use add the class metadata flag "DisableNativeTick".  I.E: meta=(DisableNativeTick) 
	 */
	Auto,
};

/** Different animation events. */
UENUM(BlueprintType)
enum class EWidgetAnimationEvent : uint8
{
	Started,
	Finished
};

/** Used to manage different animation event bindings that users want callbacks on. */
USTRUCT()
struct FAnimationEventBinding
{
	GENERATED_BODY()

public:

	FAnimationEventBinding()
		: Animation(nullptr)
		, Delegate()
		, AnimationEvent(EWidgetAnimationEvent::Started)
		, UserTag(NAME_None)
	{
	}

	/** The animation to look for. */
	UPROPERTY()
	TObjectPtr<UWidgetAnimation> Animation;

	/** The callback. */
	UPROPERTY()
	FWidgetAnimationDynamicEvent Delegate;

	/** The type of animation event. */
	UPROPERTY()
	EWidgetAnimationEvent AnimationEvent;

	/** A user tag used to only get callbacks for specific runs of the animation. */
	UPROPERTY()
	FName UserTag;
};


/**
 * The state passed into OnPaint that we can expose as a single painting structure to blueprints to
 * allow script code to override OnPaint behavior.
 */
USTRUCT(BlueprintType)
struct FPaintContext
{
	GENERATED_USTRUCT_BODY()

public:

	/** Don't ever use this constructor.  Needed for code generation. */
	UMG_API FPaintContext();

	FPaintContext(const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& InOutDrawElements, const int32 InLayerId, const FWidgetStyle& InWidgetStyle, const bool bInParentEnabled)
		: AllottedGeometry(InAllottedGeometry)
		, MyCullingRect(InMyCullingRect)
		, OutDrawElements(InOutDrawElements)
		, LayerId(InLayerId)
		, WidgetStyle(InWidgetStyle)
		, bParentEnabled(bInParentEnabled)
		, MaxLayer(InLayerId)
	{
	}

	/** We override the assignment operator to allow generated code to compile with the const ref member. */
	void operator=( const FPaintContext& Other )
	{
		FPaintContext* Ptr = this;
		Ptr->~FPaintContext();
		new(Ptr) FPaintContext(Other.AllottedGeometry, Other.MyCullingRect, Other.OutDrawElements, Other.LayerId, Other.WidgetStyle, Other.bParentEnabled);
		Ptr->MaxLayer = Other.MaxLayer;
	}

public:

	const FGeometry& AllottedGeometry;
	const FSlateRect& MyCullingRect;
	FSlateWindowElementList& OutDrawElements;
	int32 LayerId;
	const FWidgetStyle& WidgetStyle;
	bool bParentEnabled;

	int32 MaxLayer;
};

USTRUCT()
struct FNamedSlotBinding
{
	GENERATED_USTRUCT_BODY()

public:

	FNamedSlotBinding()
		: Name(NAME_None)
		, Content(nullptr)
	{ }

	UPROPERTY()
	FName Name;

#if WITH_EDITORONLY_DATA
	// GUID of the NamedSlot is used as a secondary identifier to find a binding in case the name of NamedSlot has changed.
	UPROPERTY()
	FGuid Guid;
#endif

	UPROPERTY(Instanced)
	TObjectPtr<UWidget> Content;
};

#if WITH_EDITORONLY_DATA

UENUM()
enum class EDesignPreviewSizeMode : uint8
{
	FillScreen,
	Custom,
	CustomOnScreen,
	Desired,
	DesiredOnScreen,
};

#endif

UCLASS()
class UUserWidgetFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "UMG", meta = (BlueprintAutocast, DeprecatedFunction, DeprecatedMessage="UMGSequencePlayer is deprecated, please use FWidgetAnimationState"))
	static UUMGSequencePlayer* Conv_UMGSequencePlayer(const FWidgetAnimationHandle& WidgetAnimationHandle);

	UFUNCTION(BlueprintPure, Category = "UMG")
	static UUserWidget* GetOuterUserWidget(const UWidget* Widget);
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnConstructEvent);

DECLARE_DYNAMIC_DELEGATE( FOnInputAction );

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnVisibilityChangedEvent, ESlateVisibility, InVisibility);

/**
 * A widget that enables UI extensibility through WidgetBlueprint.
 */
UCLASS(Abstract, editinlinenew, BlueprintType, Blueprintable, meta=( DontUseGenericSpawnObject="True", DisableNativeTick) , MinimalAPI)
class UUserWidget : public UWidget, public INamedSlotInterface
{
	GENERATED_BODY()

	friend class SObjectWidget;
public:
	UMG_API UUserWidget(const FObjectInitializer& ObjectInitializer);

	//~ Begin UObject interface
	UMG_API virtual class UWorld* GetWorld() const override;
	UMG_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UMG_API virtual void BeginDestroy() override;
	UMG_API virtual void PostLoad() override;
	//~ End UObject Interface

	UMG_API void DuplicateAndInitializeFromWidgetTree(UWidgetTree* InWidgetTree, const TMap<FName, UWidget*>& NamedSlotContentToMerge);

	UMG_API virtual bool Initialize();

	EWidgetTickFrequency GetDesiredTickFrequency() const { return TickFrequency; }

	/**
	 * Returns the BlueprintGeneratedClass that generated the WidgetTree.
	 * A child UserWidget that extends a parent UserWidget will not have a new WidgetTree.
	 * The child UserWidget will have the same WidgetTree as the parent UserWidget.
	 * This function returns the parent UserWidget's BlueprintClass.
	 */
	UMG_API UWidgetBlueprintGeneratedClass* GetWidgetTreeOwningClass() const;

	UMG_API void UpdateCanTick();

protected:
	/** The function is implemented only in nativized widgets (automatically converted from BP to c++) */
	virtual void InitializeNativeClassData() {}

	/** Returns whether this widget needs a legacy player. */
	virtual ERequiresLegacyPlayer GetLegacyPlayerRequirement() const { return ERequiresLegacyPlayer::AutoDetect; }

	UMG_API void InitializeNamedSlots();

public:
	//~ Begin UVisual interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UNamedSlotInterface Begin
	UMG_API virtual void GetSlotNames(TArray<FName>& SlotNames) const override;
	UMG_API virtual UWidget* GetContentForSlot(FName SlotName) const override;
	UMG_API virtual void SetContentForSlot(FName SlotName, UWidget* Content) override;
	//~ UNamedSlotInterface End

	/**
	 * Adds it to the game's viewport and fills the entire screen, unless SetDesiredSizeInViewport is called
	 * to explicitly set the size.
	 *
	 * @param ZOrder The higher the number, the more on top this widget will be.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Viewport", meta=( AdvancedDisplay = "ZOrder" ))
	UMG_API void AddToViewport(int32 ZOrder = 0);

	/**
	 * Adds the widget to the game's viewport in a section dedicated to the player.  This is valuable in a split screen
	 * game where you need to only show a widget over a player's portion of the viewport.
	 *
	 * @param ZOrder The higher the number, the more on top this widget will be.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Viewport", meta=( AdvancedDisplay = "ZOrder" ))
	UMG_API bool AddToPlayerScreen(int32 ZOrder = 0);

	/**
	 * Removes the widget from the viewport.
	 */
	UE_DEPRECATED(5.1, "RemoveFromViewport is deprecated. Use RemoveFromParent instead.")
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Viewport", meta=( DeprecatedFunction, DeprecationMessage="Use RemoveFromParent instead" ))
	UMG_API void RemoveFromViewport();

	/**
	 * Sets the widgets position in the viewport.
	 * @param Position The 2D position to set the widget to in the viewport.
	 * @param bRemoveDPIScale If you've already calculated inverse DPI, set this to false.  
	 * Otherwise inverse DPI is applied to the position so that when the location is scaled 
	 * by DPI, it ends up in the expected position.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Viewport")
	UMG_API void SetPositionInViewport(FVector2D Position, bool bRemoveDPIScale = true);

	/*  */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Viewport")
	UMG_API void SetDesiredSizeInViewport(FVector2D Size);

	/*  */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Viewport")
	UMG_API void SetAnchorsInViewport(FAnchors Anchors);

	/*  */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Viewport")
	UMG_API void SetAlignmentInViewport(FVector2D Alignment);

	/*  */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Viewport")
	UMG_API FAnchors GetAnchorsInViewport() const;

	/*  */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Viewport")
	UMG_API FVector2D GetAlignmentInViewport() const;

	/*  */
	UE_DEPRECATED(5.1, "GetIsVisible is deprecated. Please use IsInViewport instead.")
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Appearance", meta=( DeprecatedFunction, DeprecationMessage="Use IsInViewport instead" ))
	UMG_API bool GetIsVisible() const;

	/** Sets the visibility of the widget. */
	UMG_API virtual void SetVisibility(ESlateVisibility InVisibility) override;

	/** Sets the player context associated with this UI. */
	UMG_API void SetPlayerContext(const FLocalPlayerContext& InPlayerContext);

	/** Gets the player context associated with this UI. */
	UMG_API const FLocalPlayerContext& GetPlayerContext() const;

	/**
	 * Gets the local player associated with this UI.
	 * @return The owning local player.
	 */
	UMG_API virtual ULocalPlayer* GetOwningLocalPlayer() const override;
	
	/**
	 * Gets the local player associated with this UI cast to the template type.
	 * @return The owning local player. May be NULL if the cast fails.
	 */
	template < class T >
	T* GetOwningLocalPlayer() const
	{
		return Cast<T>(GetOwningLocalPlayer());
	}

	/**
	 * Sets the player associated with this UI via LocalPlayer reference.
	 * @param LocalPlayer The local player you want to be the conceptual owner of this UI.
	 */
	UMG_API void SetOwningLocalPlayer(ULocalPlayer* LocalPlayer);

	/**
	 * Gets the player controller associated with this UI.
	 * @return The player controller that owns the UI.
	 */
	UMG_API virtual APlayerController* GetOwningPlayer() const override;
	
	/**
	 * Gets the player controller associated with this UI cast to the template type.
	 * @return The player controller that owns the UI. May be NULL if the cast fails.
	 */
	template < class T >
	T* GetOwningPlayer() const
	{
		return Cast<T>(GetOwningPlayer());
	}
	
	/**
	 * Sets the local player associated with this UI via PlayerController reference.
	 * @param LocalPlayerController The PlayerController of the local player you want to be the conceptual owner of this UI.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Player")
	UMG_API void SetOwningPlayer(APlayerController* LocalPlayerController);

	/**
	 * Gets the player pawn associated with this UI.
	 * @return Gets the owning player pawn that's owned by the player controller assigned to this widget.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Player")
	UMG_API class APawn* GetOwningPlayerPawn() const;
	
	/**
	 * Gets the player pawn associated with this UI cast to the template type.
	 * @return Gets the owning player pawn that's owned by the player controller assigned to this widget.
	 * May be NULL if the cast fails.
	 */
	template < class T >
	T* GetOwningPlayerPawn() const
	{
		return Cast<T>(GetOwningPlayerPawn());
	}

	/**
	 * Get the owning player's PlayerState.
	 *
	 * @return const APlayerState*
	 */
	template <class TPlayerState = APlayerState>
	TPlayerState* GetOwningPlayerState(bool bChecked = false) const
	{
		if (auto Controller = GetOwningPlayer())
		{
			return !bChecked ? Cast<TPlayerState>(Controller->PlayerState) :
			                   CastChecked<TPlayerState>(Controller->PlayerState, ECastCheckedType::NullAllowed);
		}

		return nullptr;
	}

	/**
	 * Gets the player camera manager associated with this UI.
	 * @return Gets the owning player camera manager that's owned by the player controller assigned to this widget.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Player")
	UMG_API class APlayerCameraManager* GetOwningPlayerCameraManager() const;

	/**
	 * Gets the player camera manager associated with this UI cast to the template type.
	 * @return Gets the owning player camera manager that's owned by the player controller assigned to this widget.
	 * May be NULL if the cast fails.
	 */
	template <class T>
	T* GetOwningPlayerCameraManager() const
	{
		return Cast<T>(GetOwningPlayerCameraManager());
	}

	/** 
	 * Called once only at game time on non-template instances.
	 * While Construct/Destruct pertain to the underlying Slate, this is called only once for the UUserWidget.
	 * If you have one-time things to establish up-front (like binding callbacks to events on BindWidget properties), do so here.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="User Interface")
	UMG_API void OnInitialized();

	/**
	 * Called by both the game and the editor.  Allows users to run initial setup for their widgets to better preview
	 * the setup in the designer and since generally that same setup code is required at runtime, it's called there
	 * as well.
	 *
	 * **WARNING**
	 * This is intended purely for cosmetic updates using locally owned data, you can not safely access any game related
	 * state, if you call something that doesn't expect to be run at editor time, you may crash the editor.
	 *
	 * In the event you save the asset with blueprint code that causes a crash on evaluation.  You can turn off
	 * PreConstruct evaluation in the Widget Designer settings in the Editor Preferences.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="User Interface")
	UMG_API void PreConstruct(bool IsDesignTime);

	/**
	 * Called after the underlying slate widget is constructed.  Depending on how the slate object is used
	 * this event may be called multiple times due to adding and removing from the hierarchy.
	 * If you need a true called-once-when-created event, use OnInitialized.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="User Interface", meta=( Keywords="Begin Play" ))
	UMG_API void Construct();

	/**
	 * Called when a widget is no longer referenced causing the slate resource to destroyed.  Just like
	 * Construct this event can be called multiple times.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="User Interface", meta=( Keywords="End Play, Destroy" ))
	UMG_API void Destruct();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  MyGeometry The space allotted for this widget
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="User Interface")
	UMG_API void Tick(FGeometry MyGeometry, float InDeltaTime);

	/**
	 * 
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="User Interface | Painting")
	UMG_API void OnPaint(UPARAM(ref) FPaintContext& Context) const;

	/**
	 * Gets a value indicating if the widget is interactive.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="User Interface | Interaction")
	UMG_API bool IsInteractable() const;

	/**
	 * Called when keyboard focus is given to this widget.  This event does not bubble.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param InFocusEvent  FocusEvent
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Input")
	UMG_API FEventReply OnFocusReceived(FGeometry MyGeometry, FFocusEvent InFocusEvent);

	/**
	 * Called when this widget loses focus.  This event does not bubble.
	 *
	 * @param  InFocusEvent  FocusEvent
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Input")
	UMG_API void OnFocusLost(FFocusEvent InFocusEvent);

	/**
	 * If focus is gained on on this widget or on a child widget and this widget is added
	 * to the focus path, and wasn't previously part of it, this event is called.
	 *
	 * @param  InFocusEvent  FocusEvent
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Input")
	UMG_API void OnAddedToFocusPath(FFocusEvent InFocusEvent);

	/**
	 * If focus is lost on on this widget or on a child widget and this widget is
	 * no longer part of the focus path.
	 *
	 * @param  InFocusEvent  FocusEvent
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Input")
	UMG_API void OnRemovedFromFocusPath(FFocusEvent InFocusEvent);

	/**
	 * Called after a character is entered while this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InCharacterEvent  Character event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Input")
	UMG_API FEventReply OnKeyChar(FGeometry MyGeometry, FCharacterEvent InCharacterEvent);

	/**
	 * Called after a key (keyboard, controller, ...) is pressed when this widget or a child of this widget has focus
	 * If a widget handles this event, OnKeyDown will *not* be passed to the focused widget.
	 *
	 * This event is primarily to allow parent widgets to consume an event before a child widget processes
	 * it and it should be used only when there is no better design alternative.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, Category="Input")
	UMG_API FEventReply OnPreviewKeyDown(FGeometry MyGeometry, FKeyEvent InKeyEvent);

	/**
	 * Called after a key (keyboard, controller, ...) is pressed when this widget has focus (this event bubbles if not handled)
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Input")
	UMG_API FEventReply OnKeyDown(FGeometry MyGeometry, FKeyEvent InKeyEvent);

	/**
	 * Called after a key (keyboard, controller, ...) is released when this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Input")
	UMG_API FEventReply OnKeyUp(FGeometry MyGeometry, FKeyEvent InKeyEvent);

	/**
	* Called when an analog value changes on a button that supports analog
	*
	* @param MyGeometry The Geometry of the widget receiving the event
	* @param  InAnalogInputEvent  Analog Event
	* @return  Returns whether the event was handled, along with other possible actions
	*/
	UFUNCTION(BlueprintImplementableEvent, Category = "Input")
	UMG_API FEventReply OnAnalogValueChanged(FGeometry MyGeometry, FAnalogInputEvent InAnalogInputEvent);

	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API FEventReply OnMouseButtonDown(FGeometry MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * Just like OnMouseButtonDown, but tunnels instead of bubbling.
	 * If this event is handled, OnMouseButtonDown will not be sent.
	 * 
	 * Use this event sparingly as preview events generally make UIs more
	 * difficult to reason about.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API FEventReply OnPreviewMouseButtonDown(FGeometry MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API FEventReply OnMouseButtonUp(FGeometry MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API FEventReply OnMouseMove(FGeometry MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API void OnMouseEnter(FGeometry MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API void OnMouseLeave(const FPointerEvent& MouseEvent);

	/**
	 * Called when the mouse wheel is spun. This event is bubbled.
	 *
	 * @param  MouseEvent  Mouse event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API FEventReply OnMouseWheel(FGeometry MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * Called when a mouse button is double clicked.  Override this in derived classes.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  InMouseEvent  Mouse button event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Mouse")
	UMG_API FEventReply OnMouseButtonDoubleClick(FGeometry InMyGeometry, const FPointerEvent& InMouseEvent);

	// TODO
	//UFUNCTION(BlueprintImplementableEvent, Category="Mouse")
	//FCursorReply OnCursorQuery(FGeometry MyGeometry, const FPointerEvent& CursorEvent) const;

	// TODO
	//virtual bool OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent);

	/**
	 * Called when Slate detects that a widget started to be dragged.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  PointerEvent  MouseMove that triggered the drag
	 * @param  Operation     The drag operation that was detected.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Drag and Drop")
	UMG_API void OnDragDetected(FGeometry MyGeometry, const FPointerEvent& PointerEvent, UDragDropOperation*& Operation);

	/**
	 * Called when the user cancels the drag operation, typically when they simply release the mouse button after
	 * beginning a drag operation, but failing to complete the drag.
	 *
	 * @param  PointerEvent  Last mouse event from when the drag was canceled.
	 * @param  Operation     The drag operation that was canceled.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Drag and Drop")
	UMG_API void OnDragCancelled(const FPointerEvent& PointerEvent, UDragDropOperation* Operation);
	
	/**
	 * Called during drag and drop when the drag enters the widget.
	 *
	 * @param MyGeometry     The geometry of the widget receiving the event.
	 * @param PointerEvent   The mouse event from when the drag entered the widget.
	 * @param Operation      The drag operation that entered the widget.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Drag and Drop")
	UMG_API void OnDragEnter(FGeometry MyGeometry, FPointerEvent PointerEvent, UDragDropOperation* Operation);

	/**
	 * Called during drag and drop when the drag leaves the widget.
	 *
	 * @param PointerEvent   The mouse event from when the drag left the widget.
	 * @param Operation      The drag operation that entered the widget.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Drag and Drop")
	UMG_API void OnDragLeave(FPointerEvent PointerEvent, UDragDropOperation* Operation);

	/**
	 * Called during drag and drop when the the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry     The geometry of the widget receiving the event.
	 * @param PointerEvent   The mouse event from when the drag occurred over the widget.
	 * @param Operation      The drag operation over the widget.
	 *
	 * @return 'true' to indicate that you handled the drag over operation.  Returning 'false' will cause the operation to continue to bubble up.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Drag and Drop")
	UMG_API bool OnDragOver(FGeometry MyGeometry, FPointerEvent PointerEvent, UDragDropOperation* Operation);

	/**
	 * Called when the user is dropping something onto a widget.  Ends the drag and drop operation, even if no widget handles this.
	 *
	 * @param MyGeometry     The geometry of the widget receiving the event.
	 * @param PointerEvent   The mouse event from when the drag occurred over the widget.
	 * @param Operation      The drag operation over the widget.
	 * 
	 * @return 'true' to indicate you handled the drop operation.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Drag and Drop")
	UMG_API bool OnDrop(FGeometry MyGeometry, FPointerEvent PointerEvent, UDragDropOperation* Operation);

	/**
	 * Called when the user performs a gesture on trackpad. This event is bubbled.
	 *
	 * @param MyGeometry     The geometry of the widget receiving the event.
	 * @param  GestureEvent  gesture event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Touch Input")
	UMG_API FEventReply OnTouchGesture(FGeometry MyGeometry, const FPointerEvent& GestureEvent);

	/**
	 * Called when a touchpad touch is started (finger down)
	 * 
	 * @param MyGeometry    The geometry of the widget receiving the event.
	 * @param InTouchEvent	The touch event generated
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Touch Input")
	UMG_API FEventReply OnTouchStarted(FGeometry MyGeometry, const FPointerEvent& InTouchEvent);
	
	/**
	 * Called when a touchpad touch is moved  (finger moved)
	 * 
	 * @param MyGeometry    The geometry of the widget receiving the event.
	 * @param InTouchEvent	The touch event generated
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Touch Input")
	UMG_API FEventReply OnTouchMoved(FGeometry MyGeometry, const FPointerEvent& InTouchEvent);

	/**
	 * Called when a touchpad touch is ended (finger lifted)
	 * 
	 * @param MyGeometry    The geometry of the widget receiving the event.
	 * @param InTouchEvent	The touch event generated
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Touch Input")
	UMG_API FEventReply OnTouchEnded(FGeometry MyGeometry, const FPointerEvent& InTouchEvent);
	
	/**
	 * Called when motion is detected (controller or device)
	 * e.g. Someone tilts or shakes their controller.
	 * 
	 * @param MyGeometry    The geometry of the widget receiving the event.
	 * @param MotionEvent	The motion event generated
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Touch Input")
	UMG_API FEventReply OnMotionDetected(FGeometry MyGeometry, FMotionEvent InMotionEvent);

	/**
	 * Called when mouse capture is lost if it was owned by this widget.
	 */
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category="Touch Input")
	UMG_API void OnMouseCaptureLost();

	/**
	 * Cancels any pending Delays or timer callbacks for this widget.
	 */
	UFUNCTION(BlueprintCallable, Category = "Delay")
	UMG_API void CancelLatentActions();

	/**
	* Cancels any pending Delays or timer callbacks for this widget, and stops all active animations on the widget.
	*/
	UFUNCTION(BlueprintCallable, Category = "Delay")
	UMG_API void StopAnimationsAndLatentActions();

	/**
	* Called when a touchpad force has changed (user pressed down harder or let up)
	*
	* @param MyGeometry    The geometry of the widget receiving the event.
	* @param InTouchEvent	The touch event generated
	*/
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category = "Touch Input")
	UMG_API FEventReply OnTouchForceChanged(FGeometry MyGeometry, const FPointerEvent& InTouchEvent);

	/**
	* Called when a touchpad touch is first moved (initial finger movement)
	*
	* @param MyGeometry    The geometry of the widget receiving the event.
	* @param InTouchEvent	The touch event generated
	*/
	UFUNCTION(BlueprintImplementableEvent, BlueprintCosmetic, Category = "Touch Input")
	UMG_API FEventReply OnTouchFirstMove(FGeometry MyGeometry, const FPointerEvent& InTouchEvent);

public:

	/**
	 * Bind an animation started delegate.
	 * @param Animation the animation to listen for starting or finishing.
	 * @param Delegate the delegate to call when the animation's state changes
	 */
	UFUNCTION(BlueprintCallable, Category=Animation)
	UMG_API void BindToAnimationStarted(UWidgetAnimation* Animation, FWidgetAnimationDynamicEvent Delegate);

	/**
	 * Unbind an animation started delegate.
	 * @param Animation the animation to listen for starting or finishing.
	 * @param Delegate the delegate to call when the animation's state changes
	 */
	UFUNCTION(BlueprintCallable, Category = Animation)
	UMG_API void UnbindFromAnimationStarted(UWidgetAnimation* Animation, FWidgetAnimationDynamicEvent Delegate);

	UFUNCTION(BlueprintCallable, Category = Animation)
	UMG_API void UnbindAllFromAnimationStarted(UWidgetAnimation* Animation);

	/**
	 * Bind an animation finished delegate.
	 * @param Animation the animation to listen for starting or finishing.
	 * @param Delegate the delegate to call when the animation's state changes
	 */
	UFUNCTION(BlueprintCallable, Category = Animation)
	UMG_API void BindToAnimationFinished(UWidgetAnimation* Animation, FWidgetAnimationDynamicEvent Delegate);

	/**
	 * Unbind an animation finished delegate.
	 * @param Animation the animation to listen for starting or finishing.
	 * @param Delegate the delegate to call when the animation's state changes
	 */
	UFUNCTION(BlueprintCallable, Category = Animation)
	UMG_API void UnbindFromAnimationFinished(UWidgetAnimation* Animation, FWidgetAnimationDynamicEvent Delegate);

	UFUNCTION(BlueprintCallable, Category = Animation)
	UMG_API void UnbindAllFromAnimationFinished(UWidgetAnimation* Animation);

	/**
	 * Allows binding to a specific animation's event.
	 * @param Animation the animation to listen for starting or finishing.
	 * @param Delegate the delegate to call when the animation's state changes
	 * @param AnimationEvent the event to watch for.
	 * @param UserTag Scopes the delegate to only be called when the animation completes with a specific tag set on it when it was played.
	 */
	UFUNCTION(BlueprintCallable, Category = Animation)
	UMG_API void BindToAnimationEvent(UWidgetAnimation* Animation, FWidgetAnimationDynamicEvent Delegate, EWidgetAnimationEvent AnimationEvent, FName UserTag = NAME_None);

	/** Is this widget an editor utility widget. */
	virtual bool IsEditorUtility() const { return false; }

protected:

	/**
	 * Called when an animation is started.
	 *
	 * @param Animation the animation that started
	 */
	UFUNCTION( BlueprintNativeEvent, BlueprintCosmetic, Category = "Animation" )
	UMG_API void OnAnimationStarted( const UWidgetAnimation* Animation );

	UMG_API virtual void OnAnimationStarted_Implementation(const UWidgetAnimation* Animation);

	/**
	 * Called when an animation has either played all the way through or is stopped
	 *
	 * @param Animation The animation that has finished playing
	 */
	UFUNCTION( BlueprintNativeEvent, BlueprintCosmetic, Category = "Animation" )
	UMG_API void OnAnimationFinished( const UWidgetAnimation* Animation );

	UMG_API virtual void OnAnimationFinished_Implementation(const UWidgetAnimation* Animation);

	/** Broadcast any events based on a state transition for the sequence player, started, finished...etc. */
	UE_DEPRECATED(5.6, "UUMGSequencePlayer is deprecated")
	UMG_API void BroadcastAnimationStateChange(const UUMGSequencePlayer& Player, EWidgetAnimationEvent AnimationEvent);

private:

	UMG_API void BroadcastAnimationStateChange(const FWidgetAnimationState& State, EWidgetAnimationEvent AnimationEvent);

	UMG_API void BroadcastAnimationStateChange(const UWidgetAnimation* Animation, FName UserTag, EWidgetAnimationEvent AnimationEvent);

	UMG_API void BroadcastAnimationStartedPlaying(FWidgetAnimationState& State);
	UMG_API void BroadcastAnimationFinishedPlaying(FWidgetAnimationState& State);

protected:

	/** Called when a sequence player is finished playing an animation */
	FOnWidgetAnimationEvent& OnAnimationStartedPlaying() { return OnAnimationStartedPlayingEvent; }

	/** Called when a sequence player is finished playing an animation */
	FOnWidgetAnimationEvent& OnAnimationFinishedPlaying() { return OnAnimationFinishedPlayingEvent; }

	UE_DEPRECATED(5.6, "UUMGSequencePlayer is deprecated")
	virtual void OnAnimationStartedPlaying(UUMGSequencePlayer& Player) {}

	UE_DEPRECATED(5.6, "UUMGSequencePlayer is deprecated")
	virtual void OnAnimationFinishedPlaying(UUMGSequencePlayer& Player) {}

private:

	FOnWidgetAnimationEvent OnAnimationStartedPlayingEvent;
	FOnWidgetAnimationEvent OnAnimationFinishedPlayingEvent;

public:

	UE_DEPRECATED(5.2, "Direct access to ColorAndOpacity is deprecated. Please use the getter or setter.")
	/** The color and opacity of this widget.  Tints all child widgets. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetColorAndOpacity", Category = "Appearance")
	FLinearColor ColorAndOpacity;

	UPROPERTY()
	FGetLinearColor ColorAndOpacityDelegate;

	UE_DEPRECATED(5.2, "Direct access to ForegroundColor is deprecated. Please use the getter or setter.")
	/**
	 * The foreground color of the widget, this is inherited by sub widgets.  Any color property
	 * that is marked as inherit will use this color.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetForegroundColor", Category = "Appearance")
	FSlateColor ForegroundColor;

	UPROPERTY()
	FGetSlateColor ForegroundColorDelegate;

	/** Called when the visibility has changed */
	UPROPERTY(BlueprintAssignable, Category = "Appearance|Event")
	FOnVisibilityChangedEvent OnVisibilityChanged;
	DECLARE_EVENT_OneParam(UUserWidget, FNativeOnVisibilityChangedEvent, ESlateVisibility);
	FNativeOnVisibilityChangedEvent OnNativeVisibilityChanged;

	DECLARE_EVENT_OneParam(UUserWidget, FNativeOnDestruct, UUserWidget*);
	FNativeOnDestruct OnNativeDestruct;

	UE_DEPRECATED(5.2, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area around the content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetPadding", Category = "Appearance")
	FMargin Padding;

	UE_DEPRECATED(5.2, "Direct access to Priority is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetInputActionPriority", Setter = "SetInputActionPriority", BlueprintSetter = "SetInputActionPriority", Category = "Input")
	int32 Priority;

	UE_DEPRECATED(5.2, "Direct access to bIsFocusable is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** Setting this flag to true, allows this widget to accept focus when clicked, or when navigated to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsFocusable", Setter = "SetIsFocusable", Category = "Interaction")
	uint8 bIsFocusable : 1;
	 
	UE_DEPRECATED(5.2, "Direct access to bStopAction is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsInputActionBlocking", Setter = "SetInputActionBlocking", BlueprintSetter = "SetInputActionBlocking", Category = "Input")
	uint8 bStopAction : 1;

	/**
	 * If true, this widget will automatically register its own input component upon construction.
	 * This will allow the use of binding input delegates in the event graph.
	 * 
	 * This is set during the compilation of the widget blueprint. 
	 */
	UPROPERTY()
	uint8 bAutomaticallyRegisterInputOnConstruction : 1;

public:

	/**
	 * Sets the tint of the widget, this affects all child widgets.
	 * 
	 * @param InColorAndOpacity	The tint to apply to all child widgets.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Appearance")
	UMG_API void SetColorAndOpacity(FLinearColor InColorAndOpacity);

	/**
	 * Gets the tint of the widget.
	 */
	UMG_API const FLinearColor& GetColorAndOpacity() const;

	/**
	 * Sets the foreground color of the widget, this is inherited by sub widgets.  Any color property 
	 * that is marked as inherit will use this color.
	 *
	 * @param InForegroundColor	The foreground color.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Appearance")
	UMG_API void SetForegroundColor(FSlateColor InForegroundColor);

	/**
	 * Gets the foreground color of the widget, this is inherited by sub widgets.  Any color property
	 * that is marked as inherit uses this color.
	 */
	UMG_API const FSlateColor& GetForegroundColor() const;

	/**
	 * Sets the padding for the user widget, putting a larger gap between the widget border and it's root widget.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Appearance")
	UMG_API void SetPadding(FMargin InPadding);

	/**
	 * Gets the padding for the user widget.
	 */
	UMG_API FMargin GetPadding() const;

	/**
	 * Gets the priority of the input action.
	 */
	UMG_API int32 GetInputActionPriority() const;

	/**
	 * Returns whether the input action is blocking.
	 */
	UMG_API bool IsInputActionBlocking() const;

	/**
	 * Sets whether this widget to accept focus when clicked, or when navigated to.
	 */
	UMG_API bool IsFocusable() const;

	UMG_API void SetIsFocusable(bool InIsFocusable);

	/**
	 * Plays an animation in this widget a specified number of times
	 * 
	 * @param InAnimation The animation to play
	 * @param StartAtTime The time in the animation from which to start playing, relative to the start position. For looped animations, this will only affect the first playback of the animation.
	 * @param NumLoopsToPlay The number of times to loop this animation (0 to loop indefinitely)
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param PlayMode Specifies the playback mode
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void QueuePlayAnimation(UWidgetAnimation* InAnimation, float StartAtTime = 0.0f, int32 NumLoopsToPlay = 1, EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Plays an animation in this widget a specified number of times stopping at a specified time
	 * 
	 * @param InAnimation The animation to play
	 * @param StartAtTime The time in the animation from which to start playing, relative to the start position. For looped animations, this will only affect the first playback of the animation.
	 * @param EndAtTime The absolute time in the animation where to stop, this is only considered in the last loop.
	 * @param NumLoopsToPlay The number of times to loop this animation (0 to loop indefinitely)
	 * @param PlayMode Specifies the playback mode
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void QueuePlayAnimationTimeRange(UWidgetAnimation* InAnimation, float StartAtTime = 0.0f, float EndAtTime = 0.0f, int32 NumLoopsToPlay = 1, EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Plays an animation on this widget relative to it's current state forward.  You should use this version in situations where
	 * say a user can click a button and that causes a panel to slide out, and you want to reverse that same animation to begin sliding
	 * in the opposite direction.
	 * 
	 * @param InAnimation The animation to play
	 * @param PlayMode Specifies the playback mode
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void QueuePlayAnimationForward(UWidgetAnimation* InAnimation, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Plays an animation on this widget relative to it's current state in reverse.  You should use this version in situations where
	 * say a user can click a button and that causes a panel to slide out, and you want to reverse that same animation to begin sliding
	 * in the opposite direction.
	 *
	 * @param InAnimation The animation to play
	 * @param PlayMode Specifies the playback mode
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void QueuePlayAnimationReverse(UWidgetAnimation* InAnimation, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Stops an already running animation in this widget
	 * 
	 * @param The name of the animation to stop
	 */
	UFUNCTION(BlueprintCallable, Category="User Interface|Animation")
	UMG_API void QueueStopAnimation(const UWidgetAnimation* InAnimation);

	/**
	 * Stop All actively running animations.
	 */
	UFUNCTION(BlueprintCallable, Category="User Interface|Animation")
	UMG_API void QueueStopAllAnimations();

	/**
	 * Pauses an already running animation in this widget
	 * 
	 * @param The name of the animation to pause
	 * @return the time point the animation was at when it was paused, relative to its start position.  Use this as the StartAtTime when you trigger PlayAnimation.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Animation")
	UMG_API float QueuePauseAnimation(const UWidgetAnimation* InAnimation);

	/**
	 * Plays an animation in this widget a specified number of times
	 * 
	 * @param InAnimation The animation to play
	 * @param StartAtTime The time in the animation from which to start playing, relative to the start position. For looped animations, this will only affect the first playback of the animation.
	 * @param NumLoopsToPlay The number of times to loop this animation (0 to loop indefinitely)
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param PlayMode Specifies the playback mode
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API FWidgetAnimationHandle PlayAnimation(UWidgetAnimation* InAnimation, float StartAtTime = 0.0f, int32 NumLoopsToPlay = 1, EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Plays an animation in this widget a specified number of times stopping at a specified time
	 * 
	 * @param InAnimation The animation to play
	 * @param StartAtTime The time in the animation from which to start playing, relative to the start position. For looped animations, this will only affect the first playback of the animation.
	 * @param EndAtTime The absolute time in the animation where to stop, this is only considered in the last loop.
	 * @param NumLoopsToPlay The number of times to loop this animation (0 to loop indefinitely)
	 * @param PlayMode Specifies the playback mode
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API FWidgetAnimationHandle PlayAnimationTimeRange(UWidgetAnimation* InAnimation, float StartAtTime = 0.0f, float EndAtTime = 0.0f, int32 NumLoopsToPlay = 1, EUMGSequencePlayMode::Type PlayMode = EUMGSequencePlayMode::Forward, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Plays an animation on this widget relative to it's current state forward.  You should use this version in situations where
	 * say a user can click a button and that causes a panel to slide out, and you want to reverse that same animation to begin sliding
	 * in the opposite direction.
	 * 
	 * @param InAnimation The animation to play
	 * @param PlayMode Specifies the playback mode
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API FWidgetAnimationHandle PlayAnimationForward(UWidgetAnimation* InAnimation, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Plays an animation on this widget relative to it's current state in reverse.  You should use this version in situations where
	 * say a user can click a button and that causes a panel to slide out, and you want to reverse that same animation to begin sliding
	 * in the opposite direction.
	 *
	 * @param InAnimation The animation to play
	 * @param PlayMode Specifies the playback mode
	 * @param PlaybackSpeed The speed at which the animation should play
	 * @param bRestoreState Restores widgets to their pre-animated state when the animation stops
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API FWidgetAnimationHandle PlayAnimationReverse(UWidgetAnimation* InAnimation, float PlaybackSpeed = 1.0f, bool bRestoreState = false);

	/**
	 * Stops an already running animation in this widget
	 * 
	 * @param The name of the animation to stop
	 */
	UFUNCTION(BlueprintCallable, Category="User Interface|Animation")
	UMG_API void StopAnimation(const UWidgetAnimation* InAnimation);

	/**
	 * Stop All actively running animations.
	 * 
	 * @param The name of the animation to stop
	 */
	UFUNCTION(BlueprintCallable, Category="User Interface|Animation")
	UMG_API void StopAllAnimations();

	/**
	 * Pauses an already running animation in this widget
	 * 
	 * @param The name of the animation to pause
	 * @return the time point the animation was at when it was paused, relative to its start position.  Use this as the StartAtTime when you trigger PlayAnimation.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Animation")
	UMG_API float PauseAnimation(const UWidgetAnimation* InAnimation);

	/**
	 * Gets the current time of the animation in this widget
	 * 
	 * @param The name of the animation to get the current time for
	 * @return the current time of the animation.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API float GetAnimationCurrentTime(const UWidgetAnimation* InAnimation) const;

	/**
	 * Sets the current time of the animation in this widget. Does not change state.
	 * 
	 * @param The name of the animation to get the current time for
	 * @param The current time of the animation.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void SetAnimationCurrentTime(const UWidgetAnimation* InAnimation, float InTime);

	/**
	 * Gets whether an animation is currently playing on this widget.
	 * 
	 * @param InAnimation The animation to check the playback status of
	 * @return True if the animation is currently playing
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Animation")
	UMG_API bool IsAnimationPlaying(const UWidgetAnimation* InAnimation) const;

	/**
	 * @return True if any animation is currently playing
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="User Interface|Animation")
	UMG_API bool IsAnyAnimationPlaying() const;

	/**
	* Changes the number of loops to play given a playing animation
	*
	* @param InAnimation The animation that is already playing
	* @param NumLoopsToPlay The number of loops to play. (0 to loop indefinitely)
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void SetNumLoopsToPlay(const UWidgetAnimation* InAnimation, int32 NumLoopsToPlay);

	/**
	* Changes the playback rate of a playing animation
	*
	* @param InAnimation The animation that is already playing
	* @param PlaybackRate Playback rate multiplier (1 is default)
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void SetPlaybackSpeed(const UWidgetAnimation* InAnimation, float PlaybackSpeed = 1.0f);

	/**
	* If an animation is playing, this function will reverse the playback.
	*
	* @param InAnimation The playing animation that we want to reverse
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void ReverseAnimation(const UWidgetAnimation* InAnimation);

	/**
	 * returns true if the animation is currently playing forward, false otherwise.
	 *
	 * @param InAnimation The playing animation that we want to know about
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API bool IsAnimationPlayingForward(const UWidgetAnimation* InAnimation);

	/**
	 * Flushes all animations on all widgets to guarantee that any queued updates are processed before this call returns
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "User Interface|Animation")
	UMG_API void FlushAnimations();

	/** Find the first extension of the requested type. */
	template<typename ExtensionType>
	ExtensionType* GetExtension() const
	{
		return CastChecked<ExtensionType>(GetExtension(ExtensionType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	/** Find the first extension of the requested type. */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Extension", Meta = (DeterminesOutputType = "ExtensionType"))
	UMG_API UUserWidgetExtension* GetExtension(TSubclassOf<UUserWidgetExtension> ExtensionType) const;

	/** Find the extensions of the requested type. */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Extension", Meta = (DeterminesOutputType = "ExtensionType"))
	UMG_API TArray<UUserWidgetExtension*> GetExtensions(TSubclassOf<UUserWidgetExtension> ExtensionType) const;

	/** Add the extension of the requested type. */
	template<typename ExtensionType>
	ExtensionType* AddExtension()
	{
		return CastChecked<ExtensionType>(AddExtension(ExtensionType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	/** Add the extension of the requested type. */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Extension", Meta = (DeterminesOutputType = "InExtensionType"))
	UMG_API UUserWidgetExtension* AddExtension(TSubclassOf<UUserWidgetExtension> InExtensionType);

	/** Find an existing extension of the request type, or create it if not found */
	template<typename ExtensionType>
	ExtensionType& GetOrAddExtension()
	{
		return *CastChecked<ExtensionType>(GetOrAddExtension(ExtensionType::StaticClass()));
	}

	/** Find an existing extension of the request type, or create it if not found */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Extension", Meta = (DeterminesOutputType = "InExtensionType"))
	UMG_API UUserWidgetExtension* GetOrAddExtension(TSubclassOf<UUserWidgetExtension> InExtensionType);

	/** Remove the extension. */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Extension")
	UMG_API void RemoveExtension(UUserWidgetExtension* InExtension);

	/** Remove all extensions of the requested type. */
	template<typename ExtensionType>
	void RemoveExtensions()
	{
		return RemoveExtensions(ExtensionType::StaticClass());
	}

	/** Remove all extensions of the requested type. */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Extension")
	UMG_API void RemoveExtensions(TSubclassOf<UUserWidgetExtension> InExtensionType);

	/**
	 * Plays a sound through the UI
	 *
	 * @param The sound to play
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Sound", meta=( DeprecatedFunction, DeprecationMessage="Use the UGameplayStatics::PlaySound2D instead." ))
	UMG_API void PlaySound(class USoundBase* SoundToPlay);

	/** 
	 * Sets the child Widget that should receive focus when this UserWidget gets focus using it's name. 
	 *
	 * @param WidgetName Name of the Widget to forward the focus to when this widget receives focus.
	 * @return True if the Widget is set properly. Will return false if we can't find a child widget with the specified name.
	 */
	UMG_API bool SetDesiredFocusWidget(FName WidgetName);

	/** 
	 * Sets the child Widget that should receive focus when this UserWidget gets focus. 
	 *
	 * @param Widget Widget to forward the focus to when this widget receives focus
	 * @return True if the Widget is set properly. Will return false if it's not a child of this UserWidget.
	 */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Focus")
	UMG_API bool SetDesiredFocusWidget(UWidget* Widget);

	/** @returns The Name of the Widget that should receive focus when this UserWidget gets focus. */
	UMG_API FName GetDesiredFocusWidgetName() const;

	/** @returns The Widget that should receive focus when this UserWidget gets focus. */
	UMG_API UWidget* GetDesiredFocusWidget() const;

	/** @returns The UObject wrapper for a given SWidget */
	UMG_API UWidget* GetWidgetHandle(TSharedRef<SWidget> InWidget);

	/** @returns The root UObject widget wrapper */
	UMG_API UWidget* GetRootWidget() const;

	/** @returns The slate widget corresponding to a given name */
	UMG_API TSharedPtr<SWidget> GetSlateWidgetFromName(const FName& Name) const;

	/** @returns The uobject widget corresponding to a given name */
	UMG_API UWidget* GetWidgetFromName(const FName& Name) const;

	//~ Begin UObject Interface
	UMG_API virtual bool IsAsset() const;
	UMG_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	//~ End UObject Interface

	/** Are we currently playing any animations? */
	UFUNCTION(BlueprintCallable, Category="User Interface|Animation")
	UMG_API bool IsPlayingAnimation() const;

#if WITH_EDITOR
	//~ Begin UWidget Interface
	UMG_API virtual const FText GetPaletteCategory() override;
	//~ End UWidget Interface

	UMG_API virtual void SetDesignerFlags(EWidgetDesignFlags NewFlags) override;
	UMG_API virtual void OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs) override;
	UMG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Update the binding for this namedslot if the name is not found but GUID is matched. */
	UMG_API void UpdateBindingForSlot(FName SlotName);

	/** Add the GUID of each Namedslot widget to its corresponding binding, if any. */
	UMG_API void AssignGUIDToBindings();

	/**
	 * Final step of Widget Blueprint compilation. Allows widgets to perform custom validation and trigger compiler outputs as needed.
	 * @see ValidateCompiledDefaults
	 * @see ValidateCompiledWidgetTree
	 */
	UMG_API void ValidateBlueprint(const UWidgetTree& BlueprintWidgetTree, class IWidgetCompilerLog& CompileLog) const;

	/**
	 * Override to perform any custom inspections of the default widget tree at the end of compilation.
	 *
	 * Note: The WidgetTree and BindWidget properties of this user widget will not be established at this point,
	 * so be sure to inspect only the given BlueprintWidgetTree.
	 *
	 * Tip: If you need to validate properties of BindWidget members, you can search for them by property name within the widget tree.
	 */
	virtual void ValidateCompiledWidgetTree(const UWidgetTree& BlueprintWidgetTree, class IWidgetCompilerLog& CompileLog) const {};
#endif

	static UMG_API UUserWidget* CreateWidgetInstance(UWidget& OwningWidget, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName);
	static UMG_API UUserWidget* CreateWidgetInstance(UWidgetTree& OwningWidgetTree, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName);
	static UMG_API UUserWidget* CreateWidgetInstance(APlayerController& OwnerPC, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName);
	static UMG_API UUserWidget* CreateWidgetInstance(UGameInstance& GameInstance, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName);
	static UMG_API UUserWidget* CreateWidgetInstance(UWorld& World, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName);

private:

	static UMG_API UUserWidget* CreateInstanceInternal(UObject* Outer, TSubclassOf<UUserWidget> UserWidgetClass, FName WidgetName, UWorld* World, ULocalPlayer* LocalPlayer);

	void ClearStoppedAnimationStates();

	// If this widget's player context is invalid, this function will set it to the provided player context, otherwise it will keep its current player context and call CheckUpdatePlayerContext on its children
	void UpdatePlayerContextIfInvalid(const FLocalPlayerContext& ParentPlayerContext);

public:

	/** Animation transitions to trigger on next tick */
	UPROPERTY(Transient)
	TArray<FQueuedWidgetAnimationTransition> QueuedWidgetAnimationTransitions;

	UE_DEPRECATED(5.6, "Please use ActiveAnimations")
	UPROPERTY(Transient)
	TArray<TObjectPtr<UUMGSequencePlayer>> ActiveSequencePlayers;
	
	/** Global tick manager for running widget animations */
	UPROPERTY(Transient)
	TObjectPtr<UUMGSequenceTickManager> AnimationTickManager;

	UE_DEPRECATED(5.6, "Please use ActiveAnimations")
	UPROPERTY(Transient)
	TArray<TObjectPtr<UUMGSequencePlayer>> StoppedSequencePlayers;

private:

	/** All the sequences currently playing */
	TArray<TSharedRef<FWidgetAnimationState>> ActiveAnimations;

	/** True when we are ticking our active animations */
	bool bIsTickingAnimations = false;

private:

	/** Stores the widgets being assigned to named slots */
	UPROPERTY()
	TArray<FNamedSlotBinding> NamedSlotBindings;

	/** The UserWidget extensions */
	UPROPERTY()
	TArray<TObjectPtr<UUserWidgetExtension>> Extensions;

public:
	/** The widget tree contained inside this user widget initialized by the blueprint */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TObjectPtr<UWidgetTree> WidgetTree;

public:

#if WITH_EDITORONLY_DATA

	/** Stores the design time desired size of the user widget */
	UPROPERTY()
	FVector2D DesignTimeSize;

	UPROPERTY()
	EDesignPreviewSizeMode DesignSizeMode;

	/** The category this widget appears in the palette. */
	UPROPERTY()
	FText PaletteCategory;

	/**
	 * A preview background that you can use when designing the UI to get a sense of scale on the screen.  Use
	 * a texture with a screenshot of your game in it, for example if you were designing a HUD.
	 */
	UPROPERTY(EditDefaultsOnly, Category="Designer")
	TObjectPtr<UTexture2D> PreviewBackground;

#endif

	/** If a widget has an implemented tick blueprint function */
	UPROPERTY()
	uint8 bHasScriptImplementedTick : 1;

	/** If a widget has an implemented paint blueprint function */
	UPROPERTY()
	uint8 bHasScriptImplementedPaint : 1;

private:

	/** Has this widget been initialized by its class yet? */
	uint8 bInitialized : 1;

	/** Has this widget been preconstructed and we need to call PreConstruct on new extension. */
	uint8 bAreExtensionsPreConstructed : 1;
	
	/** Has this widget been constructed and we need to call Construct on new extension. */
	uint8 bAreExtensionsConstructed : 1;

	/** If we're stopping all animations, don't allow new animations to be created as side-effects. */
	uint8 bStoppingAllAnimations : 1;

protected:

	static UMG_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UMG_API virtual void OnWidgetRebuilt() override;

	UE_DEPRECATED(5.1, "GetFullScreenOffset is deprecated. Use the GameViewportSubsystem.")
	UMG_API FMargin GetFullScreenOffset() const;

	UMG_API virtual void NativeOnInitialized();
	UMG_API virtual void NativePreConstruct();
	UMG_API virtual void NativeConstruct();
	UMG_API virtual void NativeDestruct();
	UMG_API virtual void NativeTick(const FGeometry& MyGeometry, float InDeltaTime);

	/**
	 * Native implemented paint function for the Widget
	 * Returns the maximum LayerID painted on
	 */
	UMG_API virtual int32 NativePaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;

	inline FVector2D GetMinimumDesiredSize() const { return MinimumDesiredSize; }
	UMG_API void SetMinimumDesiredSize(FVector2D InMinimumDesiredSize);

	UMG_API virtual bool NativeIsInteractable() const;
	UMG_API virtual bool NativeSupportsKeyboardFocus() const;
	virtual bool NativeSupportsCustomNavigation() const { return false; }

	UMG_API virtual FReply NativeOnFocusReceived( const FGeometry& InGeometry, const FFocusEvent& InFocusEvent );
	UMG_API virtual void NativeOnFocusLost( const FFocusEvent& InFocusEvent );
	UMG_API virtual void NativeOnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent);
	UMG_API virtual void NativeOnAddedToFocusPath(const FFocusEvent& InFocusEvent);
	UMG_API virtual void NativeOnRemovedFromFocusPath(const FFocusEvent& InFocusEvent);
	UMG_API virtual FNavigationReply NativeOnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent, const FNavigationReply& InDefaultReply);
	UMG_API virtual FReply NativeOnKeyChar( const FGeometry& InGeometry, const FCharacterEvent& InCharEvent );
	UMG_API virtual FReply NativeOnPreviewKeyDown( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent );
	UMG_API virtual FReply NativeOnKeyDown( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent );
	UMG_API virtual FReply NativeOnKeyUp( const FGeometry& InGeometry, const FKeyEvent& InKeyEvent );
	UMG_API virtual FReply NativeOnAnalogValueChanged( const FGeometry& InGeometry, const FAnalogInputEvent& InAnalogEvent );
	UMG_API virtual FReply NativeOnMouseButtonDown( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent );
	UMG_API virtual FReply NativeOnPreviewMouseButtonDown( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent );
	UMG_API virtual FReply NativeOnMouseButtonUp( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent );
	UMG_API virtual FReply NativeOnMouseMove( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent );
	UMG_API virtual void NativeOnMouseEnter( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent );
	UMG_API virtual void NativeOnMouseLeave( const FPointerEvent& InMouseEvent );
	UMG_API virtual FReply NativeOnMouseWheel( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent );
	UMG_API virtual FReply NativeOnMouseButtonDoubleClick( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent );
	UMG_API virtual void NativeOnDragDetected( const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, UDragDropOperation*& OutOperation );
	UMG_API virtual void NativeOnDragEnter( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation );
	UMG_API virtual void NativeOnDragLeave( const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation );
	UMG_API virtual bool NativeOnDragOver( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation );
	UMG_API virtual bool NativeOnDrop( const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation );
	UMG_API virtual void NativeOnDragCancelled( const FDragDropEvent& InDragDropEvent, UDragDropOperation* InOperation );
	UMG_API virtual FReply NativeOnTouchGesture( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent );
	UMG_API virtual FReply NativeOnTouchStarted( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent );
	UMG_API virtual FReply NativeOnTouchMoved( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent );
	UMG_API virtual FReply NativeOnTouchEnded( const FGeometry& InGeometry, const FPointerEvent& InGestureEvent );
	UMG_API virtual FReply NativeOnMotionDetected( const FGeometry& InGeometry, const FMotionEvent& InMotionEvent );
	UMG_API virtual FReply NativeOnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent);
	UMG_API virtual FReply NativeOnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent);
	UMG_API virtual FCursorReply NativeOnCursorQuery( const FGeometry& InGeometry, const FPointerEvent& InCursorEvent );
	UMG_API virtual FNavigationReply NativeOnNavigation(const FGeometry& InGeometry, const FNavigationEvent& InNavigationEvent);
	UMG_API virtual void NativeOnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent);
	virtual void NativeOnFinishedPointerInput() {}

protected:

	/**
	 * Ticks the active sequences and latent actions that have been scheduled for this Widget.
	 */
	UMG_API void TickActionsAndAnimation(float InDeltaTime);

	UE_DEPRECATED(5.6, "This method isn't needed anymore")
	void PostTickActionsAndAnimation(float InDeltaTime) {}

	UMG_API void RemoveObsoleteBindings(const TArray<FName>& NamedSlots);

	UMG_API TSharedPtr<const FWidgetAnimationState> GetAnimationState(const UWidgetAnimation* InAnimation) const;
	UMG_API TSharedPtr<FWidgetAnimationState> GetAnimationState(const UWidgetAnimation* InAnimation);
	UMG_API TSharedPtr<FWidgetAnimationState> GetOrAddAnimationState(UWidgetAnimation* InAnimation);

	UMG_API void ExecuteQueuedAnimationTransitions();

	UMG_API void ConditionalTearDownAnimations();

	UMG_API void TearDownAnimations();

	UMG_API void DisableAnimations();

	UMG_API void Invalidate(EInvalidateWidgetReason InvalidateReason);
	
	/**
	 * Listens for a particular Player Input Action by name.  This requires that those actions are being executed, and
	 * that we're not currently in UI-Only Input Mode.
	 */
	UFUNCTION( BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ) )
	UMG_API void ListenForInputAction( FName ActionName, TEnumAsByte< EInputEvent > EventType, bool bConsume, FOnInputAction Callback );

	/**
	 * Removes the binding for a particular action's callback.
	 */
	UFUNCTION( BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ) )
	UMG_API void StopListeningForInputAction( FName ActionName, TEnumAsByte< EInputEvent > EventType );

	/**
	 * Stops listening to all input actions, and unregisters the input component with the player controller.
	 */
	UFUNCTION( BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ) )
	UMG_API void StopListeningForAllInputActions();

	/**
	 * ListenForInputAction will automatically Register an Input Component with the player input system.
	 * If you however, want to Pause and Resume, listening for a set of actions, the best way is to use
	 * UnregisterInputComponent to pause, and RegisterInputComponent to resume listening.
	 */
	UFUNCTION(BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ))
	UMG_API void RegisterInputComponent();

	/**
	 * StopListeningForAllInputActions will automatically Register an Input Component with the player input system.
	 * If you however, want to Pause and Resume, listening for a set of actions, the best way is to use
	 * UnregisterInputComponent to pause, and RegisterInputComponent to resume listening.
	 */
	UFUNCTION(BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ))
	UMG_API void UnregisterInputComponent();

	/**
	 * Checks if the action has a registered callback with the input component.
	 */
	UFUNCTION( BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ) )
	UMG_API bool IsListeningForInputAction( FName ActionName ) const;

	UFUNCTION( BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ) )
	UMG_API void SetInputActionPriority( int32 NewPriority );

	UFUNCTION( BlueprintCallable, Category = "Input", meta = ( BlueprintProtected = "true" ) )
	UMG_API void SetInputActionBlocking( bool bShouldBlock );

	UMG_API void OnInputAction( FOnInputAction Callback );

	UE_DEPRECATED(5.7, "InitializeInputComponent is deprecated. bAutomaticallyRegisterInputOnConstruction will be set on compilation of this widget and automatically create the input component instead.")
	UMG_API virtual void InitializeInputComponent() final;

private:
	FVector2D MinimumDesiredSize;

private:
	/**
	 * This widget is allowed to tick. If this is unchecked tick will never be called, animations will not play correctly, and latent actions will not execute.
	 * Uncheck this for performance reasons only
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Performance", meta=(AllowPrivateAccess="true"))
	EWidgetTickFrequency TickFrequency;

	UPROPERTY(EditDefaultsOnly, Category = "Interaction", meta = (AllowPrivateAccess = "true"))
	FWidgetChild DesiredFocusWidget;

protected:
	
	/**
	 * Spawns the InputComponent and binds any script delegates if bAutomaticallyRegisterInputOnConstruction is true.
	 * 
	 * bAutomaticallyRegisterInputOnConstruction is true when there are input event nodes in the graph
	 * of this widget, which would in turn need to be bound in order to receive their callbacks.
	 *
	 * This should only happen once per widget when it is spawned and initialized.
	 */
	UMG_API virtual void CreateInputComponent();

	/**
	 * Pushes the input component to the input stack and allows for any
	 * script delegates on this widget to get called.
	 */
	UMG_API virtual void StartProcessingInputScriptDelegates();

	/**
	 * Removes this widget's input component from the input stack,
	 * stopping any script delegates from being called.
	 * 
	 * This does not actually destroy or unbind any of the delegates yet, because the widget may be added 
	 * to the viewport again the future, and the binding process can be expensive.
	 */
	UMG_API virtual void StopProcessingInputScriptDelegates();

	/**
	* Unbinds all script delegates that this widget has bound to on its input component,
	* removes it from the input stack, and deletes the actual input component object.
	*/
	UMG_API virtual void DestroyInputComponent();
	
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<class UInputComponent> InputComponent;

	/**
	 * A callback bound to this widget's owning local player which will fire when the player controller has changed.
	 */
	void HandleOwningLocalPlayerControllerChanged(APlayerController* NewPlayerController);
	void StartListeningForPlayerControllerChanges();
	void StopListeningForPlayerControllerChanges();

protected:
	UPROPERTY(Transient, DuplicateTransient)
	TArray<FAnimationEventBinding> AnimationCallbacks;

private:
	static UMG_API void OnLatentActionsChanged(UObject* ObjectWhichChanged, ELatentActionChangeType ChangeType);

	/** The player context that is associated with this UI.  Think of this as the owner of the UI. */
	FLocalPlayerContext PlayerContext;

	/** Get World calls can be expensive for Widgets, we speed them up by caching the last found world until it goes away. */
	mutable TWeakObjectPtr<UWorld> CachedWorld;

	static UMG_API uint32& GetInitializingFromWidgetTree();

protected:

	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, ColorAndOpacity);
	PROPERTY_BINDING_IMPLEMENTATION(FSlateColor, ForegroundColor);

	/**
	 * The sequence player is a friend because we need to be alerted when animations start and finish without
	 * going through the normal event callbacks as people have a tendency to RemoveAll(this), which would permanently
	 * disable callbacks that are critical for UserWidget's base class - so rather we just directly report to the owning
	 * UserWidget of state transitions.
	 */
	friend UUMGSequencePlayer;
	friend FWidgetAnimationState;
	friend FWidgetAnimationHandle;

	friend UUMGSequenceTickManager;

	/** The compiler is a friend so that it can disable initialization from the widget tree */
	friend class FWidgetBlueprintCompilerContext;
};

#define LOCTEXT_NAMESPACE "UMG"

namespace CreateWidgetHelpers
{
	UMG_API bool ValidateUserWidgetClass(const UClass* UserWidgetClass);
}

DECLARE_CYCLE_STAT(TEXT("UserWidget Create"), STAT_CreateWidget, STATGROUP_Slate);

template <typename WidgetT = UUserWidget, typename OwnerType = UObject>
WidgetT* CreateWidget(OwnerType OwningObject, TSubclassOf<UUserWidget> UserWidgetClass = WidgetT::StaticClass(), FName WidgetName = NAME_None)
{
	static_assert(TIsDerivedFrom<WidgetT, UUserWidget>::IsDerived, "CreateWidget can only be used to create UserWidget instances. If creating a UWidget, use WidgetTree::ConstructWidget.");
	
	static_assert(TIsDerivedFrom<TPointedToType<OwnerType>, UWidget>::IsDerived
		|| TIsDerivedFrom<TPointedToType<OwnerType>, UWidgetTree>::IsDerived
		|| TIsDerivedFrom<TPointedToType<OwnerType>, APlayerController>::IsDerived
		|| TIsDerivedFrom<TPointedToType<OwnerType>, UGameInstance>::IsDerived
		|| TIsDerivedFrom<TPointedToType<OwnerType>, UWorld>::IsDerived, "The given OwningObject is not of a supported type for use with CreateWidget.");

	SCOPE_CYCLE_COUNTER(STAT_CreateWidget);
	FScopeCycleCounterUObject WidgetObjectCycleCounter(UserWidgetClass, GET_STATID(STAT_CreateWidget));

	if (OwningObject)
	{
		return Cast<WidgetT>(UUserWidget::CreateWidgetInstance(*OwningObject, UserWidgetClass, WidgetName));
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
