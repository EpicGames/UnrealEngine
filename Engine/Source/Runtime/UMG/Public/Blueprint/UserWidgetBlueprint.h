// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "UserWidgetBlueprint.generated.h"

UCLASS(Abstract, MinimalAPI)
class UUserWidgetBlueprint : public UBlueprint
{
	GENERATED_BODY()

public:
	/** Does the editor support widget from an editor package. */
	virtual bool AllowEditorWidget() const { return false; }

	virtual bool ShouldAutomaticallyRegisterInputOnConstruction() const override { return true; }
};
