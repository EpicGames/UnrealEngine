// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UserWidget.h"
#include "WidgetTree.h"
#include "Slate/SObjectWidget.h"

#include "UserWidgetPool.generated.h"

class APlayerController;

/**
 * Pools UUserWidget instances to minimize UObject and SWidget allocations for UMG elements with dynamic entries.
 *
 * Note that if underlying Slate instances are released when a UserWidget instance becomes inactive, NativeConstruct & NativeDestruct will be called when UUserWidget
 * instances are made active or inactive, respectively, provided the widget isn't actively referenced in the Slate hierarchy (i.e. if the shared reference count on the widget goes from/to 0).
 *
 * WARNING: Be sure to release the pool's Slate widgets within the owning widget's ReleaseSlateResources call to prevent leaking due to circular references
 *		Otherwise the cached references to SObjectWidgets will keep the UUserWidgets - and all that they reference - alive
 *
 * @see UListView
 * @see UDynamicEntryBox
 */
USTRUCT()
struct FUserWidgetPool
{
	GENERATED_BODY();

public:
	FUserWidgetPool() = default;
	UMG_API FUserWidgetPool(UWidget& InOwningWidget);
	UMG_API ~FUserWidgetPool();

	/** In the case that you don't have an owner widget, you should set a world to your pool, or it won't be able to construct widgets. */
	UMG_API void SetWorld(UWorld* OwningWorld);

	/**
	 * In the case a pool has no owner widget, a default player controller can optionally be specified allowing greater control  
	 * in split screen scenarios. The following priority rules are applied when determining the controller which should own created widgets:
	 * Controller of owning widget (if set) > Default player controller (if set) > First local player controller from world.
	 */
	UMG_API void SetDefaultPlayerController(APlayerController* InDefaultPlayerController);

	/** Triggers RebuildWidget on all currently active UserWidget instances */
	UMG_API void RebuildWidgets();

	/** Report any references to UObjects to the reference collector (only necessary if this is not already a UPROPERTY) */
	UMG_API void AddReferencedObjects(FReferenceCollector& Collector);

	bool IsInitialized() const { return OwningWidget.IsValid() || OwningWorld.IsValid(); }
	const TArray<UUserWidget*>& GetActiveWidgets() const { return ActiveWidgets; }

	/**
	 * Gets an instance of a widget of the given class.
	 * The underlying slate is stored automatically as well, so the returned widget is fully constructed and GetCachedWidget will return a valid SWidget.
	 */
	template <typename UserWidgetT = UUserWidget>
	UserWidgetT* GetOrCreateInstance(TSubclassOf<UserWidgetT> WidgetClass)
	{
		// Just make a normal SObjectWidget, same as would happen in TakeWidget
		return AddActiveWidgetInternal(WidgetClass,
			[] (UUserWidget* Widget, TSharedRef<SWidget> Content)
			{
				return SNew(SObjectWidget, Widget)[Content];
			});
	}

	using WidgetConstructFunc = TFunctionRef<TSharedPtr<SObjectWidget>(UUserWidget*, TSharedRef<SWidget>)>;

	/** Gets an instance of the widget this factory is for with a custom underlying SObjectWidget type */
	template <typename UserWidgetT = UUserWidget>
	UserWidgetT* GetOrCreateInstance(TSubclassOf<UserWidgetT> WidgetClass, WidgetConstructFunc ConstructWidgetFunc)
	{
		return AddActiveWidgetInternal(WidgetClass, ConstructWidgetFunc);
	}

	/** Return a widget object to the pool, allowing it to be reused in the future */
	UMG_API void Release(UUserWidget* Widget, bool bReleaseSlate = false);

	/** Return a widget object to the pool, allowing it to be reused in the future */
	UMG_API void Release(TArray<UUserWidget*> Widgets, bool bReleaseSlate = false);

	/** Returns all active widget objects to the inactive pool and optionally destroys all cached underlying slate widgets. */
	UMG_API void ReleaseAll(bool bReleaseSlate = false);

	/** Full reset of all created widget objects (and any cached underlying slate) */
	UMG_API void ResetPool();

	/** Reset of all cached underlying Slate widgets, only for inactive widgets in the pool. */
	UMG_API void ReleaseInactiveSlateResources();

	UE_DEPRECATED(4.24, "Use ReleaseAllSlateResources instead.")
	void ReleaseSlateResources() { ReleaseAllSlateResources(); }

	/** Reset of all cached underlying Slate widgets, but not the active UUserWidget objects */
	UMG_API void ReleaseAllSlateResources();

private:
	template <typename UserWidgetT = UUserWidget>
	UserWidgetT* AddActiveWidgetInternal(TSubclassOf<UserWidgetT> WidgetClass, WidgetConstructFunc ConstructWidgetFunc)
	{
		if (!ensure(IsInitialized()) || !WidgetClass)
		{
			return nullptr;
		}

		UUserWidget* WidgetInstance = nullptr;
		for (UUserWidget* InactiveWidget : InactiveWidgets)
		{
			if (InactiveWidget->GetClass() == WidgetClass)
			{
				WidgetInstance = InactiveWidget;
				InactiveWidgets.RemoveSingleSwap(InactiveWidget);
				break;
			}
		}

		UWidget* OwningWidgetPtr = OwningWidget.Get();
		if (!WidgetInstance)
		{
			if (OwningWidgetPtr)
			{
				WidgetInstance = CreateWidget(OwningWidgetPtr, WidgetClass);
			}
			else if (APlayerController* PlayerControllerPtr = DefaultPlayerController.Get())
			{
				WidgetInstance = CreateWidget(PlayerControllerPtr, WidgetClass);
			}
			else
			{
				WidgetInstance = CreateWidget(OwningWorld.Get(), WidgetClass);
			}
		}

		if (WidgetInstance)
		{
			ActiveWidgets.Add(WidgetInstance);
			
			// For pools owned by a widget, we never want to construct Slate widgets before the owning widget itself has built any Slate
			if (!OwningWidgetPtr || OwningWidgetPtr->GetCachedWidget().IsValid())
			{
				TSharedPtr<SWidget>& CachedSlateWidget = CachedSlateByWidgetObject.FindOrAdd(WidgetInstance);
				if (!CachedSlateWidget.IsValid())
				{
					CachedSlateWidget = WidgetInstance->TakeDerivedWidget(ConstructWidgetFunc);
				}
			}
		}

		return Cast<UserWidgetT>(WidgetInstance);
	}

	UPROPERTY(Transient)
	TArray<TObjectPtr<UUserWidget>> ActiveWidgets;
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<UUserWidget>> InactiveWidgets;

	TWeakObjectPtr<UWidget> OwningWidget;
	TWeakObjectPtr<UWorld> OwningWorld;
	TWeakObjectPtr<APlayerController> DefaultPlayerController;
	TMap<UUserWidget*, TSharedPtr<SWidget>> CachedSlateByWidgetObject;
};
