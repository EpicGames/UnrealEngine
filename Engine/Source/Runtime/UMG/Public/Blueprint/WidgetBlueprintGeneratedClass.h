// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Binding/DynamicPropertyPath.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "FieldNotificationId.h"

#include "WidgetBlueprintGeneratedClass.generated.h"

class FAssetRegistryTagsContext;
class UWidget;
class UUserWidget;
class UWidgetAnimation;
class UWidgetBlueprintGeneratedClass;
class UWidgetBlueprintGeneratedClassExtension;
class UWidgetTree;

UENUM()
enum class EBindingKind : uint8
{
	Function,
	Property
};

USTRUCT()
struct FDelegateRuntimeBinding
{
	GENERATED_USTRUCT_BODY()

	/** The widget that will be bound to the live data. */
	UPROPERTY()
	FString ObjectName;

	/** The property on the widget that will have a binding placed on it. */
	UPROPERTY()
	FName PropertyName;

	/** The function or property we're binding to on the source object. */
	UPROPERTY()
	FName FunctionName;

	/**  */
	UPROPERTY()
	FDynamicPropertyPath SourcePath;

	/** The kind of binding we're performing, are we binding to a property or a function. */
	UPROPERTY()
	EBindingKind Kind = EBindingKind::Property;
};


#if WITH_EDITOR
class FWidgetBlueprintGeneratedClassDelegates
{
public:
	// delegate for generating widget asset registry tags.
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FGetAssetTagsWithContext, const UWidgetBlueprintGeneratedClass*, FAssetRegistryTagsContext);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FGetAssetTags, const UWidgetBlueprintGeneratedClass*, TArray<UObject::FAssetRegistryTag>&);
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FCollectSaveOverrides, const UWidgetBlueprintGeneratedClass*, FObjectCollectSaveOverridesContext);

	// called by UWidgetBlueprintGeneratedClass::GetAssetRegistryTags()
	static UMG_API FGetAssetTagsWithContext GetAssetTagsWithContext;
	// called by UWidgetBlueprintGeneratedClass::CollectSaveOverrides()
	static UMG_API FCollectSaveOverrides CollectSaveOverrides;
	UE_DEPRECATED(5.4, "Subscribe to GetAssetTagsWithContext instead.")
	static UMG_API FGetAssetTags GetAssetTags;
};
#endif

/**
 * The widget blueprint generated class allows us to create blueprint-able widgets for UMG at runtime.
 * All WBPGC's are of UUserWidget classes, and they perform special post initialization using this class
 * to give themselves many of the same capabilities as AActor blueprints, like dynamic delegate binding for
 * widgets.
 */
UCLASS(MinimalAPI)
class UWidgetBlueprintGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()
	friend class FWidgetBlueprintCompilerContext;

public:
	UMG_API UWidgetBlueprintGeneratedClass();

private:

	/** A tree of the widget templates to be created */
	UPROPERTY(DuplicateTransient)
	TObjectPtr<UWidgetTree> WidgetTree;
	
	/** The extension that are considered static to the class */
	UPROPERTY()
	TArray<TObjectPtr<UWidgetBlueprintGeneratedClassExtension>> Extensions;

	/** The classes native parent requires a native tick */
	UPROPERTY()
	uint32 bClassRequiresNativeTick :1;

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(Transient)
	uint32 bCanCallPreConstruct : 1;
#endif

public:
	/**
	 * Determines whether this widget blueprint can be initialized without
	 * a valid player context (PlayerController, etc.).
	 * Required to be true for use with UMG Widget Preview.
	 */
	UPROPERTY()
	uint32 bCanCallInitializedWithoutPlayerContext : 1;

	UPROPERTY()
	TArray< FDelegateRuntimeBinding > Bindings;

	UPROPERTY()
	TArray< TObjectPtr<UWidgetAnimation> > Animations;

	/**
	 * All named slots, even the ones that have content already filled into them by a parent class and are not
	 * available for extension.
	 **/
	UPROPERTY()
	TArray<FName> NamedSlots;

#if WITH_EDITORONLY_DATA
	/** All named slots mapped the assigned GUID of their UNamedSlot widget. **/
	UPROPERTY()
	TMap<FName, FGuid> NamedSlotsWithID;

	UPROPERTY(Transient)
	TSet<FName> NamedSlotsWithContentInSameTree;

	UPROPERTY(Transient)
	TSet<FName> NameClashingInHierarchy;
#endif

	/**
	 * Available Named Slots for content in a subclass.  These are slots that are accumulated from all super
	 * classes on compile.  They will exclude any named slots that are filled by a parent class.
	 **/
	UPROPERTY(AssetRegistrySearchable)
	TArray<FName> AvailableNamedSlots;

	/**
	 * These are the set of named slots that can be used on an instance of the widget.  This set is slightly
	 * different from available named slots, because ones designated UNamedSlot::bExposeOnInstanceOnly == true
	 * will also be in this list, even though they wont be in AvailableNamedSlots, if are inherited, as inherited
	 * named slots do not have the capability to remove existing content in a named slot.
	 **/
	UPROPERTY()
	TArray<FName> InstanceNamedSlots;
	
public:
	UWidgetTree* GetWidgetTreeArchetype() const { return WidgetTree; }
	UMG_API void SetWidgetTreeArchetype(UWidgetTree* InWidgetTree);

	UMG_API void GetNamedSlotArchetypeContent(TFunctionRef<void(FName /*SlotName*/, UWidget* /*Content*/)> Predicate) const;

	// Walks up the hierarchy looking for a valid widget tree.
	UMG_API UWidgetBlueprintGeneratedClass* FindWidgetTreeOwningClass() const;

	//~ Begin UObject interface
	UMG_API virtual void Serialize(FArchive& Ar) override;
	UMG_API virtual void PostLoad() override;
	UMG_API virtual bool NeedsLoadForServer() const override;
#if WITH_EDITOR
	UMG_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UMG_API virtual void CollectSaveOverrides(FObjectCollectSaveOverridesContext SaveContext) override;
#endif
	//~ End UObject interface

	UMG_API virtual void PurgeClass(bool bRecompilingOnLoad) override;

	/**
	 * This is the function that makes UMG work.  Once a user widget is constructed, it will post load
	 * call into its generated class and ask to be initialized.  The class will perform all the delegate
	 * binding and wiring necessary to have the user's widget perform as desired.
	 */
	UMG_API void InitializeWidget(UUserWidget* UserWidget) const;

	static UMG_API void InitializeWidgetStatic(UUserWidget* UserWidget
		, const UClass* InClass
		, UWidgetTree* InWidgetTree
		, const UClass* InWidgetTreeWidgetClass
		, const TArrayView<UWidgetAnimation*> InAnimations
		, const TArrayView<const FDelegateRuntimeBinding> InBindings);

	bool ClassRequiresNativeTick() const { return bClassRequiresNativeTick; }

#if WITH_EDITOR
	UMG_API void SetClassRequiresNativeTick(bool InClassRequiresNativeTick);
#endif

	/** Find the first extension of the requested type. */
	template<typename ExtensionType>
	ExtensionType* GetExtension(bool bIncludeSuper = true)
	{
		return Cast<ExtensionType>(GetExtension(ExtensionType::StaticClass(), bIncludeSuper));
	}

	/** Find the first extension of the requested type. */
	UMG_API UWidgetBlueprintGeneratedClassExtension* GetExtension(TSubclassOf<UWidgetBlueprintGeneratedClassExtension> InExtensionType, bool bIncludeSuper = true);

	/** Find the extensions of the requested type. */
	UMG_API TArray<UWidgetBlueprintGeneratedClassExtension*> GetExtensions(TSubclassOf<UWidgetBlueprintGeneratedClassExtension> InExtensionType, bool bIncludeSuper = true);

	template<typename Predicate>
	void ForEachExtension(Predicate Pred, bool bIncludeSuper = true) const
	{
		for (UWidgetBlueprintGeneratedClassExtension* Extension : Extensions)
		{
			check(Extension);
			Pred(Extension);
		}
		if (bIncludeSuper)
		{
			if (UWidgetBlueprintGeneratedClass* ParentClass = Cast<UWidgetBlueprintGeneratedClass>(GetSuperClass()))
			{
				ParentClass->ForEachExtension(MoveTemp(Pred));
			}
		}
	}

private:
	static UMG_API void InitializeBindingsStatic(UUserWidget* UserWidget, const TArrayView<const FDelegateRuntimeBinding> InBindings, const TMap<FName, FObjectPropertyBase*>& InPropertyMap);
	static UMG_API void BindAnimationsStatic(UUserWidget* Instance, const TArrayView<UWidgetAnimation*> InAnimations, const TMap<FName, FObjectPropertyBase*>& InPropertyMap);

	UMG_API void GetExtensions(TArray<UWidgetBlueprintGeneratedClassExtension*>& OutExtensions, TSubclassOf<UWidgetBlueprintGeneratedClassExtension> InExtensionType, bool bIncludeSuper);
};
