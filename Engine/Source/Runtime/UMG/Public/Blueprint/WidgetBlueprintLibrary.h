// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "InputCoreTypes.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/Interface.h"
#include "Input/Events.h"
#include "Styling/SlateBrush.h"
#include "Components/SlateWrapperTypes.h"
#include "Blueprint/DragDropOperation.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "WidgetBlueprintLibrary.generated.h"

class UFont;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class USlateBrushAsset;
class UTexture2D;
class UUserWidget;

UCLASS(meta=(ScriptName="WidgetLibrary"), MinimalAPI)
class UWidgetBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:

	/** Creates a widget */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, meta=( WorldContext="WorldContextObject", DisplayName="Create Widget", BlueprintInternalUseOnly="true" ), Category="Widget")
	static UMG_API class UUserWidget* Create(UObject* WorldContextObject, TSubclassOf<class UUserWidget> WidgetType, APlayerController* OwningPlayer);

	/**
	 * Creates a new drag and drop operation that can be returned from a drag begin to inform the UI what i
	 * being dragged and dropped and what it looks like.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Drag and Drop", meta=( BlueprintInternalUseOnly="true" ))
	static UMG_API UDragDropOperation* CreateDragDropOperation(TSubclassOf<UDragDropOperation> OperationClass);

	/**
	 * Setup an input mode that allows only the UI to respond to user input.
	 * 
	 * Note: This means that any bound Input Events in the widget will not be called!
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Input", meta=(DisplayName="Set Input Mode UI Only"))
	static UMG_API void SetInputMode_UIOnlyEx(APlayerController* PlayerController, UWidget* InWidgetToFocus = nullptr, EMouseLockMode InMouseLockMode = EMouseLockMode::DoNotLock, const bool bFlushInput = false);
	/**
	 * Setup an input mode that allows only the UI to respond to user input, and if the UI doesn't handle it player input / player controller gets a chance.
	 * 
	 * Note: This means that any bound Input events in the widget will be called.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Input", meta = (DisplayName = "Set Input Mode Game And UI"))
	static UMG_API void SetInputMode_GameAndUIEx(APlayerController* PlayerController, UWidget* InWidgetToFocus = nullptr, EMouseLockMode InMouseLockMode = EMouseLockMode::DoNotLock, bool bHideCursorDuringCapture = true, const bool bFlushInput = false);

	/**
	 * Setup an input mode that allows only player input / player controller to respond to user input.
	 * 
	 * Note: Any bound Input Events in this widget will be called.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Input")
	static UMG_API void SetInputMode_GameOnly(APlayerController* PlayerController, const bool bFlushInput = false);

	/** */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Focus")
	static UMG_API void SetFocusToGameViewport();

	/** Draws a box */
	UFUNCTION(BlueprintCallable, Category="Painting")
	static UMG_API void DrawBox(UPARAM(ref) FPaintContext& Context, FVector2D Position, FVector2D Size, USlateBrushAsset* Brush, FLinearColor Tint = FLinearColor::White);

	/**
	 * Draws a hermite spline.
	 *
	 * @param Start			Starting position of the spline in local space.
	 * @param StartDir		The direction of the spline from the start point.
	 * @param End			Ending position of the spline in local space.
	 * @param EndDir		The direction of the spline to the end point.
	 * @param Tint			Color to render the spline.
	 * @param Thickness		How many pixels thick this spline should be.
	 */
	UFUNCTION(BlueprintCallable, meta = (AdvancedDisplay = "6"), Category = "Painting")
	static UMG_API void DrawSpline(UPARAM(ref) FPaintContext& Context, FVector2D Start, FVector2D StartDir, FVector2D End, FVector2D EndDir, FLinearColor Tint = FLinearColor::White, float Thickness = 1.0f);

	/**
	 * Draws a line.
	 *
	 * @param PositionA		Starting position of the line in local space.
	 * @param PositionB		Ending position of the line in local space.
	 * @param Tint			Color to render the line.
	 * @param bAntialias	Whether the line should be antialiased.
	 * @param Thickness		How many pixels thick this line should be.
	 */
	UFUNCTION(BlueprintCallable, meta = (AdvancedDisplay = "4"), Category = "Painting")
	static UMG_API void DrawLine(UPARAM(ref) FPaintContext& Context, FVector2D PositionA, FVector2D PositionB, FLinearColor Tint = FLinearColor::White, bool bAntiAlias = true, float Thickness = 1.0f);

	/**
	 * Draws several line segments.
	 *
	 * @param Points		Line pairs, each line needs to be 2 separate points in the array.
	 * @param Tint			Color to render the line.
	 * @param bAntialias	Whether the line should be antialiased.
	 * @param Thickness		How many pixels thick this line should be.
	 */
	UFUNCTION(BlueprintCallable, meta = (AdvancedDisplay = "3"), Category = "Painting")
	static UMG_API void DrawLines(UPARAM(ref) FPaintContext& Context, const TArray<FVector2D>& Points, FLinearColor Tint = FLinearColor::White, bool bAntiAlias = true, float Thickness = 1.0f);

	/**
	 * Draws text.
	 *
	 * @param InString		The string to draw.
	 * @param Position		The starting position where the text is drawn in local space.
	 * @param Tint			Color to render the line.
	 */
	UFUNCTION(BlueprintCallable, Category="Painting", meta=( DeprecatedFunction, DeprecationMessage = "Use Draw Text instead", DisplayName="Draw String"))
	static UMG_API void DrawText(UPARAM(ref) FPaintContext& Context, const FString& InString, FVector2D Position, FLinearColor Tint = FLinearColor::White);

	/**
	 * Draws text.
	 *
	 * @param Text			The string to draw.
	 * @param Position		The starting position where the text is drawn in local space.
	 * @param Tint			Color to render the line.
	 */
	UFUNCTION(BlueprintCallable, Category="Painting", meta=(DisplayName="Draw Text"))
	static UMG_API void DrawTextFormatted(UPARAM(ref) FPaintContext& Context, const FText& Text, FVector2D Position, UFont* Font, float FontSize = 16.0f, FName FontTypeFace = FName("Regular"), FLinearColor Tint = FLinearColor::White);

	/**
	 * The event reply to use when you choose to handle an event.  This will prevent the event 
	 * from continuing to bubble up / down the widget hierarchy.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Event Reply")
	static UMG_API FEventReply Handled();

	/** The event reply to use when you choose not to handle an event. */
	UFUNCTION(BlueprintPure, Category="Widget|Event Reply")
	static UMG_API FEventReply Unhandled();

	/**  */
	UFUNCTION(BlueprintPure, meta=( DefaultToSelf="CapturingWidget" ), Category="Widget|Event Reply")
	static UMG_API FEventReply CaptureMouse(UPARAM(ref) FEventReply& Reply, UWidget* CapturingWidget);

	/**  */
	UFUNCTION(BlueprintPure, Category="Widget|Event Reply")
	static UMG_API FEventReply ReleaseMouseCapture(UPARAM(ref) FEventReply& Reply);

	UFUNCTION( BlueprintPure, Category = "Widget|Event Reply", meta = ( HidePin = "CapturingWidget", DefaultToSelf = "CapturingWidget" ) )
	static UMG_API FEventReply LockMouse( UPARAM( ref ) FEventReply& Reply, UWidget* CapturingWidget );

	UFUNCTION( BlueprintPure, Category = "Widget|Event Reply" )
	static UMG_API FEventReply UnlockMouse( UPARAM( ref ) FEventReply& Reply );

	/**  */
	UFUNCTION(BlueprintPure, meta= (HidePin="CapturingWidget", DefaultToSelf="CapturingWidget"), Category="Widget|Event Reply")
	static UMG_API FEventReply SetUserFocus(UPARAM(ref) FEventReply& Reply, UWidget* FocusWidget, bool bInAllUsers = false);

	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "Use SetUserFocus() instead"), Category = "Widget|Event Reply")
	static UMG_API FEventReply CaptureJoystick(UPARAM(ref) FEventReply& Reply, UWidget* CapturingWidget, bool bInAllJoysticks = false);

	/**  */
	UFUNCTION(BlueprintPure, meta = (HidePin = "CapturingWidget", DefaultToSelf = "CapturingWidget"), Category = "Widget|Event Reply")
	static UMG_API FEventReply ClearUserFocus(UPARAM(ref) FEventReply& Reply, bool bInAllUsers = false);

	UFUNCTION(BlueprintPure, meta = (DeprecatedFunction, DeprecationMessage = "Use ClearUserFocus() instead"), Category = "Widget|Event Reply")
	static UMG_API FEventReply ReleaseJoystickCapture(UPARAM(ref) FEventReply& Reply, bool bInAllJoysticks = false);

	/**  */
	UFUNCTION(BlueprintPure, Category="Widget|Event Reply")
	static UMG_API FEventReply SetMousePosition(UPARAM(ref) FEventReply& Reply, FVector2D NewMousePosition);

	/**
	 * Ask Slate to detect if a user starts dragging in this widget later.  Slate internally tracks the movement
	 * and if it surpasses the drag threshold, Slate will send an OnDragDetected event to the widget.
	 *
	 * @param WidgetDetectingDrag  Detect dragging in this widget
	 * @param DragKey		       This button should be pressed to detect the drag
	 */
	UFUNCTION(BlueprintPure, meta=( HidePin="WidgetDetectingDrag", DefaultToSelf="WidgetDetectingDrag" ), Category="Widget|Drag and Drop|Event Reply")
	static UMG_API FEventReply DetectDrag(UPARAM(ref) FEventReply& Reply, UWidget* WidgetDetectingDrag, FKey DragKey);

	/**
	 * Given the pointer event, emit the DetectDrag reply if the provided key was pressed.
	 * If the DragKey is a touch key, that will also automatically work.
	 * @param PointerEvent	The pointer device event coming in.
	 * @param WidgetDetectingDrag  Detect dragging in this widget.
	 * @param DragKey		       This button should be pressed to detect the drag, won't emit the DetectDrag FEventReply unless this is pressed.
	 */
	UFUNCTION(BlueprintCallable, meta=( HidePin="WidgetDetectingDrag", DefaultToSelf="WidgetDetectingDrag" ), Category="Widget|Drag and Drop|Event Reply")
	static UMG_API FEventReply DetectDragIfPressed(const FPointerEvent& PointerEvent, UWidget* WidgetDetectingDrag, FKey DragKey);

	/**
	 * An event should return FReply::Handled().EndDragDrop() to request that the current drag/drop operation be terminated.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Drag and Drop|Event Reply")
	static UMG_API FEventReply EndDragDrop(UPARAM(ref) FEventReply& Reply);

	/**
	 * Returns true if a drag/drop event is occurring that a widget can handle.
	 */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Widget|Drag and Drop")
	static UMG_API bool IsDragDropping();

	/**
	 * Returns the drag and drop operation that is currently occurring if any, otherwise nothing.
	 */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Widget|Drag and Drop")
	static UMG_API UDragDropOperation* GetDragDroppingContent();

	/**
	 * Cancels any current drag drop operation.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Drag and Drop")
	static UMG_API void CancelDragDrop();

	/**
	 * Creates a Slate Brush from a Slate Brush Asset
	 *
	 * @return A new slate brush using the asset's brush.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API FSlateBrush MakeBrushFromAsset(USlateBrushAsset* BrushAsset);

	/** 
	 * Creates a Slate Brush from a Texture2D
	 *
	 * @param Width  When less than or equal to zero, the Width of the brush will default to the Width of the Texture
	 * @param Height  When less than or equal to zero, the Height of the brush will default to the Height of the Texture
	 *
	 * @return A new slate brush using the texture.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API FSlateBrush MakeBrushFromTexture(UTexture2D* Texture, int32 Width = 0, int32 Height = 0);

	/**
	 * Creates a Slate Brush from a Material.  Materials don't have an implicit size, so providing a widget and height
	 * is required to hint slate with how large the image wants to be by default.
	 *
	 * @return A new slate brush using the material.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API FSlateBrush MakeBrushFromMaterial(UMaterialInterface* Material, int32 Width = 32, int32 Height = 32);

	/**
	 * Gets the resource object on a brush.  This could be a UTexture2D or a UMaterialInterface.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API UObject* GetBrushResource(const FSlateBrush& Brush);

	/**
	 * Gets the brush resource as a texture 2D.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API UTexture2D* GetBrushResourceAsTexture2D(const FSlateBrush& Brush);

	/**
	 * Gets the brush resource as a material.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API UMaterialInterface* GetBrushResourceAsMaterial(const FSlateBrush& Brush);

	/**
	 * Sets the resource on a brush to be a UTexture2D.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Brush")
	static UMG_API void SetBrushResourceToTexture(UPARAM(ref) FSlateBrush& Brush, UTexture2D* Texture);

	/**
	 * Sets the resource on a brush to be a Material.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Brush")
	static UMG_API void SetBrushResourceToMaterial(UPARAM(ref) FSlateBrush& Brush, UMaterialInterface* Material);

	/**
	 * Creates a Slate Brush that wont draw anything, the "Null Brush".
	 *
	 * @return A new slate brush that wont draw anything.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API FSlateBrush NoResourceBrush();

	/**
	 * Gets the material that allows changes to parameters at runtime.  The brush must already have a material assigned to it, 
	 * if it does it will automatically be converted to a MID.
	 *
	 * @return A material that supports dynamic input from the game.
	 */
	UFUNCTION(BlueprintPure, Category="Widget|Brush")
	static UMG_API UMaterialInstanceDynamic* GetDynamicMaterial(UPARAM(ref) FSlateBrush& Brush);

	/** Closes any popup menu */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Widget|Menu")
	static UMG_API void DismissAllMenus();

	/**
	 * Find all widgets of a certain class.
	 * @param FoundWidgets The widgets that were found matching the filter.
	 * @param WidgetClass The widget class to filter by.
	 * @param TopLevelOnly Only the widgets that are direct children of the viewport will be returned.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Widget", meta=( WorldContext="WorldContextObject", DeterminesOutputType = "WidgetClass", DynamicOutputParam = "FoundWidgets" ))
	static UMG_API void GetAllWidgetsOfClass(const UObject* WorldContextObject, TArray<UUserWidget*>& FoundWidgets, TSubclassOf<UUserWidget> WidgetClass, bool TopLevelOnly = true);

	/**
	* Find all widgets in the world with the specified interface.
	* This is a slow operation, use with caution e.g. do not use every frame.
	* @param Interface The interface to find. Must be specified or result array will be empty.
	* @param FoundWidgets Output array of widgets that implement the specified interface.
	* @param TopLevelOnly Only the widgets that are direct children of the viewport will be returned.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Widget", meta = (WorldContext = "WorldContextObject", DeterminesOutputType = "WidgetClass", DynamicOutputParam = "FoundWidgets"))
	static UMG_API void GetAllWidgetsWithInterface(const UObject* WorldContextObject, TArray<UUserWidget*>& FoundWidgets, TSubclassOf<UInterface> Interface, bool TopLevelOnly);

	UFUNCTION(BlueprintPure, Category="Widget", meta = (CompactNodeTitle = "->", BlueprintAutocast))
	static UMG_API FInputEvent GetInputEventFromKeyEvent(const FKeyEvent& Event);

	UFUNCTION(BlueprintPure, Category="Widget", meta = (CompactNodeTitle = "->", BlueprintAutocast))
	static UMG_API FKeyEvent GetKeyEventFromAnalogInputEvent(const FAnalogInputEvent& Event);

	UFUNCTION(BlueprintPure, Category="Widget", meta = ( CompactNodeTitle = "->", BlueprintAutocast ))
	static UMG_API FInputEvent GetInputEventFromCharacterEvent(const FCharacterEvent& Event);

	UFUNCTION(BlueprintPure, Category="Widget", meta = ( CompactNodeTitle = "->", BlueprintAutocast ))
	static UMG_API FInputEvent GetInputEventFromPointerEvent(const FPointerEvent& Event);

	UFUNCTION(BlueprintPure, Category="Widget", meta = ( CompactNodeTitle = "->", BlueprintAutocast ))
	static UMG_API FInputEvent GetInputEventFromNavigationEvent(const FNavigationEvent& Event);

	/**
	 * Gets the amount of padding that needs to be added when accounting for the safe zone on TVs.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Safe Zone", meta=( WorldContext="WorldContextObject" ))
	static UMG_API void GetSafeZonePadding(const UObject* WorldContextObject, FVector4& SafePadding, FVector2D& SafePaddingScale, FVector4& SpillOverPadding);

	/**
	* Apply color deficiency correction settings to the game window 
	* @param Type The type of color deficiency correction to apply.
	* @param Severity Intensity of the color deficiency correction effect, from 0 to 1.
	* @param CorrectDeficiency Shifts the color spectrum to the visible range based on the current deficiency type.
	* @param ShowCorrectionWithDeficiency If you're correcting the color deficiency, you can use this to visualize what the correction looks like with the deficiency.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Widget|Accessibility", meta = (AdvancedDisplay = "3"))
	static UMG_API void SetColorVisionDeficiencyType(EColorVisionDeficiency Type, float Severity, bool CorrectDeficiency, bool ShowCorrectionWithDeficiency);

	/**
	 * Loads or sets a hardware cursor from the content directory in the game.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Hardware Cursor", meta=( WorldContext="WorldContextObject" ))
	static UMG_API bool SetHardwareCursor(const UObject* WorldContextObject, EMouseCursor::Type CursorShape, FName CursorName, FVector2D HotSpot);

	UFUNCTION(BlueprintCallable, Category = "Widget|Window Title Bar")
	static UMG_API void SetWindowTitleBarState(UWidget* TitleBarContent, EWindowTitleBarMode Mode, bool bTitleBarDragEnabled, bool bWindowButtonsVisible, bool bTitleBarVisible);

	UFUNCTION(BlueprintCallable, Category = "Widget|Window Title Bar")
	static UMG_API void RestorePreviousWindowTitleBarState();

	DECLARE_DYNAMIC_DELEGATE(FOnGameWindowCloseButtonClickedDelegate);

	UFUNCTION(BlueprintCallable, Category = "Widget|Window Title Bar")
	static UMG_API void SetWindowTitleBarOnCloseClickedDelegate(FOnGameWindowCloseButtonClickedDelegate Delegate);

	UFUNCTION(BlueprintCallable, Category = "Widget|Window Title Bar")
	static UMG_API void SetWindowTitleBarCloseButtonActive(bool bActive);
};
