// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "WidgetChild.generated.h"

class UWidget;

/**
 * Represent a Widget present in the Tree Widget of the UserWidget
 */
USTRUCT()
struct FWidgetChild
{
	GENERATED_BODY();
public:
	UMG_API FWidgetChild();
	UMG_API FWidgetChild(const class UUserWidget* Outer, FName InChildName);

	FName GetFName() const
	{
		return WidgetName;
	};

	UWidget* GetWidget() const
	{ 
		return WidgetPtr.Get(); 
	};

	/** Resolves the Widget ptr using it's name. */
	UMG_API UWidget* Resolve(const class UWidgetTree* WidgetTree);

private:
	/** This either the widget to focus, OR the name of the function to call. */
	UPROPERTY(EditAnywhere, Category = "Interaction")
	FName WidgetName;

	UPROPERTY(Transient)
	TWeakObjectPtr<UWidget> WidgetPtr;
};
