// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Layout/Geometry.h"
#include "WidgetLayoutLibrary.generated.h"

/**
 * Design-time preview information from the UMG Designer.
 * Contains the current designer preview dimensions and derived orientation.
 * Only populated at design time — all values are zeroed/false at runtime.
 */
USTRUCT(BlueprintType, MinimalAPI)
struct FWidgetDesignTimeInfo
{
	GENERATED_BODY()

	/** The current preview dimensions in the UMG Designer (width x height). */
	UPROPERTY(BlueprintReadOnly, Category = "Design Time")
	FVector2D PreviewSize = FVector2D::ZeroVector;

	/** True when the designer preview is in portrait orientation (height > width). */
	UPROPERTY(BlueprintReadOnly, Category = "Design Time")
	bool bIsPortrait = false;
};

class APlayerController;
class UGameViewportClient;
class UCanvasPanelSlot;
class UGridSlot;
class UHorizontalBoxSlot;
class UOverlaySlot;
class UUniformGridSlot;
class UVerticalBoxSlot;
class UScrollBoxSlot;
class USafeZoneSlot;
class UScaleBoxSlot;
class USizeBoxSlot;
class UStackBoxSlot;
class UWrapBoxSlot;
class UWidgetSwitcherSlot;
class UWidget;

UCLASS(MinimalAPI)
class UWidgetLayoutLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Gets the projected world to screen position for a player, then converts it into a widget
	 * position, which takes into account any quality scaling.
	 * @param PlayerController The player controller to project the position in the world to their screen.
	 * @param WorldLocation The world location to project from.
	 * @param ScreenPosition The position in the viewport with quality scale removed and DPI scale remove.
	 * @param bPlayerViewportRelative Should this be relative to the player viewport subregion (useful when using player attached widgets in split screen or when aspect-ratio constrained)
	 * @return true if the position projects onto the screen.
	 */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Viewport")
	static UMG_API bool ProjectWorldLocationToWidgetPosition(const APlayerController* PlayerController, FVector WorldLocation, FVector2D& ScreenPosition, bool bPlayerViewportRelative);

	/**
	 * Convert a World Space 3D position into a 2D Widget Screen Space position, with distance from the camera the Z component.  This
	 * takes into account any quality scaling as well.
	 *
	 * @return true if the world coordinate was successfully projected to the screen.
	 */
	static UMG_API bool ProjectWorldLocationToWidgetPositionWithDistance(const APlayerController* PlayerController, FVector WorldLocation, FVector& ScreenPosition, bool bPlayerViewportRelative);

	/**
	 * Gets the current DPI Scale being applied to the viewport and all the Widgets.
	 */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Viewport", meta=( WorldContext="WorldContextObject" ))
	static UMG_API float GetViewportScale(const UObject* WorldContextObject);

	/**
	 * Gets the current DPI Scale being applied to the viewport and all the Widgets.
	 */
	static UMG_API float GetViewportScale(const UGameViewportClient* ViewportClient);

	/**
	 * Gets the size of the game viewport.
	 */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Viewport", meta=( WorldContext="WorldContextObject", Keywords = "screen size"))
	static UMG_API FVector2D GetViewportSize(const UObject* WorldContextObject);

	/**
	 * Gets the geometry of the widget holding all widgets added to the "Viewport".  You
	 * can use this geometry to convert between absolute and local space of widgets held on this
	 * widget.
	 */
	UFUNCTION(BlueprintCallable, Category="Viewport", meta=( WorldContext="WorldContextObject" ))
	static UMG_API FGeometry GetViewportWidgetGeometry(const UObject* WorldContextObject);

	/**
	 * Gets the geometry of the widget holding all widgets added to the "Player Screen". You
	 * can use this geometry to convert between absolute and local space of widgets held on this
	 * widget.
	 */
	UFUNCTION(BlueprintCallable, Category="Viewport")
	static UMG_API FGeometry GetPlayerScreenWidgetGeometry(APlayerController* PlayerController);

	/**
	 * Gets the platform's mouse cursor position.  This is the 'absolute' desktop location of the mouse.
	 */
	UFUNCTION(BlueprintCallable, Category="Viewport")
	static UMG_API FVector2D GetMousePositionOnPlatform();

	/**
	 * Gets the platform's mouse cursor position in the local space of the viewport widget.
	 */
	UFUNCTION(BlueprintCallable, Category="Viewport", meta=( WorldContext="WorldContextObject" ))
	static UMG_API FVector2D GetMousePositionOnViewport(const UObject* WorldContextObject);

	/**
	 * Gets the mouse position of the player controller, scaled by the DPI.  If you're trying to go from raw mouse screenspace coordinates
	 * to fullscreen widget space, you'll need to transform the mouse into DPI Scaled space.  This function performs that scaling.
	 *
	 * MousePositionScaledByDPI = MousePosition * (1 / ViewportScale).
	 */
	//UE_DEPRECATED(4.17, "Use GetMousePositionOnViewport() instead.  Optionally and for more options, you can use GetViewportWidgetGeometry and GetPlayerScreenWidgetGeometry are newly introduced to give you the geometry of the viewport and the player screen for widgets to help convert between spaces.")
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category="Viewport")
	static UMG_API bool GetMousePositionScaledByDPI(const APlayerController* Player, float& LocationX, float& LocationY);
	static UMG_API bool GetMousePositionScaledByDPI(const APlayerController* Player, double& LocationX, double& LocationY);	// LWC_TODO: Temp stand in for native calls with FVector2D components.

	/**
	* Gets the slot object on the child widget as a Border Slot, allowing you to manipulate layout information.
	* @param Widget The child widget of a border panel.
	*/
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API class UBorderSlot* SlotAsBorderSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Canvas Slot, allowing you to manipulate layout information.
	 * @param Widget The child widget of a canvas panel.
	 */
	UFUNCTION(BlueprintPure, Category="Slot")
	static UMG_API UCanvasPanelSlot* SlotAsCanvasSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Grid Slot, allowing you to manipulate layout information.
	 * @param Widget The child widget of a grid panel.
	 */
	UFUNCTION(BlueprintPure, Category="Slot")
	static UMG_API UGridSlot* SlotAsGridSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Horizontal Box Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Horizontal Box.
	 */
	UFUNCTION(BlueprintPure, Category="Slot")
	static UMG_API UHorizontalBoxSlot* SlotAsHorizontalBoxSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Overlay Slot, allowing you to manipulate layout information.
	 * @param Widget The child widget of a overlay panel.
	 */
	UFUNCTION(BlueprintPure, Category="Slot")
	static UMG_API UOverlaySlot* SlotAsOverlaySlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Uniform Grid Slot, allowing you to manipulate layout information.
	 * @param Widget The child widget of a uniform grid panel.
	 */
	UFUNCTION(BlueprintPure, Category="Slot")
	static UMG_API UUniformGridSlot* SlotAsUniformGridSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Vertical Box Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Vertical Box.
	 */
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API UVerticalBoxSlot* SlotAsVerticalBoxSlot(UWidget* Widget);


	/**
	* Gets the slot object on the child widget as a Scroll Box Slot, allowing you to manipulate its information.
	* @param Widget The child widget of a Scroll Box.
	*/
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API UScrollBoxSlot* SlotAsScrollBoxSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Safe Box Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Safe Box.
	 */
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API USafeZoneSlot* SlotAsSafeBoxSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Scale Box Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Scale Box.
	 */
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API UScaleBoxSlot* SlotAsScaleBoxSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Size Box Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Size Box.
	 */
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API USizeBoxSlot* SlotAsSizeBoxSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Stack Box Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Stack Box.
	 */
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API UStackBoxSlot* SlotAsStackBoxSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Wrap Box Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Wrap Box.
	 */
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API UWrapBoxSlot* SlotAsWrapBoxSlot(UWidget* Widget);

	/**
	 * Gets the slot object on the child widget as a Widget Switcher Slot, allowing you to manipulate its information.
	 * @param Widget The child widget of a Widget Switcher Slot.
	 */
	UFUNCTION(BlueprintPure, Category = "Slot")
	static UMG_API UWidgetSwitcherSlot* SlotAsWidgetSwitcherSlot(UWidget* Widget);

	/**
	 * Removes all widgets from the viewport.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category="Viewport", meta=( WorldContext="WorldContextObject" ))
	static UMG_API void RemoveAllWidgets(const UObject* WorldContextObject);

	/**
	 * Returns design-time preview information from the UMG Designer.
	 * Use this in PreConstruct to adapt widget layout to the designer's current preview
	 * dimensions (e.g., switching between portrait and landscape layouts).
	 *
	 * Logs an error and returns zeroed values if called outside of a design-time preview
	 * (e.g., at runtime or from a non-preview context). Use viewport/display system
	 * queries instead for runtime orientation.
	 *
	 * @param Widget	The calling widget (auto-filled via DefaultToSelf).
	 */
	UFUNCTION(BlueprintPure, Category = "Design Time", meta = (DefaultToSelf = "Widget"))
	static UMG_API FWidgetDesignTimeInfo GetDesignTimeInfo(const UWidget* Widget);
};
