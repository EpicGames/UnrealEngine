// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "Types/NavigationMetaData.h"
#include "Input/NavigationMethod.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "WidgetNavigation.generated.h"

class UWidget;

DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(UWidget*, FCustomWidgetNavigationDelegate, EUINavigation, Navigation);

/**
 *
 */
USTRUCT(BlueprintType)
struct FWidgetNavigationData
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	EUINavigationRule Rule = EUINavigationRule::Escape;

	/** This either the widget to focus, OR the name of the function to call. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	FName WidgetToFocus;

	UPROPERTY()
	TWeakObjectPtr<UWidget> Widget;

	UPROPERTY()
	FCustomWidgetNavigationDelegate CustomDelegate;

	UMG_API void Resolve(class UUserWidget* Outer, class UWidgetTree* WidgetTree);

#if WITH_EDITOR
	UMG_API void TryToRenameBinding(FName OldName, FName NewName);
#endif
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UWidgetNavigation : public UObject
{
	GENERATED_UCLASS_BODY()
	
public:
	/** Happens when the user presses up arrow, joystick, d-pad. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	FWidgetNavigationData Up;

	/** Happens when the user presses down arrow, joystick, d-pad. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	FWidgetNavigationData Down;

	/** Happens when the user presses left arrow, joystick, d-pad. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	FWidgetNavigationData Left;

	/** Happens when the user presses right arrow, joystick, d-pad. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	FWidgetNavigationData Right;

	/** Happens when the user presses Tab. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	FWidgetNavigationData Next;

	/** Happens when the user presses Shift+Tab. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	FWidgetNavigationData Previous;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation")
	EWidgetNavigationRoutingPolicy RoutingPolicy = EWidgetNavigationRoutingPolicy::AcceptFocus;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Navigation", meta=(ExcludeBaseStruct))
	TInstancedStruct<FNavigationMethod> NavigationMethod;

public:

#if WITH_EDITOR

	/**  */
	UMG_API FWidgetNavigationData& GetNavigationData(EUINavigation Nav);

	/**  */
	UMG_API EUINavigationRule GetNavigationRule(EUINavigation Nav);

	/** Try to rename any explicit or custom bindings from an old to a new name. This method can be overridden to customize resolving rules. */
	UMG_API virtual void TryToRenameBinding(FName OldName, FName NewName);

#endif

	/** Resolve widget names. This method can be overridden to customize resolving rules. */
	UMG_API virtual void ResolveRules(class UUserWidget* Outer, class UWidgetTree* WidgetTree);

	/** Updates a slate metadata object to match this configured navigation ruleset. */
	UMG_API void UpdateMetaData(TSharedRef<FNavigationMetaData> MetaData);

	/** @return true if the configured navigation object is the same as an un-customized navigation rule set. */
	UMG_API bool IsDefaultNavigation() const;

private:

	void UpdateMetaDataEntry(TSharedRef<FNavigationMetaData> MetaData, const FWidgetNavigationData & NavData, EUINavigation Nav);
};
