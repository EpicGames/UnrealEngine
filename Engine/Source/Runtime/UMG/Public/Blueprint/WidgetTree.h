// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Components/PanelWidget.h"
#include "Components/NamedSlotInterface.h"
#include "Blueprint/UserWidget.h"

#include "WidgetTree.generated.h"

/** The widget tree manages the collection of widgets in a blueprint widget. */
UCLASS(MinimalAPI)
class UWidgetTree : public UObject, public INamedSlotInterface
{
	GENERATED_UCLASS_BODY()

public:

	// Begin UObject
	UMG_API virtual UWorld* GetWorld() const override;

#if WITH_EDITOR
	UMG_API virtual bool IsCapturingAsRootObjectForTransaction() const override;
#endif //WITH_EDITOR
	// End UObject

	/** Finds the widget in the tree by name. */
	UMG_API UWidget* FindWidget(const FName& Name) const;

	/** Finds a widget in the tree using the native widget as the key. */
	UMG_API UWidget* FindWidget(TSharedRef<SWidget> InWidget) const;

	/** Finds the widget in the tree by name and casts the return to the desired type. */
	template <typename WidgetT>
	inline WidgetT* FindWidget(const FName& Name) const
	{
		return Cast<WidgetT>(FindWidget(Name));
	}

	/** Removes the widget from the hierarchy and all sub widgets. */
	UMG_API bool RemoveWidget(UWidget* Widget);

	/** Gets the parent widget of a given widget, and potentially the child index. */
	static UMG_API class UPanelWidget* FindWidgetParent(UWidget* Widget, int32& OutChildIndex);

	/**
	 * Searches recursively through the children of the given ParentWidget to find a child widget of the given name.
	 * If successful, also gets the index the child ultimately occupies within the starting ParentWidget (INDEX_NONE otherwise)
	 */
	static UMG_API UWidget* FindWidgetChild(UPanelWidget* ParentWidget, FName ChildWidgetName, int32& OutChildIndex);

	/**
	 * Determines the child index of the given ParentWidget that the given ChildWidget ultimately occupies, accounting for nesting
	 * @return The child slot index within ParentWidget that ChildWidget ultimately occupies (INDEX_NONE if ChildWidget is not within ParentWidget at any level)
	 */
	static UMG_API int32 FindChildIndex(const UPanelWidget* ParentWidget, const UWidget* ChildWidget);

	/** Gathers all the widgets in the tree recursively */
	UMG_API void GetAllWidgets(TArray<UWidget*>& Widgets) const;

	/** Gathers descendant child widgets of a parent widget. */
	static UMG_API void GetChildWidgets(UWidget* Parent, TArray<UWidget*>& Widgets);

	/** Attempts to move a constructed Widget to another tree. Returns true on a successful move. */
	static UMG_API bool TryMoveWidgetToNewTree(UWidget* Widget, UWidgetTree* DestinationTree);

	/**
	 * Iterates through all widgets including widgets contained in named slots, other than
	 * investigating named slots, this code does not dive into foreign WidgetTrees, as would exist
	 * inside another user widget.
	 */
	UMG_API void ForEachWidget(TFunctionRef<void(UWidget*)> Predicate) const;

	/**
	 * Iterates through all widgets including widgets contained in named slots, other than
	 * investigating named slots.  This includes foreign widget trees inside of other UserWidgets.
	 */
	UMG_API void ForEachWidgetAndDescendants(TFunctionRef<void(UWidget*)> Predicate) const;

	/**
	 * Iterates through all child widgets including widgets contained in named slots, other than
	 * investigating named slots, this code does not dive into foreign WidgetTrees, as would exist
	 * inside another user widget.
	 */
	static UMG_API void ForWidgetAndChildren(UWidget* Widget, TFunctionRef<void(UWidget*)> Predicate);

	/**
	 * Iterates widgets according to ForEachWidget until Predicate returns false.
	 */
	void ForEachWidgetUntil(TFunctionRef<bool(UWidget*)> Predicate) const;

	/**
	 * Iterates widgets according to ForWidgetAndChildren until Predicate returns false.
	 */
	static UMG_API bool ForEachWidgetAndChildrenUntil(UWidget* Widget, TFunctionRef<bool(UWidget*)> Predicate);


	/** Constructs the widget, and adds it to the tree. */
	template <typename WidgetT>
	[[nodiscard]] inline WidgetT* ConstructWidget(TSubclassOf<WidgetT> WidgetClass = WidgetT::StaticClass(), FName WidgetName = NAME_None)
	{
		if(WidgetClass != nullptr)
		{
			if constexpr(std::is_base_of_v<UUserWidget, WidgetT>)
			{
				return CreateWidget<WidgetT>(this, *WidgetClass, WidgetName);
			}
			else
			{
				static_assert(std::is_base_of_v<UWidget, WidgetT>, "WidgetTree::ConstructWidget can only create UWidget objects.");
				return NewObject<WidgetT>(this, WidgetClass, WidgetName, RF_Transactional);
			}
		}

		return nullptr;
	}

	// INamedSlotInterface
	//----------------------------------------------------------------------------------------
	
	/** Gets the names for slots that we can store widgets into. */
	UMG_API virtual void GetSlotNames(TArray<FName>& SlotNames) const override;

	/** Gets the widget for a given slot by name, will return nullptr if no widget is in the slot. */
	UMG_API virtual UWidget* GetContentForSlot(FName SlotName) const override;

	/** Sets the widget for a given slot by name. */
	UMG_API virtual void SetContentForSlot(FName SlotName, UWidget* Content) override;

	//----------------------------------------------------------------------------------------

	// UObject interface
	UMG_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UMG_API virtual void PostLoad() override;
	// End of UObject interface

#if WITH_EDITORONLY_DATA
	void FixupOrphanedWidgetNames();
#endif //WITH_EDITORONLY_DATA

public:
	/** The root widget of the tree */
	UPROPERTY(Instanced)
	TObjectPtr<UWidget> RootWidget;

	/**
	 * Stores the widgets being assigned to named slots, these widgets will be slotted into the named slots of the
	 * user widget that owns this widget tree after the user widget is constructed.  This is how we store the
	 * template content in order to have named slot inheritance, and merging widget trees.
	 */
	UPROPERTY()
	TMap<FName, TObjectPtr<UWidget>> NamedSlotBindings;

protected:

#if WITH_EDITORONLY_DATA
	UPROPERTY(Instanced)
	TArray< TObjectPtr<UWidget> > AllWidgets;
#endif
};
