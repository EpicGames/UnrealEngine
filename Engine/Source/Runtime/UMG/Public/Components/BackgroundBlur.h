// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ContentWidget.h"
#include "BackgroundBlur.generated.h"

/**
 * A background blur is a container widget that can contain one child widget, providing an opportunity 
 * to surround it with adjustable padding and apply a post-process Gaussian blur to all content beneath the widget.
 *
 * * Single Child
 * * Blur Effect
 */
UCLASS(MinimalAPI)
class UBackgroundBlur : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter=SetPadding, Category = Content)
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the content horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter=SetHorizontalAlignment, Category = Content)
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the content vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter=SetVerticalAlignment, Category = Content)
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

	UE_DEPRECATED(5.1, "Direct access to bApplyAlphaToBlur is deprecated. Please use the getter or setter.")
	/** True to modulate the strength of the blur based on the widget alpha. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter=GetApplyAlphaToBlur, Setter=SetApplyAlphaToBlur, BlueprintSetter=SetApplyAlphaToBlur, Category = Content)
	bool bApplyAlphaToBlur;

	UE_DEPRECATED(5.1, "Direct access to BlurStrength is deprecated. Please use the getter or setter.")
	/**
	 * How blurry the background is.  Larger numbers mean more blurry but will result in larger runtime cost on the GPU.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter=SetBlurStrength, Category = Appearance, meta = (ClampMin = 0, ClampMax = 100))
	float BlurStrength;

	UE_DEPRECATED(5.1, "Direct access to bOverrideAutoRadiusCalculation is deprecated. Please use the getter or setter.")
	/** When OverrideAutoRadiusCalculation is set to true, BlurRadius is used for the radius of the blur. When false, it's automatically calculated using the BlurStength value. */
	UPROPERTY(Getter = GetOverrideAutoRadiusCalculation, Setter = SetOverrideAutoRadiusCalculation)
	bool bOverrideAutoRadiusCalculation;

	UE_DEPRECATED(5.1, "Direct access to BlurRadius is deprecated. Please use the getter or setter.")
	/**
	 * This is the number of pixels which will be weighted in each direction from any given pixel when computing the blur
	 * A larger value is more costly but allows for stronger blurs.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Getter, Setter, BlueprintSetter=SetBlurRadius, Category = Appearance, meta = (ClampMin = 0, ClampMax = 255, EditCondition = "bOverrideAutoRadiusCalculation"))
	int32 BlurRadius;

	UE_DEPRECATED(5.1, "Direct access to CornerRadius is deprecated. Please use the getter or setter.")
	/** Radius in Slate Units applied to the blur widget at each corner. X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Getter, Setter, BlueprintSetter=SetCornerRadius, Category = Appearance)
	FVector4 CornerRadius;

	UE_DEPRECATED(5.1, "Direct access to LowQualityFallbackBrush is deprecated. Please use the getter or setter.")
	/**
	 * An image to draw instead of applying a blur when low quality override mode is enabled. 
	 * You can enable low quality mode for background blurs by setting the cvar Slate.ForceBackgroundBlurLowQualityOverride to 1. 
	 * This is usually done in the project's scalability settings
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter=SetLowQualityFallbackBrush, Category = Appearance)
	FSlateBrush LowQualityFallbackBrush;

public:
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetApplyAlphaToBlur(bool bInApplyAlphaToBlur);

	UMG_API bool GetApplyAlphaToBlur() const;

	UMG_API void SetOverrideAutoRadiusCalculation(bool InOverrideAutoRadiusCalculation);

	UMG_API bool GetOverrideAutoRadiusCalculation() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetBlurRadius(int32 InBlurRadius);

	UMG_API int32 GetBlurRadius() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API virtual void SetBlurStrength(float InStrength);

	UMG_API float GetBlurStrength() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API virtual void SetCornerRadius(FVector4 InCornerRadius);
	
	UMG_API FVector4 GetCornerRadius() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetLowQualityFallbackBrush(const FSlateBrush& InBrush);

	UMG_API FSlateBrush GetLowQualityFallbackBrush() const;

	/** UObject interface */
	UMG_API virtual void Serialize(FArchive& Ar) override;
	UMG_API virtual void PostLoad() override;

protected:
	/** UWidget interface */
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UMG_API virtual void SynchronizeProperties() override;

	/** UPanelWidget interface */
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;

protected:
	TSharedPtr<class SBackgroundBlur> MyBackgroundBlur;
};
