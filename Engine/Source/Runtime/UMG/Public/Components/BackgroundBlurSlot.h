// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ScriptMacros.h"
#include "Components/SlateWrapperTypes.h"
#include "Components/PanelSlot.h"
#include "Layout/Margin.h"
#include "BackgroundBlurSlot.generated.h"


class SBackgroundBlur;
class UBackgroundBlur;

/**
 * The Slot for the UBackgroundBlurSlot, contains the widget displayed in a BackgroundBlur's single slot
 */
UCLASS(MinimalAPI)
class UBackgroundBlurSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Background Blur Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Background Blur Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Background Blur Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

protected:
	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetPadding", Category="Layout|Background Blur Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Background Blur Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Background Blur Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	//~ Begin UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying slot for the slate BackgroundBlur. */
	UMG_API void BuildSlot(TSharedRef<SBackgroundBlur> InBackgroundBlur);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

public:

#if WITH_EDITOR

	//~ Begin UObject interface
	UMG_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End of UObject interface

#endif

private:

	/** A pointer to the BackgroundBlur to allow us to adjust the size, padding...etc at runtime. */
	TSharedPtr<SBackgroundBlur> BackgroundBlur;

	friend UBackgroundBlur;
};
