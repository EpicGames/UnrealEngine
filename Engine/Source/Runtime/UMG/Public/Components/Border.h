// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateBrush.h"
#include "Components/ContentWidget.h"
#include "Border.generated.h"

class SBorder;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class USlateBrushAsset;
class UTexture2D;

/**
 * A border is a container widget that can contain one child widget, providing an opportunity 
 * to surround it with a background image and adjustable padding.
 *
 * * Single Child
 * * Image
 */
UCLASS(MinimalAPI)
class UBorder : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the content horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Content")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the content vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetVerticalAlignment", Category="Content")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

	UE_DEPRECATED(5.1, "Direct access to bShowEffectWhenDisabled is deprecated. Please use the getter or setter.")
	/** Whether or not to show the disabled effect when this border is disabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="GetShowEffectWhenDisabled", Setter="SetShowEffectWhenDisabled", BlueprintSetter="SetShowEffectWhenDisabled", Category="Appearance", AdvancedDisplay)
	uint8 bShowEffectWhenDisabled:1;

	UE_DEPRECATED(5.1, "Direct access to ContentColorAndOpacity is deprecated. Please use the getter or setter.")
	/** Color and opacity multiplier of content in the border */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetContentColorAndOpacity", Category="Content", meta=( sRGB="true" ))
	FLinearColor ContentColorAndOpacity;

	/** A bindable delegate for the ContentColorAndOpacity. */
	UPROPERTY()
	FGetLinearColor ContentColorAndOpacityDelegate;

	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetPadding", Category="Content")
	FMargin Padding;

	/** Brush to drag as the background */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance, meta=( DisplayName="Brush" ))
	FSlateBrush Background;

	/** A bindable delegate for the Brush. */
	UPROPERTY()
	FGetSlateBrush BackgroundDelegate;

	UE_DEPRECATED(5.1, "Direct access to BrushColor is deprecated. Please use the getter or setter.")
	/** Color and opacity of the actual border image */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetBrushColor", Category="Appearance", meta = (sRGB="true"))
	FLinearColor BrushColor;

	/** A bindable delegate for the BrushColor. */
	UPROPERTY()
	FGetLinearColor BrushColorDelegate;

	UE_DEPRECATED(5.1, "Direct access to DesiredSizeScale is deprecated. Please use the getter or setter.")
	/**
	 * Scales the computed desired size of this border and its contents. Useful
	 * for making things that slide open without having to hard-code their size.
	 * Note: if the parent widget is set up to ignore this widget's desired size,
	 * then changing this value will have no effect.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetDesiredSizeScale", Category="Appearance")
	FVector2D DesiredSizeScale;

	/** Flips the background image if the localization's flow direction is RightToLeft */
	UPROPERTY(EditAnywhere, Category="Localization")
	bool bFlipForRightToLeftFlowDirection;

public:

	UPROPERTY(EditAnywhere, Category=Events, meta=( IsBindableEvent="True" ))
	FOnPointerEvent OnMouseButtonDownEvent;

	UPROPERTY(EditAnywhere, Category=Events, meta=( IsBindableEvent="True" ))
	FOnPointerEvent OnMouseButtonUpEvent;

	UPROPERTY(EditAnywhere, Category=Events, meta=( IsBindableEvent="True" ))
	FOnPointerEvent OnMouseMoveEvent;

	UPROPERTY(EditAnywhere, Category=Events, meta=( IsBindableEvent="True" ))
	FOnPointerEvent OnMouseDoubleClickEvent;

public:
	/**  */
	UMG_API FLinearColor GetContentColorAndOpacity() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetContentColorAndOpacity(FLinearColor InContentColorAndOpacity);

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	/**  */
	UMG_API FLinearColor GetBrushColor() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetBrushColor(FLinearColor InBrushColor);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetBrush(const FSlateBrush& InBrush);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetBrushFromAsset(USlateBrushAsset* Asset);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetBrushFromTexture(UTexture2D* Texture);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetBrushFromMaterial(UMaterialInterface* Material);

	/**  */
	UMG_API bool GetShowEffectWhenDisabled() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetShowEffectWhenDisabled(bool bInShowEffectWhenDisabled);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API UMaterialInstanceDynamic* GetDynamicMaterial();

public:
	/** Gets the DesiredSizeScale of this border. */
	UMG_API FVector2D GetDesiredSizeScale() const;

	/**
	 * Sets the DesiredSizeScale of this border.
	 *
	 * @param InScale    The X and Y multipliers for the desired size
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetDesiredSizeScale(FVector2D InScale);

public:

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	//~ Begin UObject Interface
	UMG_API virtual void PostLoad() override;
	//~ End UObject Interface

#if WITH_EDITOR
	//~ Begin UObject Interface
	UMG_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:
	TSharedPtr<SBorder> MyBorder;

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

	UMG_API FReply HandleMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& MouseEvent);
	UMG_API FReply HandleMouseButtonUp(const FGeometry& Geometry, const FPointerEvent& MouseEvent);
	UMG_API FReply HandleMouseMove(const FGeometry& Geometry, const FPointerEvent& MouseEvent);
	UMG_API FReply HandleMouseDoubleClick(const FGeometry& Geometry, const FPointerEvent& MouseEvent);

	/** Translates the bound brush data and assigns it to the cached brush used by this widget. */
	UMG_API const FSlateBrush* ConvertImage(TAttribute<FSlateBrush> InImageAsset) const;

	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, ContentColorAndOpacity)
};
