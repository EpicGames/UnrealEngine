// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/PanelSlot.h"

#include "BorderSlot.generated.h"

class SBorder;
class UBorder;

/**
 * The Slot for the UBorderSlot, contains the widget displayed in a border's single slot
 */
UCLASS(MinimalAPI)
class UBorderSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Border Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Border Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Border Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

protected:
	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetPadding", Category="Layout|Border Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Border Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Border Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	//~ Begin UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying slot for the slate border. */
	UMG_API void BuildSlot(TSharedRef<SBorder> InBorder);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

public:

#if WITH_EDITOR

	//~ Begin UObject interface
	UMG_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End of UObject interface

#endif

private:

	/** A pointer to the border to allow us to adjust the size, padding...etc at runtime. */
	TWeakPtr<SBorder> Border;

	friend UBorder;
};
