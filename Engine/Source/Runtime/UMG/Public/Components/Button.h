// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Input/Reply.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "Framework/SlateDelegates.h"

#include "Button.generated.h"

class SButton;
class USlateWidgetStyleAsset;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnButtonClickedEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnButtonPressedEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnButtonReleasedEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnButtonHoverEvent);

/**
 * The button is a click-able primitive widget to enable basic interaction, you
 * can place any other widget inside a button to make a more complex and 
 * interesting click-able element in your UI.
 *
 * * Single Child
 * * Clickable
 */
UCLASS(MinimalAPI)
class UButton : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.2, "Direct access to WidgetStyle is deprecated. Please use the getter and setter.")
	/** The button style used at runtime */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetStyle", Setter = "SetStyle", BlueprintSetter = "SetStyle", Category = "Appearance", meta = (DisplayName = "Style"))
	FButtonStyle WidgetStyle;
	
	UE_DEPRECATED(5.2, "Direct access to ColorAndOpacity is deprecated. Please use the getter and setter.")
	/** The color multiplier for the button content */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetColorAndOpacity" , Category="Appearance", meta=( sRGB="true" ))
	FLinearColor ColorAndOpacity;
	
	UE_DEPRECATED(5.2, "Direct access to BackgroundColor is deprecated. Please use the getter and setter.")
	/** The color multiplier for the button background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetBackgroundColor", Category="Appearance", meta=( sRGB="true" ))
	FLinearColor BackgroundColor;

	UE_DEPRECATED(5.2, "Direct access to ClickMethod is deprecated. Please use the getter and setter.")
	/** The type of mouse action required by the user to trigger the buttons 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetClickMethod", Category="Interaction", AdvancedDisplay)
	TEnumAsByte<EButtonClickMethod::Type> ClickMethod;

	UE_DEPRECATED(5.2, "Direct access to TouchMethod is deprecated. Please use the getter and setter.")
	/** The type of touch action required by the user to trigger the buttons 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetTouchMethod", Category="Interaction", AdvancedDisplay)
	TEnumAsByte<EButtonTouchMethod::Type> TouchMethod;

	UE_DEPRECATED(5.2, "Direct access to PressMethod is deprecated. Please use the getter and setter.")
	/** The type of keyboard/gamepad button press action required by the user to trigger the buttons 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetPressMethod", Category="Interaction", AdvancedDisplay)
	TEnumAsByte<EButtonPressMethod::Type> PressMethod;

	UE_DEPRECATED(5.2, "Direct access to IsFocusable is deprecated. Please use the getter.")
	/** Sometimes a button should only be mouse-clickable and never keyboard focusable. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Interaction")
	bool IsFocusable;

public:

	/** Called when the button is clicked */
	UPROPERTY(BlueprintAssignable, Category="Button|Event")
	FOnButtonClickedEvent OnClicked;

	/** Called when the button is pressed */
	UPROPERTY(BlueprintAssignable, Category="Button|Event")
	FOnButtonPressedEvent OnPressed;

	/** Called when the button is released */
	UPROPERTY(BlueprintAssignable, Category="Button|Event")
	FOnButtonReleasedEvent OnReleased;

	UPROPERTY( BlueprintAssignable, Category = "Button|Event" )
	FOnButtonHoverEvent OnHovered;

	UPROPERTY( BlueprintAssignable, Category = "Button|Event" )
	FOnButtonHoverEvent OnUnhovered;

	/** Delegate triggered when a user starts to drag a button */
	FOnDragDetected OnButtonDragDetected;

	/** Delegate triggered when a user's drag enters the bounds of this button*/
	FOnDragEnter OnButtonDragEnter;

	/** Delegate triggered when a user's drag leaves the bounds of this button */
	FOnDragLeave OnButtonDragLeave;

	/** Delegate triggered when a user's drag leaves the bounds of this button */
	FOnDragOver OnButtonDragOver;

	/** Delegate triggered when a user's drag is dropped in the bounds of this button */
	FOnDrop OnButtonDrop;

	/** Called when the button receives focus */
	FSimpleDelegate OnReceivedFocus;

	/** Called when the button loses focus */
	FSimpleDelegate OnLostFocus;

public:
	
	/** Sets the color multiplier for the button background */
	UFUNCTION(BlueprintCallable, Category="Button|Appearance")
	UMG_API void SetStyle(const FButtonStyle& InStyle);

	UMG_API const FButtonStyle& GetStyle() const;

	/** Sets the color multiplier for the button content */
	UFUNCTION(BlueprintCallable, Category="Button|Appearance")
	UMG_API void SetColorAndOpacity(FLinearColor InColorAndOpacity);

	UMG_API FLinearColor GetColorAndOpacity() const;

	/** Sets the color multiplier for the button background */
	UFUNCTION(BlueprintCallable, Category="Button|Appearance")
	UMG_API void SetBackgroundColor(FLinearColor InBackgroundColor);

	UMG_API FLinearColor GetBackgroundColor() const;

	/**
	 * Returns true if the user is actively pressing the button.  Do not use this for detecting 'Clicks', use the OnClicked event instead.
	 *
	 * @return true if the user is actively pressing the button otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API bool IsPressed() const;

	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API void SetClickMethod(EButtonClickMethod::Type InClickMethod);

	UMG_API EButtonClickMethod::Type GetClickMethod() const;

	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API void SetTouchMethod(EButtonTouchMethod::Type InTouchMethod);

	UMG_API EButtonTouchMethod::Type GetTouchMethod() const;

	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API void SetPressMethod(EButtonPressMethod::Type InPressMethod);

	UMG_API EButtonPressMethod::Type GetPressMethod() const;

	UMG_API bool GetIsFocusable() const;

	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API void SetAllowDragDrop(bool bInAllowDragDrop);

public:

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	//~ Begin UObject Interface
	UMG_API virtual void PostLoad() override;
	//~ End UObject Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:
	/** Handle the actual click event from slate and forward it on */
	UMG_API FReply SlateHandleClicked();
	UMG_API void SlateHandlePressed();
	UMG_API void SlateHandleReleased();
	UMG_API void SlateHandleHovered();
	UMG_API void SlateHandleUnhovered();
	UMG_API void SlateHandleOnReceivedFocus();
	UMG_API void SlateHandleOnLostFocus();

	// Drag and Drop
	UMG_API FReply SlateHandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UMG_API void SlateHandleDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
	UMG_API void SlateHandleDragLeave(const FDragDropEvent& DragDropEvent);
	UMG_API FReply SlateHandleDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
	UMG_API FReply SlateHandleDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
	// End Drag and Drop

	// Initialize IsFocusable in the constructor before the SWidget is constructed.
	UMG_API void InitIsFocusable(bool InIsFocusable);
protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
#if WITH_EDITOR
	virtual TSharedRef<SWidget> RebuildDesignWidget(TSharedRef<SWidget> Content) override { return Content; }
#endif
	//~ End UWidget Interface

#if WITH_ACCESSIBILITY
	UMG_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const override;
#endif

protected:
	/** Cached pointer to the underlying slate button owned by this UWidget */
	TSharedPtr<SButton> MyButton;

	/** True if this button can be dragged */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Interaction")
	bool bAllowDragDrop;
};
