// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/PanelSlot.h"

#include "ButtonSlot.generated.h"

class SButton;

/** The Slot for the UButtonSlot, contains the widget displayed in a button's single slot */
UCLASS(MinimalAPI)
class UButtonSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:
	
	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetPadding", Category="Layout|Button Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Button Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Button Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Button Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Button Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Button Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	// UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UPanelSlot interface

	/** Builds the underlying slot for the slate button. */
	UMG_API void BuildSlot(TSharedRef<SButton> InButton);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

private:

	/** A pointer to the button to allow us to adjust the size, padding...etc at runtime. */
	TWeakPtr<SButton> Button;
};
