// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Components/PanelWidget.h"
#include "CanvasPanel.generated.h"

class UCanvasPanelSlot;

/**
 * The canvas panel is a designer friendly panel that allows widgets to be laid out at arbitrary 
 * locations, anchored and z-ordered with other children of the canvas.  The canvas is a great widget
 * for manual layout, but bad when you want to procedurally just generate widgets and place them in a 
 * container (unless you want absolute layout).
 *
 * * Many Children
 * * Absolute Layout
 * * Anchors
 */
UCLASS(meta = (ShortTooltip = "A designer-friendly panel useful for laying out top-level widgets. Use sparingly."), MinimalAPI)
class UCanvasPanel : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:

	/**  */
	UFUNCTION(BlueprintCallable, Category="Canvas Panel")
	UMG_API UCanvasPanelSlot* AddChildToCanvas(UWidget* Content);

	/** Gets the underlying native canvas widget if it has been constructed */
	UMG_API TSharedPtr<class SConstraintCanvas> GetCanvasWidget() const;

	/** Computes the geometry for a particular slot based on the current geometry of the canvas. */
	UMG_API bool GetGeometryForSlot(int32 SlotIndex, FGeometry& ArrangedGeometry) const;

	/** Computes the geometry for a particular slot based on the current geometry of the canvas. */
	UMG_API bool GetGeometryForSlot(UCanvasPanelSlot* Slot, FGeometry& ArrangedGeometry) const;

	UMG_API void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	// UWidget interface
	UMG_API virtual const FText GetPaletteCategory() override;
	// End UWidget interface

	// UWidget interface
	virtual bool LockToPanelOnDrag() const override
	{
		return true;
	}
	// End UWidget interface
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:

	TSharedPtr<class SConstraintCanvas> MyCanvas;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
