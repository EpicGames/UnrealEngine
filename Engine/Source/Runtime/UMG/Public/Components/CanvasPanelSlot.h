// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Layout/Geometry.h"
#include "Widgets/Layout/Anchors.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Components/PanelSlot.h"

#include "CanvasPanelSlot.generated.h"

USTRUCT(BlueprintType)
struct FAnchorData
{
public:
	GENERATED_USTRUCT_BODY()

public:

	/** Offset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AnchorData)
	FMargin Offsets;
	
	/** Anchors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AnchorData)
	FAnchors Anchors;

	/**
	 * Alignment is the pivot point of the widget.  Starting in the upper left at (0,0),
	 * ending in the lower right at (1,1).  Moving the alignment point allows you to move
	 * the origin of the widget.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AnchorData)
	FVector2D Alignment = FVector2D::ZeroVector;

	/**
	* Compares this set of FAnchorData with another for equality.
	*
	* @param Other The other FAnchorData.
	* @return true if the two FAnchors are equal, false otherwise.
	*/
	bool operator==(const FAnchorData& Other) const
	{
		return Offsets == Other.Offsets && 
			   Anchors == Other.Anchors && 
			   Alignment == Other.Alignment;
	}

	/**
	* Compares this set of FAnchorData with another for inequality.
	*
	* @param Other The other FAnchorData.
	* @return true if the two FAnchors are not equal, false otherwise.
	*/
	bool operator!=(const FAnchorData& Other) const
	{
		return !(*this == Other);
	}
};

UCLASS(MinimalAPI)
class UCanvasPanelSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.1, "Direct access to LayoutData is deprecated. Please use the getter or setter.")
	/** The anchoring information for the slot */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="GetLayout", Setter="SetLayout", BlueprintGetter ="GetLayout", BlueprintSetter="SetLayout", Category = "Layout|Canvas Slot")
	FAnchorData LayoutData;

	UE_DEPRECATED(5.1, "Direct access to bAutoSize is deprecated. Please use the getter or setter.")
	/** When AutoSize is true we use the widget's desired size */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="GetAutoSize", Setter="SetAutoSize", BlueprintGetter="GetAutoSize", BlueprintSetter="SetAutoSize", Category = "Layout|Canvas Slot", AdvancedDisplay, meta = (DisplayName = "Size To Content"))
	bool bAutoSize;

	UE_DEPRECATED(5.1, "Direct access to ZOrder is deprecated. Please use the getter or setter.")
	/** The order priority this widget is rendered in.  Higher values are rendered last (and so they will appear to be on top). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetZOrder", BlueprintSetter="SetZOrder", Category="Layout|Canvas Slot")
	int32 ZOrder;

public:
#if WITH_EDITOR
	UMG_API virtual bool NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize) override;
	UMG_API virtual bool DragDropPreviewByDesigner(const FVector2D& LocalCursorPosition, const TOptional<int32>& XGridSnapSize, const TOptional<int32>& YGridSnapSize) override;
	UMG_API virtual void SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot) override;
#endif //WITH_EDITOR

	/** Sets the layout data of the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetLayout(const FAnchorData& InLayoutData);

	/** Gets the layout data of the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API FAnchorData GetLayout() const;

	/** Sets the position of the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetPosition(FVector2D InPosition);

	/** Gets the position of the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API FVector2D GetPosition() const;

	/** Sets the size of the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetSize(FVector2D InSize);

	/** Gets the size of the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API FVector2D GetSize() const;

	/** Sets the offset data of the slot, which could be position and size, or margins depending on the anchor points */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetOffsets(FMargin InOffset);

	/** Gets the offset data of the slot, which could be position and size, or margins depending on the anchor points */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API FMargin GetOffsets() const;
	
	/** Sets the anchors on the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetAnchors(FAnchors InAnchors);

	/** Gets the anchors on the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API FAnchors GetAnchors() const;

	/** Sets the alignment on the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetAlignment(FVector2D InAlignment);

	/** Gets the alignment on the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API FVector2D GetAlignment() const;

	/** Sets if the slot to be auto-sized */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetAutoSize(bool InbAutoSize);

	/** Gets if the slot to be auto-sized */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API bool GetAutoSize() const;

	/** Sets the z-order on the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API void SetZOrder(int32 InZOrder);

	/** Gets the z-order on the slot */
	UFUNCTION(BlueprintCallable, Category="Layout|Canvas Slot")
	UMG_API int32 GetZOrder() const;

public:

	/** Sets the anchors on the slot */
	UFUNCTION()
	UMG_API void SetMinimum(FVector2D InMinimumAnchors);

	/** Sets the anchors on the slot */
	UFUNCTION()
	UMG_API void SetMaximum(FVector2D InMaximumAnchors);

public:

	UMG_API void BuildSlot(TSharedRef<SConstraintCanvas> Canvas);

	// UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UPanelSlot interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	// UObject interface
	using Super::PreEditChange;
	UMG_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	UMG_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	// End of UObject interface

	/** Stores the current layout information about the slot and parent canvas. */
	UMG_API void SaveBaseLayout();

	UMG_API void SetDesiredPosition(FVector2D InPosition);

	/** Compares the saved base layout against the current state.  Updates the necessary properties to maintain a stable position. */
	UMG_API void RebaseLayout(bool PreserveSize = true);
#endif

private:
	SConstraintCanvas::FSlot* Slot;

#if WITH_EDITORONLY_DATA
	FGeometry PreEditGeometry;
	FAnchorData PreEditLayoutData;

	TOptional<FVector2D> DesiredPosition;
#endif
};
