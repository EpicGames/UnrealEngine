// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Binding/States/WidgetStateRegistration.h"
#include "Components/ContentWidget.h"

#include "CheckBox.generated.h"

class SCheckBox;
class USlateBrushAsset;
class USlateWidgetStyleAsset;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam( FOnCheckBoxComponentStateChanged, bool, bIsChecked );

/**
 * The checkbox widget allows you to display a toggled state of 'unchecked', 'checked' and 
 * 'indeterminable.  You can use the checkbox for a classic checkbox, or as a toggle button,
 * or as radio buttons.
 * 
 * * Single Child
 * * Toggle
 */
UCLASS(MinimalAPI)
class UCheckBox : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.1, "Direct access to CheckedState is deprecated. Please use the getter or setter.")
	/** Whether the check box is currently in a checked state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetCheckedState", BlueprintSetter="SetCheckedState", FieldNotify, Category="Appearance")
	ECheckBoxState CheckedState;

	UE_DEPRECATED(5.2, "Direct access to CheckedStateDelegate is deprecated. Please use the InitCheckedStateDelegate() function.")
	/** A bindable delegate for the IsChecked. */
	UPROPERTY()
	FGetCheckBoxState CheckedStateDelegate;

	UE_DEPRECATED(5.1, "Direct access to WidgetStyle is deprecated. Please use the getter or setter.")
	/** The checkbox bar style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Style", meta = (DisplayName="Style"))
	FCheckBoxStyle WidgetStyle;

	/** How the content of the toggle button should align within the given space */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Appearance")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to ClickMethod is deprecated. Please use the getter or setter.")
	/** The type of mouse action required by the user to trigger the buttons 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetClickMethod", Category="Interaction", AdvancedDisplay)
	TEnumAsByte<EButtonClickMethod::Type> ClickMethod;

	UE_DEPRECATED(5.1, "Direct access to TouchMethod is deprecated. Please use the getter or setter.")
	/** The type of touch action required by the user to trigger the buttons 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetTouchMethod", Category="Interaction", AdvancedDisplay)
	TEnumAsByte<EButtonTouchMethod::Type> TouchMethod;

	UE_DEPRECATED(5.1, "Direct access to PressMethod is deprecated. Please use the getter or setter.")
	/** The type of keyboard/gamepad button press action required by the user to trigger the buttons 'Click' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetPressMethod", Category="Interaction", AdvancedDisplay)
	TEnumAsByte<EButtonPressMethod::Type> PressMethod;

	UE_DEPRECATED(5.2, "Direct access to bIsFocusable is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** Sometimes a button should only be mouse-clickable and never keyboard focusable. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Interaction")
	bool IsFocusable;

public:

	/** Called when the checked state has changed */
	UPROPERTY(BlueprintAssignable, Category="CheckBox|Event")
	FOnCheckBoxComponentStateChanged OnCheckStateChanged;

public:

	/** Returns true if this button is currently pressed */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool IsPressed() const;
	
	/** Returns true if the checkbox is currently checked */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool IsChecked() const;

	/** Returns the full current checked state. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API ECheckBoxState GetCheckedState() const;

	/** Sets the checked state. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetIsChecked(bool InIsChecked);

	/** Sets the checked state. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetCheckedState(ECheckBoxState InCheckedState);

	/** Returns the local style. */
	UMG_API const FCheckBoxStyle& GetWidgetStyle() const;

	/** Sets the style. */
	UMG_API void SetWidgetStyle(const FCheckBoxStyle& InStyle);

	/** Returns the click method. */
	UMG_API EButtonClickMethod::Type GetClickMethod() const;

	/** Sets the click method. */
	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API void SetClickMethod(EButtonClickMethod::Type InClickMethod);

	/** Returns the touch method. */
	UMG_API EButtonTouchMethod::Type GetTouchMethod() const;

	/** Sets the touch method. */
	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API void SetTouchMethod(EButtonTouchMethod::Type InTouchMethod);

	/** Returns the press method. */
	UMG_API EButtonPressMethod::Type GetPressMethod() const;

	/** Sets the press method. */
	UFUNCTION(BlueprintCallable, Category="Button")
	UMG_API void SetPressMethod(EButtonPressMethod::Type InPressMethod);

	/** Is the checkbox focusable. */
	UMG_API bool GetIsFocusable() const;

public:
	
	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

	/** Initialize IsFocusable in the constructor before the SWidget is constructed. */
	UMG_API void InitIsFocusable(bool InIsFocusable);

	UMG_API void InitCheckedStateDelegate(FGetCheckBoxState InCheckedStateDelegate);
protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
#if WITH_EDITOR
	virtual TSharedRef<SWidget> RebuildDesignWidget(TSharedRef<SWidget> Content) override
	{
		return Content;
	}
#endif
	//~ End UWidget Interface

	UMG_API void SlateOnCheckStateChangedCallback(ECheckBoxState NewState);

#if WITH_ACCESSIBILITY
	UMG_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const override;
#endif
	
protected:
	TSharedPtr<SCheckBox> MyCheckbox;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PROPERTY_BINDING_IMPLEMENTATION(ECheckBoxState, CheckedState)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

UCLASS(Transient, MinimalAPI)
class UWidgetCheckedStateRegistration : public UWidgetEnumStateRegistration
{
	GENERATED_BODY()

public:

	/** Post-load initialized values corresponding to this enum state */
	static UMG_API FWidgetStateBitfield Unchecked;
	static UMG_API FWidgetStateBitfield Checked;
	static UMG_API FWidgetStateBitfield Undetermined;

	static const inline FName StateName = FName("CheckedState");

	//~ Begin UWidgetEnumStateRegistration Interface.
	UMG_API virtual FName GetStateName() const override;
	UMG_API virtual uint8 GetRegisteredWidgetState(const UWidget* InWidget) const override;
	//~ End UWidgetEnumStateRegistration Interface

	/** Convenience method to get widget state bitfield from enum value */
	static UMG_API const FWidgetStateBitfield& GetBitfieldFromValue(uint8 InValue);

protected:
	friend UWidgetStateSettings;

	//~ Begin UWidgetEnumStateRegistration Interface.
	UMG_API virtual void InitializeStaticBitfields() const override;
	//~ End UWidgetEnumStateRegistration Interface
};
