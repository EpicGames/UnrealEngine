// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateBrush.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "CircularThrobber.generated.h"

class USlateBrushAsset;

/**
 * A throbber widget that orients images in a spinning circle.
 * 
 * * No Children
 * * Spinner Progress
 */
UCLASS(MinimalAPI)
class UCircularThrobber : public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.2, "Direct access to NumberOfPieces is deprecated. Please use the getter or setter.")
	/** How many pieces there are */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetNumberOfPieces", Category = Appearance, meta = (ClampMin = "1", ClampMax = "25", UIMin = "1", UIMax = "25"))
	int32 NumberOfPieces;

	UE_DEPRECATED(5.2, "Direct access to Period is deprecated. Please use the getter or setter.")
	/** The amount of time for a full circle (in seconds) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetPeriod", Category=Appearance, meta=( ClampMin="0", UIMin="0" ))
	float Period;

	UE_DEPRECATED(5.2, "Direct access to Radius is deprecated. Please use the getter or setter.")
	/** The radius of the circle. If the throbber is a child of Canvas Panel, the 'Size to Content' option must be enabled in order to set Radius. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetRadius", Category=Appearance, meta=(EditCondition="bEnableRadius"))
	float Radius;

	UE_DEPRECATED(5.2, "Direct access to Image is deprecated. Please use the getter or setter.")
	/** The throbber image. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Appearance)
	FSlateBrush Image;

public:

	/** Sets how many pieces there are. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetNumberOfPieces(int32 InNumberOfPieces);
	UMG_API int32 GetNumberOfPieces() const;

	/** Sets the amount of time for a full circle (in seconds). */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetPeriod(float InPeriod);
	UMG_API float GetPeriod() const;

	/** Sets the radius of the circle. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetRadius(float InRadius);
	UMG_API float GetRadius() const;

	/** Sets the throbber image. */
	UMG_API void SetImage(const FSlateBrush& InRadius);
	UMG_API const FSlateBrush& GetImage() const;

	//~ Begin UWidget Interface
public:
	UMG_API virtual void SynchronizeProperties() override;
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

private:
	UPROPERTY(Transient, EditAnywhere, Category="Appearance", meta=(InlineEditConditionToggle))
	bool bEnableRadius;

	/** The CircularThrobber widget managed by this object. */
	TSharedPtr<SCircularThrobber> MyCircularThrobber;
};
