// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Components/Widget.h"
#include "ComboBox.generated.h"

/**
 * The combobox allows you to display a list of options to the user in a dropdown menu for them to select one.
 */
UCLASS(Experimental, meta=( DisplayName="ComboBox (Object)" ), MinimalAPI)
class UComboBox : public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Style")
	FScrollBarStyle ScrollBarStyle;

	/** The list of items to be displayed on the combobox. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Content)
	TArray<TObjectPtr<UObject>> Items;

	/** Called when the widget is needed for the item. */
	UPROPERTY(EditAnywhere, Category=Events, meta=( IsBindableEvent="True" ))
	FGenerateWidgetForObject OnGenerateWidgetEvent;

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Interaction)
	bool bIsFocusable;

protected:
	UMG_API TSharedRef<SWidget> HandleGenerateWidget(UObject* Item) const;

	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

protected:

	TSharedPtr< SComboBox<UObject*> > MyComboBox;
};
