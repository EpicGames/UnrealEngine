// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/Widget.h"
#include "Widgets/Input/SComboBox.h"

#include "ComboBoxKey.generated.h"


/**
 * The combobox allows you to display a list of options to the user in a dropdown menu for them to select one.
 * Use OnGenerateConentWidgetEvent to return a custom built widget.
 */
UCLASS(meta = (DisplayName = "ComboBox (Key)"), MinimalAPI)
class UComboBoxKey : public UWidget
{
	GENERATED_BODY()

public:
	UMG_API UComboBoxKey();

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnSelectionChangedEvent, FName, SelectedItem, ESelectInfo::Type, SelectionType);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnOpeningEvent);
	DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(UWidget*, FGenerateWidgetEvent, FName, Item);

private:
	/** . */
	UPROPERTY(EditAnywhere, Category = Content)
	TArray<FName> Options;

	/** */
	UPROPERTY(EditAnywhere, FieldNotify, Category = Content)
	FName SelectedOption;

public:
	UE_DEPRECATED(5.2, "Direct access to WidgetStyle is deprecated. Please use the getter or setter.")
	/** The combobox style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Style, meta = (DisplayName = "Style"))
	FComboBoxStyle WidgetStyle;

	UE_DEPRECATED(5.2, "Direct access to ItemStyle is deprecated. Please use the getter or setter.")
	/** The item row style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Style)
	FTableRowStyle ItemStyle;

	UE_DEPRECATED(5.2, "Direct access to ScrollBarStyle is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** The scroll bar style. */
    UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Style")
    FScrollBarStyle ScrollBarStyle;

	UE_DEPRECATED(5.2, "Direct access to ForegroundColor is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** The foreground color to pass through the hierarchy. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category = Style, meta = (DesignerRebuild))
	FSlateColor ForegroundColor;

	UE_DEPRECATED(5.2, "Direct access to ContentPadding is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Style)
	FMargin ContentPadding;

	UE_DEPRECATED(5.2, "Direct access to MaxListHeight is deprecated. Please use the getter or setter.")
	/** The max height of the combobox list that opens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Style, AdvancedDisplay)
	float MaxListHeight;

	UE_DEPRECATED(5.2, "Direct access to bHasDownArrow is deprecated. Please use the getter or setter.")
	/**
	 * When false, the down arrow is not generated and it is up to the API consumer
	 * to make their own visual hint that this is a drop down.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsHasDownArrow", Setter = "SetHasDownArrow", Category = Style, AdvancedDisplay)
	bool bHasDownArrow;

	UE_DEPRECATED(5.2, "Direct access to bEnableGamepadNavigationMode is deprecated. Please use the getter or setter.")
	/**
	 * When false, directional keys will change the selection. When true, ComboBox
	 * must be activated and will only capture arrow input while activated.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsEnableGamepadNavigationMode", Setter = "SetEnableGamepadNavigationMode", Category = Style, AdvancedDisplay)
	bool bEnableGamepadNavigationMode;

	UE_DEPRECATED(5.2, "Direct access to bIsFocusable is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** When true, allows the combo box to receive keyboard focus */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsFocusable", Category = Style)
	bool bIsFocusable;

public: // Events

	/** Called when the widget is needed for the content. */
	UPROPERTY(EditAnywhere, Category = Events, meta = (IsBindableEvent = "True"))
	FGenerateWidgetEvent OnGenerateContentWidget;
	
	/** Called when the widget is needed for the item. */
	UPROPERTY(EditAnywhere, Category = Events, meta = (IsBindableEvent = "True"))
	FGenerateWidgetEvent OnGenerateItemWidget;

	/** Called when a new item is selected in the combobox. */
	UPROPERTY(BlueprintAssignable, Category = Events)
	FOnSelectionChangedEvent OnSelectionChanged;

	/** Called when the combobox is opening */
	UPROPERTY(BlueprintAssignable, Category = Events)
	FOnOpeningEvent OnOpening;

public:

	

	/** Add an element to the option list. */
	UFUNCTION(BlueprintCallable, Category = "ComboBox")
	UMG_API void AddOption(FName Option);

	/** Remove an element to the option list. */
	UFUNCTION(BlueprintCallable, Category = "ComboBox")
	UMG_API bool RemoveOption(FName Option);

	/** Remove all the elements of the option list. */
	UFUNCTION(BlueprintCallable, Category = "ComboBox")
	UMG_API void ClearOptions();

	/** Clear the current selection. */
	UFUNCTION(BlueprintCallable, Category = "ComboBox")
	UMG_API void ClearSelection();

	/** Set the current selected option. */
	UFUNCTION(BlueprintCallable, Category = "ComboBox")
	UMG_API void SetSelectedOption(FName Option);

	/** Get the current selected option */
	UFUNCTION(BlueprintCallable, Category = "ComboBox")
	UMG_API FName GetSelectedOption() const;

	/** Is the combobox menu opened. */
	UFUNCTION(BlueprintCallable, Category = "ComboBox", Meta = (ReturnDisplayName = "bOpen"))
	UMG_API bool IsOpen() const;

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	/** Set the padding for content. */
	UMG_API void SetContentPadding(FMargin InPadding);

	/** Get the padding for content. */
	UMG_API FMargin GetContentPadding() const;

	/** Is the combobox navigated by gamepad. */
	UMG_API bool IsEnableGamepadNavigationMode() const;

	/** Set whether the combobox is navigated by gamepad. */
	UMG_API void SetEnableGamepadNavigationMode(bool InEnableGamepadNavigationMode);

	/** Is the combobox arrow showing. */
	UMG_API bool IsHasDownArrow() const;

	/** Set whether the combobox arrow is showing. */
	UMG_API void SetHasDownArrow(bool InHasDownArrow);

	/** Get the maximum height of the combobox list. */
	UMG_API float GetMaxListHeight() const;

	/** Set the maximum height of the combobox list. */
	UMG_API void SetMaxListHeight(float InMaxHeight);

	/** Get the style of the combobox. */
	UMG_API const FComboBoxStyle& GetWidgetStyle() const;

	/** Set the style of the combobox. */
	UMG_API void SetWidgetStyle(const FComboBoxStyle& InWidgetStyle);

	/** Get the style of the items. */
	UMG_API const FTableRowStyle& GetItemStyle() const;

	/** Set the style of the items. */
	UMG_API void SetItemStyle(const FTableRowStyle& InItemStyle);

	/** Get the style of the scrollbar. */
	UMG_API const FScrollBarStyle& GetScrollBarStyle() const;

	/** Is the combobox focusable. */
	UMG_API bool IsFocusable() const;

	/** Get the foreground color of the button. */
	UMG_API FSlateColor GetForegroundColor() const;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	/** Initialize the scrollbar style in the constructor before the SWidget is constructed. */
	UMG_API void InitScrollBarStyle(const FScrollBarStyle& InScrollBarStyle);

	/** Initialize IsFocusable in the constructor before the SWidget is constructed. */
	UMG_API void InitIsFocusable(bool InIsFocusable);

	/** Initialize ForegroundColor in the constructor before the SWidget is constructed. */
	UMG_API void InitForegroundColor(FSlateColor InForegroundColor);

private:
	/** Called by slate when it needs to generate the widget in the content box */
	void GenerateContent();

	/** Called by slate when it needs to generate a new item for the combobox */
	TSharedRef<SWidget> HandleGenerateItemWidget(FName Item);

	/** Called by slate when the underlying combobox selection changes */
	void HandleSelectionChanged(FName Item, ESelectInfo::Type SelectionType);

	/** Called by slate when the underlying combobox is opening */
	void HandleOpening();

	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

private:
	/** A shared pointer to the underlying slate combobox */
	TSharedPtr<SComboBox<FName>> MyComboBox;

	/** A shared pointer to a container that holds the combobox content that is selected */
	TSharedPtr< SBox > ComboBoxContent;
};
