// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Components/Widget.h"

#include "ComboBoxString.generated.h"

/**
 * The combobox allows you to display a list of options to the user in a dropdown menu for them to select one.
 */
UCLASS(meta=( DisplayName="ComboBox (String)"), MinimalAPI)
class UComboBoxString : public UWidget
{
	GENERATED_UCLASS_BODY()

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnSelectionChangedEvent, FString, SelectedItem, ESelectInfo::Type, SelectionType);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnOpeningEvent);

private:

	/** The default list of items to be displayed on the combobox. */
	UPROPERTY(EditAnywhere, Category=Content)
	TArray<FString> DefaultOptions;

	/** The item in the combobox to select by default */
	UPROPERTY(EditAnywhere, FieldNotify, Category=Content)
	FString SelectedOption;

public:

	UE_DEPRECATED(5.2, "Direct access to WidgetStyle is deprecated. Please use the getter or setter.")
	/** The style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Style, meta=( DisplayName="Style" ))
	FComboBoxStyle WidgetStyle;

	UE_DEPRECATED(5.2, "Direct access to ItemStyle is deprecated. Please use the getter or setter.")
	/** The item row style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Style)
	FTableRowStyle ItemStyle;
	
	UE_DEPRECATED(5.2, "Direct access to ScrollBarStyle is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** The scroll bar style. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Style")
    FScrollBarStyle ScrollBarStyle;

	UE_DEPRECATED(5.2, "Direct access to ContentPadding is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Content)
	FMargin ContentPadding;

	UE_DEPRECATED(5.2, "Direct access to MaxListHeight is deprecated. Please use the getter or setter.")
	/** The max height of the combobox list that opens */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Content, AdvancedDisplay)
	float MaxListHeight;

	UE_DEPRECATED(5.2, "Direct access to HasDownArrow is deprecated. Please use the getter or setter.")
	/**
	 * When false, the down arrow is not generated and it is up to the API consumer
	 * to make their own visual hint that this is a drop down.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsHasDownArrow", Setter = "SetHasDownArrow", Category = Content, AdvancedDisplay)
	bool HasDownArrow;

	UE_DEPRECATED(5.2, "Direct access to EnableGamepadNavigationMode is deprecated. Please use the getter or setter.")
	/**
	* When false, directional keys will change the selection. When true, ComboBox 
	* must be activated and will only capture arrow input while activated.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsEnableGamepadNavigationMode", Setter = "SetEnableGamepadNavigationMode", Category = Content, AdvancedDisplay)
	bool EnableGamepadNavigationMode;

	UE_DEPRECATED(5.2, "Direct access to Font is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/**
	 * The default font to use in the combobox, only applies if you're not implementing OnGenerateWidgetEvent
	 * to factory each new entry.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category=Style)
	FSlateFontInfo Font;

	UE_DEPRECATED(5.2, "Direct access to ForegroundColor is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** The foreground color to pass through the hierarchy. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category=Style, meta=(DesignerRebuild))
	FSlateColor ForegroundColor;

	UE_DEPRECATED(5.2, "Direct access to bIsFocusable is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter= "IsFocusable", Category=Interaction)
	bool bIsFocusable;

public: // Events

	/** Called when the widget is needed for the item. */
	UPROPERTY(EditAnywhere, Category=Events, meta=( IsBindableEvent="True" ))
	FGenerateWidgetForString OnGenerateWidgetEvent;

	/** Called when a new item is selected in the combobox. */
	UPROPERTY(BlueprintAssignable, Category=Events)
	FOnSelectionChangedEvent OnSelectionChanged;

	/** Called when the combobox is opening */
	UPROPERTY(BlueprintAssignable, Category=Events)
	FOnOpeningEvent OnOpening;

public:

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API void AddOption(const FString& Option);

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API bool RemoveOption(const FString& Option);

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API int32 FindOptionIndex(const FString& Option) const;

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API FString GetOptionAtIndex(int32 Index) const;

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API void ClearOptions();

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API void ClearSelection();

	/**
	 * Refreshes the list of options.  If you added new ones, and want to update the list even if it's
	 * currently being displayed use this.
	 */
	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API void RefreshOptions();

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API void SetSelectedOption(FString Option);

	UFUNCTION(BlueprintCallable, Category = "ComboBox")
	UMG_API void SetSelectedIndex(const int32 Index);

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API FString GetSelectedOption() const;

	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API int32 GetSelectedIndex() const;

	/** Returns the number of options */
	UFUNCTION(BlueprintCallable, Category="ComboBox")
	UMG_API int32 GetOptionCount() const;
	
	UFUNCTION(BlueprintCallable, Category="ComboBox", Meta = (ReturnDisplayName = "bOpen"))
	UMG_API bool IsOpen() const;

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	/** Set the padding for content. */
	UMG_API void SetContentPadding(FMargin InPadding);

	/** Get the padding for content. */
	UMG_API FMargin GetContentPadding() const;

	/** Is the combobox navigated by gamepad. */
	UMG_API bool IsEnableGamepadNavigationMode() const;

	/** Set whether the combobox is navigated by gamepad. */
	UMG_API void SetEnableGamepadNavigationMode(bool InEnableGamepadNavigationMode);

	/** Is the combobox arrow showing. */
	UMG_API bool IsHasDownArrow() const;

	/** Set whether the combobox arrow is showing. */
	UMG_API void SetHasDownArrow(bool InHasDownArrow);

	/** Get the maximum height of the combobox list. */
	UMG_API float GetMaxListHeight() const;

	/** Set the maximum height of the combobox list. */
	UMG_API void SetMaxListHeight(float InMaxHeight);

	/** Get the default font for Combobox if no OnGenerateWidgetEvent is specified. */
	UMG_API const FSlateFontInfo& GetFont() const;

	/** Get the style of the combobox. */
	UMG_API const FComboBoxStyle& GetWidgetStyle() const;

	/** Set the style of the combobox. */
	UMG_API void SetWidgetStyle(const FComboBoxStyle& InWidgetStyle);

	/** Get the style of the items. */
	UMG_API const FTableRowStyle& GetItemStyle() const;

	/** Set the style of the items. */
	UMG_API void SetItemStyle(const FTableRowStyle& InItemStyle);

	/** Get the style of the scrollbar. */
	UMG_API const FScrollBarStyle& GetScrollBarStyle() const;

	/** Is the combobox focusable. */
	UMG_API bool IsFocusable() const;

	/** Get the foreground color of the button. */
	UMG_API FSlateColor GetForegroundColor() const;

	//~ Begin UObject Interface
	UMG_API virtual void PostInitProperties() override;
	UMG_API virtual void Serialize(FArchive& Ar) override;
	UMG_API virtual void PostLoad() override;
	//~ End UObject Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	/** Refresh ComboBoxContent with the correct widget/data when the selected option changes */
	UMG_API void UpdateOrGenerateWidget(TSharedPtr<FString> Item);

	/** Called by slate when it needs to generate a new item for the combobox */
	UMG_API virtual TSharedRef<SWidget> HandleGenerateWidget(TSharedPtr<FString> Item) const;

	/** Called by slate when the underlying combobox selection changes */
	UMG_API virtual void HandleSelectionChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectionType);

	/** Called by slate when the underlying combobox is opening */
	UMG_API virtual void HandleOpening();

	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

	/** Initialize the scrollbar style in the constructor before the SWidget is constructed. */
	UMG_API void InitScrollBarStyle(const FScrollBarStyle& InScrollBarStyle);

	/** Initialize the default font in the constructor before the SWidget is constructed. */
	UMG_API void InitFont(FSlateFontInfo InFont);

	/** Initialize IsFocusable in the constructor before the SWidget is constructed. */
	UMG_API void InitIsFocusable(bool InIsFocusable);

	/** Initialize ForegroundColor in the constructor before the SWidget is constructed. */
	UMG_API void InitForegroundColor(FSlateColor InForegroundColor);

protected:
	/** The true objects bound to the Slate combobox. */
	TArray< TSharedPtr<FString> > Options;

	/** A shared pointer to the underlying slate combobox */
	TSharedPtr< SComboBox< TSharedPtr<FString> > > MyComboBox;

	/** A shared pointer to a container that holds the combobox content that is selected */
	TSharedPtr< SBox > ComboBoxContent;

	/** If OnGenerateWidgetEvent is not bound, this will store the default STextBlock generated */
	TWeakPtr<STextBlock> DefaultComboBoxContent;

	/** A shared pointer to the current selected string */
	TSharedPtr<FString> CurrentOptionPtr;
};
