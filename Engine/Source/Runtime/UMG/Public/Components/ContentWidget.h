// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PanelWidget.h"
#include "ContentWidget.generated.h"

/**  */
UCLASS(Abstract, MinimalAPI)
class UContentWidget : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:
	/**  */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API UPanelSlot* GetContentSlot() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API UPanelSlot* SetContent(UWidget* Content);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API UWidget* GetContent() const;

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	// End UPanelWidget
};
