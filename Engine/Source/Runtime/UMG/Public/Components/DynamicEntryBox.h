// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DynamicEntryBoxBase.h"

#include "DynamicEntryBox.generated.h"

/**
 * A special box panel that auto-generates its entries at both design-time and runtime.
 * Useful for cases where you can have a varying number of entries, but it isn't worth the effort or conceptual overhead to set up a list/tile view.
 * Note that entries here are *not* virtualized as they are in the list views, so generally this should be avoided if you intend to scroll through lots of items.
 *
 * No children can be manually added in the designer - all are auto-generated based on the given entry class.
 */
UCLASS(MinimalAPI)
class UDynamicEntryBox : public UDynamicEntryBoxBase
{
	GENERATED_BODY()

public:
	TSubclassOf<UUserWidget> GetEntryWidgetClass() const { return EntryWidgetClass; }
	
	template <typename WidgetT = UUserWidget>
	WidgetT* CreateEntry(const TSubclassOf<WidgetT>& ExplicitEntryClass = {})
	{
		TSubclassOf<UUserWidget> EntryClass = ExplicitEntryClass ? ExplicitEntryClass : EntryWidgetClass;
		if (EntryClass && EntryClass->IsChildOf(WidgetT::StaticClass()) && IsEntryClassValid(EntryClass))
		{
			return Cast<WidgetT>(CreateEntryInternal(EntryClass));
		}
		return nullptr;
	}
	
	template <typename WidgetT = UUserWidget>
	void Reset(TFunctionRef<void(WidgetT&)> ResetEntryFunc, bool bDeleteWidgets = false)
	{
		for (UUserWidget* EntryWidget : GetAllEntries())
		{
			if (WidgetT* TypedEntry = Cast<WidgetT>(EntryWidget))
			{
				ResetEntryFunc(*TypedEntry);
			}
		}

		Reset(bDeleteWidgets);
	}

	/** Clear out the box entries, optionally deleting the underlying Slate widgets entirely as well. */
	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox)
	UMG_API void Reset(bool bDeleteWidgets = false);

	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox)
	UMG_API void RemoveEntry(UUserWidget* EntryWidget);

	//~ Begin UWidget Interface
#if WITH_EDITOR	
	UMG_API virtual void ValidateCompiledDefaults(class IWidgetCompilerLog& CompileLog) const override;
#endif
	//~ End UWidget Interface

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = DynamicEntryBox, meta = (ClampMin = 0, ClampMax = 20))
	int32 NumDesignerPreviewEntries = 3;

	/** 
	 * Called whenever a preview entry is made for this widget in the designer. 
	 * Intended to allow a containing widget to do any additional modifications needed in the interest of maintaining an accurate designer preview.
	 */
	TFunction<void(UUserWidget*)> OnPreviewEntryCreatedFunc;
#endif

	UMG_API virtual void SynchronizeProperties() override;

private:
	/** Creates and establishes a new dynamic entry in the box */
	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox, meta = (DisplayName = "Create Entry", AllowPrivateAccess = true))
	UMG_API UUserWidget* BP_CreateEntry();

	/** Creates and establishes a new dynamic entry in the box using the specified class instead of the default. */
	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox, meta = (DisplayName = "Create Entry of Class", AllowPrivateAccess = true, DeterminesOutputType = "EntryClass"))
	UMG_API UUserWidget* BP_CreateEntryOfClass(TSubclassOf<UUserWidget> EntryClass);

	/**
	 * The class of widget to create entries of.
	 * If natively binding this widget, use the EntryClass UPROPERTY metadata to specify the desired entry widget base class.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = EntryLayout, meta = (AllowPrivateAccess = true))
	TSubclassOf<UUserWidget> EntryWidgetClass;
	
	// Let the details customization manipulate us directly
	friend class FDynamicEntryBoxDetails;
};
