// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "Blueprint/UserWidgetPool.h"
#include "RadialBoxSettings.h"
#include "Widgets/SPanel.h"
#include "DynamicEntryBoxBase.generated.h"

class UUserWidget;

UENUM(BlueprintType)
enum class EDynamicBoxType : uint8
{
	Horizontal,
	Vertical,
	Wrap,
	VerticalWrap,
	Radial,
	Overlay
};

/**
 * Base for widgets that support a dynamic number of auto-generated entries at both design- and run-time.
 * Contains all functionality needed to create, construct, and cache an arbitrary number of entry widgets, but exposes no means of entry creation or removal
 * It's up to child classes to decide how they want to perform the population (some may do so entirely on their own without exposing a thing)
 *
 * @see UDynamicEntryBox for a ready-to-use version
 */
UCLASS(Abstract, MinimalAPI)
class UDynamicEntryBoxBase : public UWidget
{
	GENERATED_BODY()

protected:

	UE_DEPRECATED(5.2, "Direct access to EntrySpacing is deprecated. Please use the getter or setter.")
	/**
	 * The padding to apply between entries in the box.
	 * Note horizontal boxes only use the X and vertical boxes only use Y. Value is also ignored for the first entry in the box.
	 * Wrap and Overlay types use both X and Y for spacing.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetEntrySpacing", Category = EntryLayout)
	FVector2D EntrySpacing;

	//@todo DanH EntryBox: Consider giving a callback option as well/instead. Then this thing could actually create circular or pinwheel layouts...
	/** The looping sequence of entry paddings to apply as entries are created. Overlay boxes only. Ignores EntrySpacing if not empty. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = EntryLayout)
	TArray<FVector2D> SpacingPattern;

	UE_DEPRECATED(5.2, "Direct access to EntryBoxType is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** The type of box panel into which created entries are added. Some differences in functionality exist between each type. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "GetBoxType", Category = DynamicEntryBox, meta = (DesignerRebuild))
	EDynamicBoxType EntryBoxType;

	UE_DEPRECATED(5.2, "Direct access to EntrySizeRule is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** Sizing rule to apply to generated entries. Horizontal/Vertical boxes only. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category = EntryLayout)
	FSlateChildSize EntrySizeRule;

	UE_DEPRECATED(5.2, "Direct access to EntryHorizontalAlignment is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** Horizontal alignment of generated entries. Horizontal/Vertical/Wrap boxes only. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category = EntryLayout)
	TEnumAsByte<EHorizontalAlignment> EntryHorizontalAlignment;

	UE_DEPRECATED(5.2, "Direct access to EntryVerticalAlignment is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** Vertical alignment of generated entries. Horizontal/Vertical/Wrap boxes only. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category = EntryLayout)
	TEnumAsByte<EVerticalAlignment> EntryVerticalAlignment;

	UE_DEPRECATED(5.2, "Direct access to MaxElementSize is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/** The maximum size of each entry in the dominant axis of the box. Vertical/Horizontal boxes only. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category = EntryLayout)
	int32 MaxElementSize = 0;

	UE_DEPRECATED(5.2, "Direct access to RadialBoxSettings is deprecated. Please use the getter or setter.")
	/** Settings only relevant to RadialBox */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter = "SetRadialSettings", BlueprintSetter = "SetRadialSettings", Category = EntryLayout)
	FRadialBoxSettings RadialBoxSettings;

public:
	UMG_API UDynamicEntryBoxBase(const FObjectInitializer& Initializer);
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	UMG_API EDynamicBoxType GetBoxType() const;
	UMG_API const FVector2D& GetEntrySpacing() const;

	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox)
	UMG_API const TArray<UUserWidget*>& GetAllEntries() const;

	UMG_API const FSlateChildSize& GetEntrySizeRule() const;

	UMG_API const FRadialBoxSettings& GetRadialBoxSettings() const;

	template <typename EntryWidgetT = UUserWidget>
	TArray<EntryWidgetT*> GetTypedEntries() const
	{
		TArray<EntryWidgetT*> TypedEntries;
		for (UUserWidget* Entry : GetAllEntries())
		{
			if (EntryWidgetT* TypedEntry = Cast<EntryWidgetT>(Entry))
			{
				TypedEntries.Add(TypedEntry);
			}
		}
		return TypedEntries;
	}

	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox)
	UMG_API int32 GetNumEntries() const;

	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox)
	UMG_API void SetEntrySpacing(const FVector2D& InEntrySpacing);

	UFUNCTION(BlueprintCallable, Category = DynamicEntryBox)
	UMG_API void SetRadialSettings(const FRadialBoxSettings& InSettings);

	UMG_API EVerticalAlignment GetEntryVerticalAlignment() const;

	UMG_API EHorizontalAlignment GetEntryHorizontalAlignment() const;

	UMG_API int32 GetMaxElementSize() const;

protected:
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UMG_API virtual void SynchronizeProperties() override;
	UMG_API virtual void AddEntryChild(UUserWidget& ChildWidget);

	UMG_API bool IsEntryClassValid(TSubclassOf<UUserWidget> InEntryClass) const;
	UMG_API UUserWidget* CreateEntryInternal(TSubclassOf<UUserWidget> InEntryClass);
	UMG_API void RemoveEntryInternal(UUserWidget* EntryWidget, bool bReleaseSlate = false);
	UMG_API FMargin BuildEntryPadding(const FVector2D& DesiredSpacing);

	/** Clear out the box entries, optionally deleting the underlying Slate widgets entirely as well. */
	UMG_API void ResetInternal(bool bDeleteWidgets = false);

	/** Clear out the box entries, executing the provided reset function for each and optionally deleting the underlying Slate widgets entirely as well. */
	template <typename WidgetT = UUserWidget>
	void ResetInternal(TFunctionRef<void(WidgetT&)> ResetEntryFunc, bool bDeleteWidgets = false)
	{
		for (UUserWidget* EntryWidget : EntryWidgetPool.GetActiveWidgets())
		{
			if (WidgetT* TypedEntry = Cast<WidgetT>(EntryWidget))
			{
				ResetEntryFunc(*TypedEntry);
			}
		}

		ResetInternal(bDeleteWidgets);
	}

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// Initialize EntryBoxType in the constructor before the SWidget is constructed.
	UMG_API void InitEntryBoxType(EDynamicBoxType InEntryBoxType);

	// Initialize EntrySizeRule in the constructor before the SWidget is constructed.
	UMG_API void InitEntrySizeRule(FSlateChildSize InEntrySizeRule);

	// Initialize EntryHorizontalAlignment in the constructor before the SWidget is constructed.
	UMG_API void InitEntryHorizontalAlignment(EHorizontalAlignment InEntryHorizontalAlignment);

	// Initialize EntryVerticalAlignment in the constructor before the SWidget is constructed.
	UMG_API void InitEntryVerticalAlignment(EVerticalAlignment InEntryVerticalAlignment);

	// Initialize MaxElementSize in the constructor before the SWidget is constructed.
	UMG_API void InitMaxElementSize(int32 InMaxElementSize);

protected:
	// Can be a horizontal, vertical, wrap box, or overlay
	TSharedPtr<SPanel> MyPanelWidget;

private:
	UPROPERTY(Transient)
	FUserWidgetPool EntryWidgetPool;

	// Let the details customization manipulate us directly
	friend class FDynamicEntryBoxBaseDetails;
};
