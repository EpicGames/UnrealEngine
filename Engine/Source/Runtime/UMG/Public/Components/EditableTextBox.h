// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Margin.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Components/TextWidgetTypes.h"
#include "Widgets/Text/ISlateEditableTextWidget.h"
#include "Framework/Text/TextLayout.h"
#include "EditableTextBox.generated.h"

class SEditableTextBox;
class USlateWidgetStyleAsset;

/**
 * Allows the user to type in custom text.  Only permits a single line of text to be entered.
 * 
 * * No Children
 * * Text Entry
 */
UCLASS(meta=(DisplayName="Text Box"), MinimalAPI)
class UEditableTextBox : public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnEditableTextBoxChangedEvent, const FText&, Text);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnEditableTextBoxCommittedEvent, const FText&, Text, ETextCommit::Type, CommitMethod);
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnEditableTextBoxCursorMovedWithSelectionEvent, const FTextLocation&, const FTextSelection&);
	DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(FOnAllFontFacesFinishLoading_EditableTextBox, UEditableTextBox, OnAllFontFacesFinishLoading);
public:
	UE_DEPRECATED(5.1, "Direct access to Text is deprecated. Please use the getter or setter.")
	/** The text content for this editable text box widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter = "GetText", BlueprintSetter = "SetText", FieldNotify, Category = "Content")
	FText Text;

	/** A bindable delegate to allow logic to drive the text of the widget */
	UPROPERTY()
	FGetText TextDelegate;

public:

	/** The style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Appearance, meta=(DisplayName="Style", ShowOnlyInnerProperties))
	FEditableTextBoxStyle WidgetStyle;

	UE_DEPRECATED(5.1, "Direct access to Hint Text is deprecated. Please use the getter or setter.")
	/** Hint text that appears when there is no text in the text box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetHintText", Category = Content, meta = (MultiLine = "true"))
	FText HintText;

	/** A bindable delegate to allow logic to drive the hint text of the widget */
	UPROPERTY()
	FGetText HintTextDelegate;

	UE_DEPRECATED(5.1, "Direct access to IsReadOnly is deprecated. Please use the getter or setter.")
	/** Sets the Text Box as Readonly to prevent it from being modified interactively by the user */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Appearance)
	bool IsReadOnly;

	UE_DEPRECATED(5.1, "Direct access to IsPassword is deprecated. Please use the getter or setter.")
	/** Sets whether this text box is for storing a password */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetIsPassword", Category = Appearance)
	bool IsPassword;

	UE_DEPRECATED(5.1, "Direct access to MinimumDesiredWidth is deprecated. Please use the getter or setter.")
	/** The minimum desired size for the text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter=SetMinDesiredWidth, Category = "Appearance")
	float MinimumDesiredWidth;

	UE_DEPRECATED(5.1, "Direct access to IsCaretMovedWhenGainFocus is deprecated. Please use the getter or setter.")
	/** Workaround as we lose focus when the auto completion closes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Behavior, AdvancedDisplay)
	bool IsCaretMovedWhenGainFocus;

	UE_DEPRECATED(5.1, "Direct access to SelectAllTextWhenFocused is deprecated. Please use the getter or setter.")
	/** Whether to select all text when the user clicks to give focus on the widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Behavior, AdvancedDisplay)
	bool SelectAllTextWhenFocused;

	UE_DEPRECATED(5.1, "Direct access to RevertTextOnEscape is deprecated. Please use the getter or setter.")
	/** Whether to allow the user to back out of changes when they press the escape key */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Behavior, AdvancedDisplay)
	bool RevertTextOnEscape;

	UE_DEPRECATED(5.1, "Direct access to ClearKeyboardFocusOnCommit is deprecated. Please use the getter or setter.")
	/** Whether to clear keyboard focus when pressing enter to commit changes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Behavior, AdvancedDisplay)
	bool ClearKeyboardFocusOnCommit;

	UE_DEPRECATED(5.1, "Direct access to SelectAllTextOnCommit is deprecated. Please use the getter or setter.")
	/** Whether to select all text when pressing enter to commit changes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Behavior, AdvancedDisplay)
	bool SelectAllTextOnCommit;

	/** Whether the context menu can be opened */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	bool AllowContextMenu;

	/** If we're on a platform that requires a virtual keyboard, what kind of keyboard should this widget use? */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	TEnumAsByte<EVirtualKeyboardType::Type> KeyboardType;

	/** Additional options to use for the virtual keyboard summoned by this widget */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	FVirtualKeyboardOptions VirtualKeyboardOptions;

	/** The type of event that will trigger the display of the virtual keyboard */
	UPROPERTY(EditAnywhere, Category = Behavior, AdvancedDisplay)
	EVirtualKeyboardTrigger VirtualKeyboardTrigger;

	/** What action should be taken when the virtual keyboard is dismissed? */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	EVirtualKeyboardDismissAction VirtualKeyboardDismissAction;
	
	UE_DEPRECATED(5.1, "Direct access to Justification is deprecated. Please use the getter or setter.")
	/** How the text should be aligned with the margin. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetJustification, Category = Appearance)
	TEnumAsByte<ETextJustify::Type> Justification;

	UE_DEPRECATED(5.1, "Direct access to OverflowPolicy is deprecated. Please use the getter or setter.")
	/** Sets what happens to text that is clipped and doesn't fit within the clip rect for this widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="GetTextOverflowPolicy", Setter= "SetTextOverflowPolicy", BlueprintSetter = "SetTextOverflowPolicy", Category = "Clipping", AdvancedDisplay, meta = (DisplayName = "Overflow Policy"))
	ETextOverflowPolicy OverflowPolicy;


	/** Controls how the text within this widget should be shaped. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Localization, AdvancedDisplay, meta=(ShowOnlyInnerProperties))
	FShapedTextOptions ShapedTextOptions;

public:

	/** Called whenever the text is changed programmatically or interactively by the user */
	UPROPERTY(BlueprintAssignable, Category="TextBox|Event")
	FOnEditableTextBoxChangedEvent OnTextChanged;

	/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
	UPROPERTY(BlueprintAssignable, Category="TextBox|Event")
	FOnEditableTextBoxCommittedEvent OnTextCommitted;

	/** Called whenever the user's selection and/or cursor position of the text has changed. */
	FOnEditableTextBoxCursorMovedWithSelectionEvent OnCursorMovedWithSelectionEvent;
	
	/** Provide a alternative mechanism for error reporting. */
	//SLATE_ARGUMENT(TSharedPtr<class IErrorReportingWidget>, ErrorReporting)

public:

	/**
	* Gets the widget text
	* @return The widget text
	*/
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="GetText (Text Box)"))
	UMG_API FText GetText() const;

	/**
	* Directly sets the widget text.
	* Warning: This will wipe any binding created for the Text property!
	* @param InText The text to assign to the widget
	*/
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetText (Text Box)"))
	UMG_API void SetText(FText InText);
	
	/** Gets the Hint text that appears when there is no text in the text box */
	UMG_API FText GetHintText() const;

	/**
	* Sets the Hint text that appears when there is no text in the text box
	* @param InHintText The text that appears when there is no text in the text box
	*/
	UFUNCTION(BlueprintCallable, Category = "Widget", meta = (DisplayName = "Set Hint Text (Text Box)"))
	UMG_API void SetHintText(FText InText);

	/** @return the minimum desired width for this text box */
	UMG_API float GetMinimumDesiredWidth() const;

	/**
	*  Set the minimum desired width for this text box
	*
	*  @param InMinDesiredWidth new minimum desired width
	*/
	UMG_API void SetMinDesiredWidth(float InMinDesiredWidth);
	
	/** When set to true the caret is moved when gaining focus */
	UMG_API void SetIsCaretMovedWhenGainFocus(bool bIsCaretMovedWhenGainFocus);

	/** Return true when the caret is moved when gaining focus */
	UMG_API bool GetIsCaretMovedWhenGainFocus() const;

	/** Set to true to select all text when the user clicks to give focus on the widget */
	UMG_API void SetSelectAllTextWhenFocused(bool bSelectAllTextWhenFocused);

	/** Whether to select all text when the user clicks to give focus on the widget */
	UMG_API bool GetSelectAllTextWhenFocused() const;

	/** Set to true to allow the user to back out of changes when they press the escape key */
	UMG_API void SetRevertTextOnEscape(bool bRevertTextOnEscape);

	/** Whether to allow the user to back out of changes when they press the escape key  */
	UMG_API bool GetRevertTextOnEscape() const;

	/** Set to true to clear keyboard focus when pressing enter to commit changes */
	UMG_API void SetClearKeyboardFocusOnCommit(bool bClearKeyboardFocusOnCommit);

	/** Whether to clear keyboard focus when pressing enter to commit changes */
	UMG_API bool GetClearKeyboardFocusOnCommit() const;

	/** Set to true to select all text when pressing enter to commit changes */
	UMG_API void SetSelectAllTextOnCommit(bool bSelectAllTextOnCommit);

	/** Whether to select all text when pressing enter to commit changes */
	UMG_API bool GetSelectAllTextOnCommit() const;

	/** Whether any text is selected in the text block */
	UMG_API bool IsAnyTextSelected() const;

	UFUNCTION(BlueprintCallable, Category="Widget",  meta=(DisplayName="SetError (Text Box)"))
	UMG_API void SetError(FText InError);

	/** Return true when this text cannot be modified interactively by the user */
	UMG_API bool GetIsReadOnly() const;

	/** Sets the Text as Readonly to prevent it from being modified interactively by the user */
	UFUNCTION(BlueprintCallable, Category = "Widget", meta = (DisplayName = "SetIsReadOnly (Editable Text)"))
	UMG_API void SetIsReadOnly(UPARAM(DisplayName = "ReadyOnly") bool bReadOnly);

	UMG_API bool GetIsPassword() const;

	UFUNCTION(BlueprintCallable, Category = "Widget", meta = (DisplayName = "IsPassword"))
	UMG_API void SetIsPassword(bool bIsPassword);

	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void ClearError();

	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API bool HasError() const;

	UMG_API ETextJustify::Type GetJustification() const;

	UFUNCTION(BlueprintSetter)
	UMG_API void SetJustification(ETextJustify::Type InJustification);

	/** @return the text overflow policy for this text block. */
	UMG_API ETextOverflowPolicy GetTextOverflowPolicy() const;

	/**
	 * Set the text overflow policy for this text box.
	 *
	 * @param InOverflowPolicy the new text overflow policy.
	 */
	UFUNCTION(BlueprintSetter)
	UMG_API void SetTextOverflowPolicy(ETextOverflowPolicy InOverflowPolicy);

	UMG_API void SetWidgetStyle(const FEditableTextBoxStyle& InStyle);

	UMG_API const FEditableTextBoxStyle& GetWidgetStyle() const;

	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetForegroundColor (Text Box)"))
	UMG_API void SetForegroundColor(FLinearColor color);

	UMG_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif
	UMG_API virtual void Serialize(FArchive& Ar) override;

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UMG_API virtual void HandleOnTextChanged(const FText& Text);
	UMG_API virtual void HandleOnTextCommitted(const FText& Text, ETextCommit::Type CommitMethod);
	UMG_API virtual void HandleOnCursorMovedWithSelection(const FTextLocation& InLocation, const FTextSelection& Selection);

#if WITH_ACCESSIBILITY
	UMG_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const override;
#endif

	void SlateHandleAllFontFacesFinishLoading()
	{
		OnAllFontFacesFinishLoading.Broadcast();
	}
protected:
	TSharedPtr<SEditableTextBox> MyEditableTextBlock;

	PROPERTY_BINDING_IMPLEMENTATION(FText, Text);
	PROPERTY_BINDING_IMPLEMENTATION(FText, HintText);

private:
	/** @return true if the text was changed, or false if identical. */
	bool SetTextInternal(const FText& InText);

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bIsFontDeprecationDone;
#endif

protected:
	/** Controls how painting the text in the widget should behave when font faces for the text is still loading. */
	UPROPERTY(EditAnywhere, Setter, Category = Font)
	EFontFacesLoadingPaintPolicy FontFacesLoadingPaintPolicy;

	/** Called when all font faces for the text has finished async loading. Is not called if font faces are synchronously loaded. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event", meta = (DisplayName = "OnAllFontFacesFinishLoading(Editable Text)"))
	FOnAllFontFacesFinishLoading_EditableTextBox OnAllFontFacesFinishLoading;
};
