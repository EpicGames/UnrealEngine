// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Components/NamedSlotInterface.h"
#include "ExpandableArea.generated.h"

class UExpandableArea;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnExpandableAreaExpansionChanged, UExpandableArea*, Area, bool, bIsExpanded);

/**
 * 
 */
UCLASS(MinimalAPI)
class UExpandableArea : public UWidget, public INamedSlotInterface
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.2, "Direct access to Style is deprecated. Please use SetStyle or GetStyle.")
	UPROPERTY(EditAnywhere, Category = "Style")
	FExpandableAreaStyle Style;

	UE_DEPRECATED(5.2, "Direct access to BorderBrush is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Style" )
	FSlateBrush BorderBrush;

	UE_DEPRECATED(5.2, "Direct access to BorderColor is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Style")
	FSlateColor BorderColor;

	/**  */
	UE_DEPRECATED(5.2, "Direct access to bIsExpanded is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetIsExpanded", Setter = "SetIsExpanded", BlueprintGetter = "GetIsExpanded", BlueprintSetter = "SetIsExpanded", FieldNotify, Category = "Expansion")
	bool bIsExpanded;

	UE_DEPRECATED(5.2, "Direct access to MaxHeight is deprecated. Please use the getter or setter.")
	/** The maximum height of the area */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Expansion")
	float MaxHeight;
	
	UE_DEPRECATED(5.2, "Direct access to HeaderPadding is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Expansion" )
	FMargin HeaderPadding;
	
	/**  */
	UE_DEPRECATED(5.2, "Direct access to AreaPadding is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Expansion")
	FMargin AreaPadding;

	/** A bindable delegate for the IsChecked. */
	UPROPERTY(BlueprintAssignable, Category="ExpandableArea|Event")
	FOnExpandableAreaExpansionChanged OnExpansionChanged;

public:

	UFUNCTION(BlueprintCallable, Category="Expansion")
	UMG_API bool GetIsExpanded() const;

	UFUNCTION(BlueprintCallable, Category="Expansion")
	UMG_API void SetIsExpanded(bool IsExpanded);

	UFUNCTION(BlueprintCallable, Category = "Expansion")
	UMG_API void SetIsExpanded_Animated(bool IsExpanded);
	
	UMG_API const FExpandableAreaStyle& GetStyle() const;

	UMG_API void SetStyle(const FExpandableAreaStyle& InStyle);

	UMG_API const FSlateBrush& GetBorderBrush() const;

	UMG_API void SetBorderBrush(const FSlateBrush& InBorderBrush);

	UMG_API const FSlateColor& GetBorderColor() const;

	UMG_API void SetBorderColor(const FSlateColor& InBorderColor);

	UMG_API float GetMaxHeight() const;

	UMG_API void SetMaxHeight(float InMaxHeight);

	UMG_API FMargin GetHeaderPadding() const;

	UMG_API void SetHeaderPadding(FMargin InHeaderPadding);

	UMG_API FMargin GetAreaPadding() const;
	UMG_API void SetAreaPadding(FMargin InAreaPadding);

	// Begin INamedSlotInterface
	UMG_API virtual void GetSlotNames(TArray<FName>& SlotNames) const override;
	UMG_API virtual UWidget* GetContentForSlot(FName SlotName) const override;
	UMG_API virtual void SetContentForSlot(FName SlotName, UWidget* Content) override;
	// End INamedSlotInterface

public:
	
	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual void OnDescendantSelectedByDesigner(UWidget* DescendantWidget) override;
	UMG_API virtual void OnDescendantDeselectedByDesigner(UWidget* DescendantWidget) override;
#endif

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	UMG_API void SlateExpansionChanged(bool NewState);

protected:
	UPROPERTY()
	TObjectPtr<UWidget> HeaderContent;

	UPROPERTY()
	TObjectPtr<UWidget> BodyContent;

	TSharedPtr<SExpandableArea> MyExpandableArea;
};
