// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/PanelWidget.h"
#include "GridPanel.generated.h"

class IWidgetCompilerLog;
class SGridPanel;
class UGridSlot;

/**
 * A table-like panel that retains the width of every column throughout the table.
 */
UCLASS(MinimalAPI)
class UGridPanel : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:

	/** The column fill rules */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Fill Rules")
	TArray<float> ColumnFill;

	/** The row fill rules */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Fill Rules")
	TArray<float> RowFill;

public:

	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API UGridSlot* AddChildToGrid(UWidget* Content, int32 InRow = 0, int32 InColumn = 0);

	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetColumnFill(int32 ColumnIndex, float Coefficient);

	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetRowFill(int32 RowIndex, float Coefficient);

	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void ClearFill();

public:

	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:

	TSharedPtr<SGridPanel> MyGridPanel;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
