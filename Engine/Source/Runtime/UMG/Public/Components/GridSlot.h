// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/PanelSlot.h"
#include "Widgets/Layout/SGridPanel.h"

#include "GridSlot.generated.h"

/**
 * A slot for UGridPanel, these slots all share the same size as the largest slot
 * in the grid.
 */
UCLASS(MinimalAPI)
class UGridSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetPadding", Category="Layout|Grid Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Grid Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Grid Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;
	
	UE_DEPRECATED(5.1, "Direct access to Row is deprecated. Please use the getter or setter.")
	/** The row index of the cell this slot is in */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetRow", meta=( UIMin = "0" ), Category="Layout|Grid Slot")
	int32 Row;
	
	/**  */
	UE_DEPRECATED(5.1, "Direct access to RowSpan is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetRowSpan", Category="Layout|Grid Slot")
	int32 RowSpan;

	UE_DEPRECATED(5.1, "Direct access to Column is deprecated. Please use the getter or setter.")
	/** The column index of the cell this slot is in */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetColumn", meta=( UIMin = "0" ), Category="Layout|Grid Slot")
	int32 Column;

	/**  */
	UE_DEPRECATED(5.1, "Direct access to ColumnSpan is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetColumnSpan", Category="Layout|Grid Slot")
	int32 ColumnSpan;

	UE_DEPRECATED(5.1, "Direct access to Layer is deprecated. Please use the getter or setter.")
	/** Positive values offset this cell to be hit-tested and drawn on top of others. Default is 0; i.e. no offset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetLayer", Category="Layout|Grid Slot")
	int32 Layer;

	UE_DEPRECATED(5.1, "Direct access to Nudge is deprecated. Please use the getter or setter.")
	/** Offset this slot's content by some amount; positive values offset to lower right */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetNudge", Category="Layout|Grid Slot")
	FVector2D Nudge;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Border Slot")
	UMG_API void SetPadding(FMargin InPadding);

	/** Gets the row index of the slot */
	UMG_API int32 GetRow() const;

	/** Sets the row index of the slot, this determines what cell the slot is in the panel */
	UFUNCTION(BlueprintCallable, Category="Layout|Grid Slot")
	UMG_API void SetRow(int32 InRow);

	/** Gets how many rows this this slot spans over */
	UMG_API int32 GetRowSpan() const;

	/** How many rows this this slot spans over */
	UFUNCTION(BlueprintCallable, Category="Layout|Grid Slot")
	UMG_API void SetRowSpan(int32 InRowSpan);

	/** Gets the column index of the slot */
	UMG_API int32 GetColumn() const;

	/** Sets the column index of the slot, this determines what cell the slot is in the panel */
	UFUNCTION(BlueprintCallable, Category="Layout|Grid Slot")
	UMG_API void SetColumn(int32 InColumn);

	/** Gets how many columns this slot spans over */
	UMG_API int32 GetColumnSpan() const;

	/** How many columns this slot spans over */
	UFUNCTION(BlueprintCallable, Category="Layout|Grid Slot")
	UMG_API void SetColumnSpan(int32 InColumnSpan);

	/** Gets the offset this cell drawn on top of others. */
	UMG_API int32 GetLayer() const;

	/** Sets positive values offset this cell to be hit-tested and drawn on top of others. */
	UFUNCTION(BlueprintCallable, Category = "Layout|Grid Slot")
	UMG_API void SetLayer(int32 InLayer);

	/** Gets the offset for this slot's content */
	UMG_API FVector2D GetNudge() const;

	/** Sets the offset for this slot's content by some amount; positive values offset to lower right*/
	UFUNCTION(BlueprintCallable, Category = "Layout|Grid Slot")
	UMG_API void SetNudge(FVector2D InNudge);

	/** */
	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Layout|Grid Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	/**  */
	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Layout|Grid Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	// UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UPanelSlot interface

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SGridPanel> GridPanel);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual bool NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize) override;
	UMG_API virtual void SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot) override;
#endif //WITH_EDITOR

private:
	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SGridPanel::FSlot* Slot;
};
