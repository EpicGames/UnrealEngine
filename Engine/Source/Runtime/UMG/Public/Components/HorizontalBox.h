// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Components/PanelWidget.h"
#include "HorizontalBox.generated.h"

class UHorizontalBoxSlot;

/**
 * Allows widgets to be laid out in a flow horizontally.
 *
 * * Many Children
 * * Flow Horizontal
 */
UCLASS(meta = (ShortTooltip = "A layout panel for automatically laying child widgets out horizontally"), MinimalAPI)
class UHorizontalBox : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

	/**  */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API UHorizontalBoxSlot* AddChildToHorizontalBox(UWidget* Content);

#if WITH_EDITOR
	// UWidget interface
	UMG_API virtual const FText GetPaletteCategory() override;
	// End UWidget interface
#endif

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:

	TSharedPtr<class SHorizontalBox> MyHorizontalBox;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
