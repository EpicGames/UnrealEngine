// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Widgets/SBoxPanel.h"
#include "Components/PanelSlot.h"
#include "Components/SlateWrapperTypes.h"

#include "HorizontalBoxSlot.generated.h"

UCLASS(MinimalAPI)
class UHorizontalBoxSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

private:
	SHorizontalBox::FSlot* Slot;

public:
	
	UE_DEPRECATED(5.1, "Direct access to Size is deprecated. Please use the getter or setter.")
	/** How much space this slot should occupy in the direction of the panel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter = "SetSize", Category = "Layout|Horizontal Box Slot")
	FSlateChildSize Size;

	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The amount of padding between the slots parent and the content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter = "SetPadding", Category = "Layout|Horizontal Box Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter = "SetHorizontalAlignment", Category = "Layout|Horizontal Box Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter = "SetVerticalAlignment", Category = "Layout|Horizontal Box Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Horizontal Box Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API FSlateChildSize GetSize() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Horizontal Box Slot")
	UMG_API void SetSize(FSlateChildSize InSize);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Horizontal Box Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Horizontal Box Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

public:

	UMG_API void BuildSlot(TSharedRef<SHorizontalBox> HorizontalBox);

	// UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UPanelSlot interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual bool NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize) override;
	UMG_API virtual void SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot) override;
#endif //WITH_EDITOR
};
