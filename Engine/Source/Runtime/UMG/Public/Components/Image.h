// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Styling/SlateBrush.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "UObject/ScriptInterface.h"
#include "Slate/SlateTextureAtlasInterface.h"
#include "Engine/StreamableManager.h"
#include "Image.generated.h"

class SImage;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class USlateBrushAsset;
class UTexture2D;
struct FStreamableHandle;

/**
 * The image widget allows you to display a Slate Brush, or texture or material in the UI.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class UImage : public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.2, "Direct access to Brush is deprecated. Please use the getter or setter.")
	/** Image to draw */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetBrush", FieldNotify, Category=Appearance)
	FSlateBrush Brush;

	/** A bindable delegate for the Image. */
	UPROPERTY()
	FGetSlateBrush BrushDelegate;

	UE_DEPRECATED(5.2, "Direct access to ColorAndOpacity is deprecated. Please use the getter or setter.")
	/** Color and opacity */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetColorAndOpacity", Category=Appearance, meta=( sRGB="true") )
	FLinearColor ColorAndOpacity;

	/** A bindable delegate for the ColorAndOpacity. */
	UPROPERTY()
	FGetLinearColor ColorAndOpacityDelegate;

	UE_DEPRECATED(5.2, "Direct access to bFlipForRightToLeftFlowDirection is deprecated. Please use the getter or setter.")
	/** Flips the image if the localization's flow direction is RightToLeft */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "ShouldFlipForRightToLeftFlowDirection", Setter = "SetFlipForRightToLeftFlowDirection", Category = "Localization")
	bool bFlipForRightToLeftFlowDirection;

public:

	UPROPERTY(EditAnywhere, Category=Events, meta=( IsBindableEvent="True" ))
	FOnPointerEvent OnMouseButtonDownEvent;

public:

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetColorAndOpacity(FLinearColor InColorAndOpacity);

	UMG_API const FLinearColor& GetColorAndOpacity() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetOpacity(float InOpacity);

	/**  */
	UE_DEPRECATED(5.0, "Deprecated. Use SetDesiredSizeOverride instead.")
	UMG_API void SetBrushSize(FVector2D DesiredSize);

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetDesiredSizeOverride(FVector2D DesiredSize);

	/**  */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetBrushTintColor(FSlateColor TintColor);

	/**  */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetBrushResourceObject(UObject* ResourceObject);
	
	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API virtual void SetBrush(const FSlateBrush& InBrush);

	UMG_API const FSlateBrush& GetBrush() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API virtual void SetBrushFromAsset(USlateBrushAsset* Asset);

	/**
	* Sets the Brush to the specified Texture.
	*
	*   @param Texture Texture to use to set on Brush.
	*	@param bMatchSize If true, image will change its size to texture size. If false, texture will be stretched to image size.
	*/
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API virtual void SetBrushFromTexture(UTexture2D* Texture, bool bMatchSize = false);

	/**
	* Sets the Brush to the specified Atlas Region.
	*
	*   @param AtlasRegion Region of the Atlas to use to set on Brush.
	*	@param bMatchSize If true, image will change its size to atlas region size. If false, atlas region will be stretched to image size.
	*/
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API virtual void SetBrushFromAtlasInterface(TScriptInterface<ISlateTextureAtlasInterface> AtlasRegion, bool bMatchSize = false);

	/**
	* Sets the Brush to the specified Dynamic Texture.
	*
	*   @param Texture Dynamic Texture to use to set on Brush.
	*	@param bMatchSize If true, image will change its size to texture size. If false, texture will be stretched to image size.
	*/
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API virtual void SetBrushFromTextureDynamic(UTexture2DDynamic* Texture, bool bMatchSize = false);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API virtual void SetBrushFromMaterial(UMaterialInterface* Material);

	/**
	* Sets the Brush to the specified Soft Texture.
	*
	*   @param SoftTexture Soft Texture to use to set on Brush.
	*	@param bMatchSize If true, image will change its size to texture size. If false, texture will be stretched to image size.
	*/
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API virtual void SetBrushFromSoftTexture(TSoftObjectPtr<UTexture2D> SoftTexture, bool bMatchSize = false);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API virtual void SetBrushFromSoftMaterial(TSoftObjectPtr<UMaterialInterface> SoftMaterial);

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API UMaterialInstanceDynamic* GetDynamicMaterial();

	UMG_API void SetFlipForRightToLeftFlowDirection(bool InbFlipForRightToLeftFlowDirection);

	UMG_API bool ShouldFlipForRightToLeftFlowDirection() const;

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	//~ Begin UWidget Interface
	UMG_API virtual const FText GetPaletteCategory() override;
	//~ End UWidget Interface
#endif

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

	/** Translates the bound brush data and assigns it to the cached brush used by this widget. */
	UMG_API const FSlateBrush* ConvertImage(TAttribute<FSlateBrush> InImageAsset) const;

	// Called when we need to stream in content.
	UMG_API void RequestAsyncLoad(TSoftObjectPtr<UObject> SoftObject, TFunction<void()>&& Callback);
	UMG_API virtual void RequestAsyncLoad(TSoftObjectPtr<UObject> SoftObject, FStreamableDelegate DelegateToCall);

	// Called when we need to abort the texture being streamed in.
	UMG_API virtual void CancelImageStreaming();

	// Called when the image streaming starts, after the other one was cancelled.
	UMG_API virtual void OnImageStreamingStarted(TSoftObjectPtr<UObject> SoftObject);

	// Called when the image streaming completes.
	UMG_API virtual void OnImageStreamingComplete(TSoftObjectPtr<UObject> LoadedSoftObject);

	//
	UMG_API FReply HandleMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& MouseEvent);

#if WITH_ACCESSIBILITY
	UMG_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const override;
#endif

protected:
	TSharedPtr<SImage> MyImage;

	TSharedPtr<FStreamableHandle> StreamingHandle;
	FSoftObjectPath StreamingObjectPath;

protected:

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PROPERTY_BINDING_IMPLEMENTATION(FSlateColor, ColorAndOpacity);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
