// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Framework/Commands/InputChord.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Margin.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Styling/SlateTypes.h"
#include "Layout/Visibility.h"
#include "InputKeySelector.generated.h"

class SInputKeySelector;
struct FButtonStyle;

/** A widget for selecting a single key or a single key with a modifier. */
UCLASS(MinimalAPI)
class UInputKeySelector : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam( FOnKeySelected, FInputChord, SelectedKey );
	DECLARE_DYNAMIC_MULTICAST_DELEGATE( FOnIsSelectingKeyChanged );

public:
	UE_DEPRECATED(5.2, "Direct access to WidgetStyle is now deprecated. Use the setter or getter.")
	/** The button style used at runtime */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetButtonStyle", Setter = "SetButtonStyle", Category = Appearance, meta = (DisplayName = "Style"))
	FButtonStyle WidgetStyle;

	UE_DEPRECATED(5.2, "Direct access to TextStyle is now deprecated. Use the setter or getter.")
	/** The button style used at runtime */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Appearance, meta = (DisplayName = "Text Style"))
	FTextBlockStyle TextStyle;

	UE_DEPRECATED(5.2, "Direct access to SelectedKey is now deprecated. Use the setter or getter.")
	/** The currently selected key chord. */
	UPROPERTY( BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetSelectedKey", FieldNotify, Category = "Key Selection")
	FInputChord SelectedKey;

	UE_DEPRECATED(5.2, "Direct access to Margin is now deprecated. Use the setter or getter.")
	/** The amount of blank space around the text used to display the currently selected key. */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Appearance )
	FMargin Margin;

	UE_DEPRECATED(5.2, "Direct access to KeySelectionText is now deprecated. Use the setter or getter.")
	/** Sets the text which is displayed while selecting keys. */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetKeySelectionText", Category = Appearance )
	FText KeySelectionText;

	UE_DEPRECATED(5.2, "Direct access to NoKeySpecifiedText is now deprecated. Use the setter or getter.")
	/** Sets the text to display when no key text is available or not selecting a key. */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetNoKeySpecifiedText", Category = Appearance )
	FText NoKeySpecifiedText;

	UE_DEPRECATED(5.2, "Direct access to bAllowModifierKeys is now deprecated. Use the setter or getter.")
	/**
	 * When true modifier keys such as control and alt are allowed in the
	 * input chord representing the selected key, if false modifier keys are ignored.
	 */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter = "AllowModifierKeys", Setter = "SetAllowModifierKeys", BlueprintSetter = "SetAllowModifierKeys", Category = "Key Selection")
	bool bAllowModifierKeys;

	UE_DEPRECATED(5.2, "Direct access to bAllowGamepadKeys is now deprecated. Use the setter or getter.")
	/** When true gamepad keys are allowed in the input chord representing the selected key, otherwise they are ignored. */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter = "AllowGamepadKeys", Setter = "SetAllowGamepadKeys", BlueprintSetter = "SetAllowGamepadKeys", Category = "Key Selection")
	bool bAllowGamepadKeys;

	/** When true gamepad keys are allowed in the input chord representing the selected key, otherwise they are ignored. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Key Selection")
	TArray<FKey> EscapeKeys;

public:
	/** Called whenever a new key is selected by the user. */
	UPROPERTY( BlueprintAssignable, Category = "Widget Event" )
	FOnKeySelected OnKeySelected;

	/** Called whenever the key selection mode starts or stops. */
	UPROPERTY( BlueprintAssignable, Category = "Widget Event" )
	FOnIsSelectingKeyChanged OnIsSelectingKeyChanged;

	/** Sets the currently selected key. */
	UFUNCTION( BlueprintCallable, Category = "Widget" )
	UMG_API void SetSelectedKey(const FInputChord& InSelectedKey);

	/** Returns the currently selected key. */
	UMG_API FInputChord GetSelectedKey() const;

	/** Sets the text which is displayed while selecting keys. */
	UFUNCTION( BlueprintCallable, Category = "Widget" )
	UMG_API void SetKeySelectionText( FText InKeySelectionText );

	/** Returns the text which is displayed while selecting keys. */
	UMG_API const FText& GetKeySelectionText() const;

	/** Sets the text to display when no key text is available or not selecting a key. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetNoKeySpecifiedText(FText InNoKeySpecifiedText);

	/** Returns the text to display when no key text is available or not selecting a key. */
	UMG_API const FText& GetNoKeySpecifiedText() const;

	/** Sets whether or not modifier keys are allowed in the selected key. */
	UFUNCTION(BlueprintCallable, Category = "Widget" )
	UMG_API void SetAllowModifierKeys(bool bInAllowModifierKeys );

	/** Returns true if modifier keys keys are allowed, otherwise returns false. */
	UMG_API bool AllowModifierKeys() const;

	/** Sets whether or not gamepad keys are allowed in the selected key. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetAllowGamepadKeys(bool bInAllowGamepadKeys);

	/** Returns true if gamepad keys are allowed, otherwise returns false. */
	UMG_API bool AllowGamepadKeys() const;

	/** Returns true if the widget is currently selecting a key, otherwise returns false. */
	UFUNCTION( BlueprintCallable, Category = "Widget" )
	UMG_API bool GetIsSelectingKey() const;

	/** Sets the visibility of the text block. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetTextBlockVisibility(const ESlateVisibility InVisibility);

	/** Sets the style of the button used to start key selection mode. */
	UMG_API void SetButtonStyle(const FButtonStyle& ButtonStyle);

	/** Sets the style of the button used to start key selection mode. */
	UE_DEPRECATED(5.2, "SetButtonStyle with pointer type parameter is deprecated. Please pass a reference instead.")
	UMG_API void SetButtonStyle(const FButtonStyle* ButtonStyle);

	/** Returns the style of the button used to start key selection mode. */
	UMG_API const FButtonStyle& GetButtonStyle() const;

	/** Sets the style of the text used inside the button. */
	UMG_API void SetTextStyle(const FTextBlockStyle& InTextStyle);

	/** Returns the style of the text used inside the button. */
	UMG_API const FTextBlockStyle& GetTextStyle() const;

	/** Sets the amount of blank space around the text used to display the currently selected key. */
	UMG_API void SetMargin(const FMargin& InMargin);

	/** Returns the amount of blank space around the text used to display the currently selected key. */
	UMG_API const FMargin& GetMargin() const;

	/** Sets escape keys. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetEscapeKeys(const TArray<FKey>& InKeys);

	//~ Begin UWidget Interface
#if WITH_EDITOR	
	UMG_API virtual const FText GetPaletteCategory() override;
#endif
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

protected:
	//~ Begin UObject Interface
	UMG_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

private:
	UMG_API virtual void HandleKeySelected(const FInputChord& InSelectedKey);
	void HandleIsSelectingKeyChanged();

private:
	/** The input key selector widget managed by this object. */
	TSharedPtr<SInputKeySelector> MyInputKeySelector;
};
