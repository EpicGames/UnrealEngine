// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "InvalidationBox.generated.h"

class SInvalidationPanel;

/**
 * Invalidate
 * * Single Child
 * * Caching / Performance
 */
UCLASS(MinimalAPI)
class UInvalidationBox : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * 
	 */
	UE_DEPRECATED(4.27, "InvalidationCache is not used.")
	UFUNCTION(BlueprintCallable, Category="Invalidation Box", meta=(DeprecatedFunction))
	UMG_API void InvalidateCache();

	/**
	 * @returns true when the invalidation box cache the widgets.
	 * The widgets will be updated only if they get invalidated.
	 */
	UFUNCTION(BlueprintCallable, Category="Invalidation Box")
	UMG_API bool GetCanCache() const;

	/**
	 * Tell the InvalidationBox to use the invalidation process.
	 * @note the other internal flags can disable the option.
	 */
	UFUNCTION(BlueprintCallable, Category="Invalidation Box")
	UMG_API void SetCanCache(bool CanCache);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	//~ Begin UPanelWidget interface
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	//~ End UPanelWidget interface

	//~ Begin UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget interface

protected:
	UE_DEPRECATED(5.2, "Direct access to bCanCache is deprecated. Please use the getter or setter.")
	/**
	 * Should the invalidation panel cache the widgets?  Making this false makes it so the invalidation
	 * panel stops acting like an invalidation panel, just becomes a simple container widget.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetCanCache", Setter = "SetCanCache", BlueprintGetter = "GetCanCache", BlueprintSetter = "SetCanCache", Category = "Caching")
	bool bCanCache;

protected:
	TSharedPtr<class SInvalidationPanel> MyInvalidationPanel;
};
