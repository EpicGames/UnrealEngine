// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ListViewBase.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SCompoundWidget.h"
#include "Input/DragAndDrop.h"

#include "ListView.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FSimpleListItemEventDynamic, UObject*, Item);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnListEntryInitializedDynamic, UObject*, Item, UUserWidget*, Widget);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnListItemSelectionChangedDynamic, UObject*, Item, bool, bIsSelected);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnItemIsHoveredChangedDynamic, UObject*, Item, bool, bIsHovered);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnListItemScrolledIntoViewDynamic, UObject*, Item, UUserWidget*, Widget);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnListViewScrolledDynamic, float, ItemOffset, float, DistanceRemaining);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnListViewFinishedScrollingDynamic);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnListViewTouchStartDynamic);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnListViewTouchMoveDynamic);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnListViewTouchEndDynamic);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FOnIsItemSelectableOrNavigableDynamic, UObject*, Item);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnItemDragDropMulticastDynamic, UObject*, Item, UDragDropOperation*, Operation);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnItemGeometryMulticastDynamic, UObject*, Item, const FGeometry&, MyGeometry);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnItemZoneMulticastDynamic, UObject*, Item, EUMGItemDropZone, DropZone, UDragDropOperation*, Operation);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnItemDragCancelledDynamic, UDragDropOperation*, Operation);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnListViewDraggingStateChangedDynamic, bool, bIsDragging);

/*
 * A virtualized list that allows up to thousands of items to be displayed.
 * 
 * An important distinction to keep in mind here is "Item" vs. "Entry"
 * The list itself is based on a list of n items, but only creates as many entry widgets as can fit on screen.
 * For example, a scrolling ListView of 200 items with 5 currently visible will only have created 5 entry widgets.
 *
 * To make a widget usable as an entry in a ListView, it must inherit from the IUserObjectListEntry interface.
 */
UCLASS(meta = (EntryInterface = "/Script/UMG.UserObjectListEntry"), MinimalAPI)
class UListView : public UListViewBase, public ITypedUMGListView<UObject*>
{
	GENERATED_BODY()

	IMPLEMENT_TYPED_UMG_LIST(UObject*, MyListView)

public:
	UMG_API UListView(const FObjectInitializer& Initializer);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UMG_API virtual void PostLoad() override;

	/** Set the list of items to display within this listview */
	template <typename ItemObjectT, typename AllocatorType = FDefaultAllocator>
	void SetListItems(const TArray<ItemObjectT, AllocatorType>& InListItems)
	{
		ClearListItems();
		ListItems.Reserve(InListItems.Num());
		for (const ItemObjectT ListItem : InListItems)
		{
			if (ListItem != nullptr)
			{
				ListItems.Add(ListItem);
			}
		}

		OnItemsChanged(ListItems, TArray<UObject*>());

		RequestRefresh();
	}

	ESelectionMode::Type GetSelectionMode() const { return SelectionMode; }
	EOrientation GetOrientation() const { return Orientation; }

	template <typename RowWidgetT = UUserWidget>
	RowWidgetT* GetEntryWidgetFromItem(const UObject* Item) const
	{
		return Item ? ITypedUMGListView<UObject*>::GetEntryWidgetFromItem<RowWidgetT>(const_cast<UObject*>(Item)) : nullptr;
	}
	UMG_API void SetSelectedItem(const UObject* Item);

	/** Gets the first selected item, if any; recommended that you only use this for single selection lists. */
	template <typename ObjectT = UObject>
	ObjectT* GetSelectedItem() const
	{
		return Cast<ObjectT>(ITypedUMGListView<UObject*>::GetSelectedItem());
	}

	/**
	 * Gets the list of all items in the list.
	 * Note that each of these items only has a corresponding entry widget when visible. Use GetDisplayedEntryWidgets to get the currently displayed widgets.
	 */
	UFUNCTION(BlueprintCallable, Category = ListView)
	const TArray<UObject*>& GetListItems() const { return ListItems; }
	
	/** Adds an the item to the end of the list */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void AddItem(UObject* Item);

	/** Adds an the item to the list at the given index */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void AddItemAt(UObject* Item, int32 Index);

	/** Adds multiple items to the end of the list */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void AddItems(const TArray<UObject*>& Items);

	/** Adds multiple items to the list starting at the given index */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void AddItemsAt(const TArray<UObject*>& Items, int32 Index);

	/** Removes an the item from the list */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void RemoveItem(UObject* Item);

	/** Removes multiple items from the list */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void RemoveItems(const TArray<UObject*>& Items);

	/** Returns the item at the given index */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API UObject* GetItemAt(int32 Index) const;

	/** Returns the total number of items */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API int32 GetNumItems() const;

	/** Returns the index that the specified item is at. Will return the first found, or -1 for not found */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API int32 GetIndexForItem(const UObject* Item) const;

	/** Removes all items from the list */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void ClearListItems();

	/** Sets the new selection mode, preserving the current selection where possible. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void SetSelectionMode(TEnumAsByte<ESelectionMode::Type> SelectionMode);
	
	/* Sets ScrollIntoViewAlignment which allows to stick the selected item to either side or center */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void SetScrollIntoViewAlignment (EScrollIntoViewAlignment NewScrollIntoViewAlignment);

	/** Sets whether spatially proximate focus routing is enabled for this list view at runtime. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void SetEnableProximateEntryNavigation(bool bNewValue);

	/** Sets padding around the scrollbar. */
	UFUNCTION(Category = ListView)
	UMG_API void SetScrollBarPadding(const FMargin& InScrollBarPadding);

	/** Returns true if a refresh is pending and the list will be rebuilt on the next tick */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API bool IsRefreshPending() const;

	/** Requests that the item at the given index is scrolled into view */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void ScrollIndexIntoView(int32 Index);

	/** Sets the item at the given index as the sole selected item. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void SetSelectedIndex(int32 Index);

	/** Requests that the item at the given index navigated to, scrolling it into view if needed. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void NavigateToIndex(int32 Index);

	UMG_API virtual UObject* GetListObjectFromEntry(UUserWidget& EntryWidget) override;

protected:
	UMG_API virtual void OnItemsChanged(const TArray<UObject*>& AddedItems, const TArray<UObject*>& RemovedItems);

	UFUNCTION()
	UMG_API void OnListItemEndPlayed(AActor* Item, EEndPlayReason::Type EndPlayReason);

	UFUNCTION()
	UMG_API void OnListItemOuterEndPlayed(AActor* ItemOuter, EEndPlayReason::Type EndPlayReason);

	UMG_API virtual TSharedRef<STableViewBase> RebuildListWidget() override;
	UMG_API virtual void HandleListEntryHovered(UUserWidget& EntryWidget) override;
	UMG_API virtual void HandleListEntryUnhovered(UUserWidget& EntryWidget) override;
	// Drag Drop handlers
	UMG_API virtual UDragDropOperation* HandleListEntryDragDetected(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent, UUserWidget& EntryWidget) override;
	UMG_API virtual TOptional<EItemDropZone> HandleListEntryCanAcceptDrop(const FDragDropEvent& DropEvent, EItemDropZone DropZone, UUserWidget& EntryWidget) override;
	UMG_API virtual FReply HandleListEntryAcceptDrop(const FDragDropEvent& DropEvent, EItemDropZone DropZone, UUserWidget& EntryWidget) override;
	UMG_API virtual void HandleListEntryDragEnter(const FDragDropEvent& DropEvent, UUserWidget& EntryWidget) override;
	UMG_API virtual void HandleListEntryDragLeave(const FDragDropEvent& DropEvent, UUserWidget& EntryWidget) override;
	UMG_API virtual void HandleListEntryDragCancelled(const FDragDropEvent& DropEvent) override;
	//End Drag Drop

#if WITH_EDITOR
	UMG_API virtual void OnRefreshDesignerItems() override;
#endif

	UMG_API virtual UUserWidget& OnGenerateEntryWidgetInternal(UObject* Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable) override;
	UMG_API virtual FMargin GetDesiredEntryPadding(UObject* Item) const override;

	UMG_API virtual void OnItemClickedInternal(UObject* Item) override;
	UMG_API virtual void OnItemDoubleClickedInternal(UObject* Item) override;
	UMG_API virtual void OnSelectionChangedInternal(UObject* FirstSelectedItem) override;
	UMG_API virtual void OnItemScrolledIntoViewInternal(UObject* Item, UUserWidget& EntryWidget) override;
	UMG_API virtual void OnListViewScrolledInternal(float ItemOffset, float DistanceRemaining) override;
	UMG_API virtual void OnListViewFinishedScrollingInternal() override;
	UMG_API virtual void OnListViewTouchStartInternal() override;
	UMG_API virtual void OnListViewTouchMoveInternal() override;
	UMG_API virtual void OnListViewTouchEndInternal() override;
	UMG_API virtual bool OnIsSelectableOrNavigableInternal(UObject* FirstSelectedItem) override;

	UMG_API void HandleOnEntryInitializedInternal(UObject* Item, const TSharedRef<ITableRow>& TableRow);

	/** SListView construction helper - useful if using a custom STreeView subclass */
	template <template<typename> class ListViewT = SListView>
	TSharedRef<ListViewT<UObject*>> ConstructListView()
	{
		FListViewConstructArgs Args;
		Args.bAllowFocus = bIsFocusable;
		Args.SelectionMode = SelectionMode;
		Args.bClearSelectionOnClick = bClearSelectionOnClick;
		Args.ConsumeMouseWheel = ConsumeMouseWheel;
		Args.bReturnFocusToSelection = bReturnFocusToSelection;
		Args.bEnableProximateEntryNavigation = bEnableProximateEntryNavigation;
		Args.bClearScrollVelocityOnSelection = bClearScrollVelocityOnSelection;
		Args.Orientation = Orientation;
		Args.ListViewStyle = &WidgetStyle;
		Args.ScrollBarStyle = &ScrollBarStyle;
		Args.ShadowBoxStyle = &ShadowBrushStyle;
		Args.bEnableShadowBoxStyle = bEnableShadowBrush;
		Args.ScrollBarPadding = ScrollBarPadding;
		Args.ScrollIntoViewAlignment = ScrollIntoViewAlignment;
		MyListView = ITypedUMGListView<UObject*>::ConstructListView<ListViewT>(this, ListItems, Args);
		
		MyListView->SetOnEntryInitialized(SListView<UObject*>::FOnEntryInitialized::CreateUObject(this, &UListView::HandleOnEntryInitializedInternal));

		return StaticCastSharedRef<ListViewT<UObject*>>(MyListView.ToSharedRef());
	}

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView, meta = (DisplayName = "Style"))
	FTableViewStyle WidgetStyle;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	FScrollBarStyle ScrollBarStyle;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Style", meta=( DisplayName="Enable Shadow Brush Style" ))
	bool bEnableShadowBrush = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Style", meta=( DisplayName="Shadow Brush Style", EditCondition = bEnableShadowBrush, EditConditionHides ))
	FScrollBoxStyle ShadowBrushStyle;

	/** 
	 * The scroll & layout orientation of the list. ListView and TileView only. 
	 * Vertical will scroll vertically and arrange tiles into rows.
	 * Horizontal will scroll horizontally and arrange tiles into columns.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	TEnumAsByte<EOrientation> Orientation;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	TEnumAsByte<ESelectionMode::Type> SelectionMode = ESelectionMode::Single;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	EConsumeMouseWheel ConsumeMouseWheel = EConsumeMouseWheel::WhenScrollingPossible;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	bool bClearSelectionOnClick = false;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	bool bIsFocusable = true;
	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	bool bClearScrollVelocityOnSelection = true;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	bool bReturnFocusToSelection = false;

	/**
	 * EXPERIMENTAL - Enables the ability to naturally navigate items within this listview, always
	 * preferring the closest focusable target. This also works with nested focus targets.
	 */
	UPROPERTY(Experimental, EditAnywhere, BlueprintReadOnly, Category = ListView)
	bool bEnableProximateEntryNavigation = false;

	/** Sets where to scroll a widget to when using explicit navigation */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Scrolling)
	EScrollIntoViewAlignment ScrollIntoViewAlignment = EScrollIntoViewAlignment::CenterAligned;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>> ListItems;

	TSharedPtr<SListView<UObject*>> MyListView;

#if WITH_EDITORONLY_DATA
	/**
	 * This deprecated property was originally BlueprintReadOnly. To satisfy the compiler requirment to have a BlueprintGetter for this property, 
	 * it relies on the newly added UFunction GetHorizontalEntrySpacing() to act as its BlueprintGetter.
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "EntrySpacing has been deprecated. Please use HorizontalEntrySpacing and VerticalEntrySpacing."), BlueprintReadOnly, BlueprintGetter = "GetHorizontalEntrySpacing", Category = ListView, meta = (ClampMin = 0))
	float EntrySpacing = 0.f;
#endif  // WITH_EDITOR

private:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = ListView, meta = (ClampMin = 0, AllowPrivateAccess = "true"))
	float HorizontalEntrySpacing = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = ListView, meta = (ClampMin = 0, AllowPrivateAccess = "true"))
	float VerticalEntrySpacing = 0.f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = ListView, meta = (AllowPrivateAccess = "true", DisplayAfter="ScrollBarStyle"))
	FMargin ScrollBarPadding;


public:
	/** Get the horizontal spacing between entries. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	float GetHorizontalEntrySpacing() const { return HorizontalEntrySpacing; }

	/** Get the vertical spacing between entries. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	float GetVerticalEntrySpacing() const { return VerticalEntrySpacing; }

	/** Gets padding around the scrollbar. */
	UFUNCTION(Category = ListView)
	FMargin GetScrollBarPadding() const { return ScrollBarPadding; }

	/** Set the return Focus to selection flag. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	void SetReturnFocusToSelection(bool bEnabled);

	/** Set the vertical spacing between entries. */
	UMG_API void SetVerticalEntrySpacing(float InVerticalEntrySpacing);

	/** Set the horizontal spacing between entries. */
	UMG_API void SetHorizontalEntrySpacing(float InHorizontalEntrySpacing);

	/** Sets the Shadow Brush Style when scrolling */
	UMG_API void SetShadowBrushStyle(const FScrollBoxStyle& NewWidgetStyle);

	/** Gets the Shadow Brush Style when scrolling */
	UMG_API const FScrollBoxStyle& GetShadowBrushStyle() const;


protected:
	UE_DEPRECATED(5.6, "InitHorizontalEntrySpacing is deprecated, please use SetHorizontalEntrySpacing instead.")
	UMG_API void InitHorizontalEntrySpacing(float InHorizontalEntrySpacing);

	UE_DEPRECATED(5.6, "InitVerticalEntrySpacing is deprecated, please use SetVerticalEntrySpacing instead.")
	UMG_API void InitVerticalEntrySpacing(float InVerticalEntrySpacing);

private:
	// BP exposure of ITypedUMGListView API

	/** Sets the given item as the sole selected item. */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Set Selected Item"))
	UMG_API void BP_SetSelectedItem(UObject* Item);

	/** Sets whether the given item is selected. */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Set Item Selection"))
	UMG_API void BP_SetItemSelection(UObject* Item, bool bSelected);

	/** Clear selection */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Clear Selection"))
	UMG_API void BP_ClearSelection();

	/** Gets the number of items currently selected in the list */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Get Num Items Selected"))
	UMG_API int32 BP_GetNumItemsSelected() const;

	/** Gets a list of all the currently selected items */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "GetSelectedItems"))
	UMG_API bool BP_GetSelectedItems(TArray<UObject*>& Items) const;

	/** Gets whether the entry for the given object is currently visible in the list */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Is Item Visible"))
	UMG_API bool BP_IsItemVisible(UObject* Item) const;

	/** Requests that the given item is navigated to, scrolling it into view if needed. */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Navigate To Item"))
	UMG_API void BP_NavigateToItem(UObject* Item);

	/** Requests that the given item is scrolled into view */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Scroll Item Into View"))
	UMG_API void BP_ScrollItemIntoView(UObject* Item);

	/** Cancels a previous request to scroll and item into view. */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Cancel Scroll Into View"))
	UMG_API void BP_CancelScrollIntoView();

	/** Sets the array of objects to display rows for in the list */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (AllowPrivateAccess = true, DisplayName = "Set List Items", ViewmodelBlueprintWidgetExtension = "EntryViewModel"))
	UMG_API void BP_SetListItems(const TArray<UObject*>& InListItems);

	/** Gets the first selected item, if any; recommended that you only use this for single selection lists. */
	UFUNCTION(BlueprintCallable, Category = ListView, meta = (DisplayName = "Get Selected Item", AllowPrivateAccess = true))
	UMG_API UObject* BP_GetSelectedItem() const;

private:
	/** Called when a row widget is generated for a list item */
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Entry Initialized"))
	FOnListEntryInitializedDynamic BP_OnEntryInitialized;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Clicked"))
	FSimpleListItemEventDynamic BP_OnItemClicked;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Double Clicked"))
	FSimpleListItemEventDynamic BP_OnItemDoubleClicked;
	
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Drag Detected"))
	FOnItemGeometryMulticastDynamic BP_OnItemDragDetected;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Drag Enter"))
	FOnItemDragDropMulticastDynamic BP_OnItemDragEnter;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Drag Leave"))
	FOnItemDragDropMulticastDynamic BP_OnItemDragLeave;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Accept Drop"))
	FOnItemZoneMulticastDynamic BP_OnItemAcceptDrop;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Drag Cancelled"))
	FOnItemDragCancelledDynamic BP_OnItemDragCancelled;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On List View Dragging State Changed"))
	FOnListViewDraggingStateChangedDynamic BP_OnListViewDraggingStateChanged;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Is Hovered Changed"))
	FOnItemIsHoveredChangedDynamic BP_OnItemIsHoveredChanged;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Selection Changed"))
	FOnListItemSelectionChangedDynamic BP_OnItemSelectionChanged;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Item Scrolled Into View"))
	FOnListItemScrolledIntoViewDynamic BP_OnItemScrolledIntoView;
	
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On List View Scrolled"))
	FOnListViewScrolledDynamic BP_OnListViewScrolled;	
	
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On List View Finished Scrolling"))
	FOnListViewFinishedScrollingDynamic BP_OnListViewFinishedScrolling;	
	
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On List View Touch Start"))
	FOnListViewTouchStartDynamic BP_OnListViewTouchStart;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On List View Touch Move"))
	FOnListViewTouchMoveDynamic BP_OnListViewTouchMove;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On List View Touch End"))
	FOnListViewTouchEndDynamic BP_OnListViewTouchEnd;
	
	UPROPERTY(BlueprintReadWrite, Category = Events, meta = (AllowPrivateAccess, DisplayName = "On Is Item Selectable Or Navigable"))
	FOnIsItemSelectableOrNavigableDynamic BP_OnIsItemSelectableOrNavigable;
};
