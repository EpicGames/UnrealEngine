// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/Widget.h"
#include "Slate/SObjectTableRow.h"
#include "Blueprint/UserWidgetPool.h"
#include "Input/DragAndDrop.h"
#include "Blueprint/DragDropOperation.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/UMGCoreStyle.h"

#include "ListViewBase.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnListEntryGeneratedDynamic, UUserWidget*, Widget);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnListEntryReleasedDynamic, UUserWidget*, Widget);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnListEntriesGeneratedDynamic, int32, NumEntries);

//////////////////////////////////////////////////////////////////////////
// ITypedUMGListView<T>
//////////////////////////////////////////////////////////////////////////

/**
 * Mirrored SListView<T> API for easier interaction with a bound UListViewBase widget
 * See declarations on SListView for more info on each function and event
 *
 * Note that, being a template class, this is not a UClass and therefore cannot be exposed to Blueprint.
 * If you are using UObject* items, just use (or inherit from) UListView directly
 * Otherwise, it is up to the child class to propagate events and/or expose functions to BP as needed
 *
 * Use the IMPLEMENT_TYPED_UMG_LIST() macro for the implementation boilerplate in your implementing class.
 * @see UListView for an implementation example.
 */
template <typename ItemType>
class ITypedUMGListView
{
public:
	using NullableItemType = typename SListView<ItemType>::NullableItemType;

	//////////////////////////////////////////////////////////////////////////
	// Automatically implemented via IMPLEMENT_TYPED_UMG_LIST()
	//////////////////////////////////////////////////////////////////////////
	DECLARE_MULTICAST_DELEGATE_OneParam(FSimpleListItemEvent, ItemType);
	virtual FSimpleListItemEvent& OnItemClicked() const = 0;
	virtual FSimpleListItemEvent& OnItemDoubleClicked() const = 0;

	// Drag and Drop
	virtual FSimpleListItemEvent& OnItemDragDetected() const = 0;
	virtual FSimpleListItemEvent& OnItemAcceptDrop() const = 0;
	virtual FSimpleListItemEvent& OnItemDragEnter() const = 0;
	virtual FSimpleListItemEvent& OnItemDragLeave() const = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnItemDragCancelled, const FDragDropEvent&);
	virtual FOnItemDragCancelled& OnItemDragCancelled() const = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnItemCanAcceptDrop, ItemType, bool);
	virtual FOnItemCanAcceptDrop& OnItemCanAcceptDrop() const = 0;
	// End Drag and Drop

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnItemIsHoveredChanged, ItemType, bool);
	virtual FOnItemIsHoveredChanged& OnItemIsHoveredChanged() const = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnItemSelectionChanged, NullableItemType);
	virtual FOnItemSelectionChanged& OnItemSelectionChanged() const = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnListViewScrolled, float, float);
	virtual FOnListViewScrolled& OnListViewScrolled() const = 0;

	DECLARE_MULTICAST_DELEGATE(FOnFinishedScrolling);
	virtual FOnFinishedScrolling& OnFinishedScrolling() const = 0;

	DECLARE_MULTICAST_DELEGATE(FOnTouchStart);
	virtual FOnTouchStart& OnTouchStart() const = 0;

	DECLARE_MULTICAST_DELEGATE(FOnTouchMove);
	virtual FOnTouchMove& OnTouchMove() const = 0;

	DECLARE_MULTICAST_DELEGATE(FOnTouchEnd);
	virtual FOnTouchEnd& OnTouchEnd() const = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnItemScrolledIntoView, ItemType, UUserWidget&);
	virtual FOnItemScrolledIntoView& OnItemScrolledIntoView() const = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnItemExpansionChanged, ItemType, bool);
	virtual FOnItemExpansionChanged& OnItemExpansionChanged() const = 0;

	DECLARE_DELEGATE_RetVal_OneParam(TSubclassOf<UUserWidget>, FOnGetEntryClassForItem, ItemType);
	virtual FOnGetEntryClassForItem& OnGetEntryClassForItem() const = 0;

	virtual TSubclassOf<UUserWidget> GetDefaultEntryClass() const = 0;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsItemSelectableOrNavigable, ItemType);
	virtual FOnIsItemSelectableOrNavigable& OnIsItemSelectableOrNavigable() const = 0;

protected:
	virtual SListView<ItemType>* GetMyListView() const = 0;
	virtual uint32 GetOwningUserIndex() const = 0;
	virtual bool IsDesignerPreview() const = 0;
	//////////////////////////////////////////////////////////////////////////

public:
	/**
	 * Default behavior is to check the delegate, then fall back to the default if that fails.
	 * Feel free to override directly in child classes to determine the class yourself.
	 */
	virtual TSubclassOf<UUserWidget> GetDesiredEntryClassForItem(ItemType Item) const
	{
		//@todo DanH: Need some way to allow the design time preview entries to match up with the various possible runtime entries without a possibility for inaccuracy
		if (!IsDesignerPreview())
		{
			TSubclassOf<UUserWidget> CustomClass = OnGetEntryClassForItem().IsBound() ? OnGetEntryClassForItem().Execute(Item) : nullptr;
			if (CustomClass)
			{
				return CustomClass;
			}
		}
		
		return GetDefaultEntryClass();
	}

	//////////////////////////////////////////////////////////////////////////
	// Public API to match that of SListView
	//////////////////////////////////////////////////////////////////////////

	NullableItemType GetSelectedItem() const
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			TArray<ItemType> SelectedItems = MyListView->GetSelectedItems();
			if (SelectedItems.Num() > 0)
			{
				return SelectedItems[0];
			}
		}
		return TListTypeTraits<ItemType>::MakeNullPtr();
	}

	const TObjectPtrWrapTypeOf<ItemType>* ItemFromEntryWidget(const UUserWidget& EntryWidget) const
	{
		SListView<ItemType>* MyListView = GetMyListView();
		if (ensure(EntryWidget.Implements<UUserListEntry>()) && MyListView)
		{
			TSharedPtr<SObjectTableRow<ItemType>> ObjectTableRow = StaticCastSharedPtr<SObjectTableRow<ItemType>>(EntryWidget.GetCachedWidget());
			if (ObjectTableRow.IsValid())
			{
				return MyListView->ItemFromWidget(ObjectTableRow.Get());
			}
		}
		return nullptr;
	}

	template <typename RowWidgetT = UUserWidget>
	RowWidgetT* GetEntryWidgetFromItem(const ItemType& Item) const
	{
		TSharedPtr<SObjectTableRow<ItemType>> ObjectRow = GetObjectRowFromItem(Item);
		if (ObjectRow.IsValid())
		{
			return Cast<RowWidgetT>(ObjectRow->GetWidgetObject());
		}
		return nullptr;
	}

	int32 GetIndexInList(const ItemType& Item) const
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			if (TSharedPtr<ITableRow> RowWidget = MyListView->WidgetFromItem(Item))
			{
				return RowWidget->GetIndexInList();
			}
		}
		return INDEX_NONE;
	}

	int32 GetSelectedItems(TArray<ItemType>& OutSelectedItems) const
	{
		SListView<ItemType>* MyListView = GetMyListView();
		return MyListView ? MyListView->GetSelectedItems(OutSelectedItems) : 0;
	}

	int32 GetNumItemsSelected() const
	{
		SListView<ItemType>* MyListView = GetMyListView();
		return MyListView ? MyListView->GetNumItemsSelected() : 0;
	}

	void SetSelectedItem(const ItemType& SoleSelectedItem, ESelectInfo::Type SelectInfo = ESelectInfo::Direct) 
	{ 
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			MyListView->SetSelection(SoleSelectedItem, SelectInfo);
		}
	}

	void SetItemSelection(const ItemType& Item, bool bIsSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct)
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			MyListView->SetItemSelection(Item, bIsSelected, SelectInfo);
		}
	}

	void ClearSelection()
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			MyListView->ClearSelection();
		}
	}

	bool IsItemVisible(const ItemType& Item) const
	{
		SListView<ItemType>* MyListView = GetMyListView();
		return MyListView ? MyListView->IsItemVisible(Item) : false;
	}

	bool IsItemSelected(const ItemType& Item) const
	{
		SListView<ItemType>* MyListView = GetMyListView();
		return MyListView ? MyListView->IsItemSelected(Item) : false;
	}

	void RequestNavigateToItem(const ItemType& Item)
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			MyListView->RequestNavigateToItem(Item, GetOwningUserIndex());
		}
	}

	void RequestScrollItemIntoView(const ItemType& Item)
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			MyListView->RequestScrollIntoView(Item, GetOwningUserIndex());
		}
	}

	void CancelScrollIntoView()
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			MyListView->CancelScrollIntoView();
		}
	}
	//////////////////////////////////////////////////////////////////////////

protected:
	/**
	 * ListView construction helpers
	 * Use these instead of SNew-ing your owned ListView directly to get exposed events for free
	 */

	struct FListViewConstructArgs
	{
		bool bAllowFocus = true;
		ESelectionMode::Type SelectionMode = ESelectionMode::Single;
		bool bClearSelectionOnClick = false;
		EConsumeMouseWheel ConsumeMouseWheel = EConsumeMouseWheel::WhenScrollingPossible;
		bool bReturnFocusToSelection = false;
		bool bEnableProximateEntryNavigation = false;
		bool bClearScrollVelocityOnSelection = true;
		EOrientation Orientation = Orient_Vertical;
		EScrollIntoViewAlignment ScrollIntoViewAlignment = EScrollIntoViewAlignment::CenterAligned;
		const FTableViewStyle* ListViewStyle = &FUMGCoreStyle::Get().GetWidgetStyle<FTableViewStyle>("ListView");
		const FScrollBarStyle* ScrollBarStyle = &FUMGCoreStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar");
		const FScrollBoxStyle* ShadowBoxStyle = &FUMGCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>("ScrollBox");
		bool bEnableShadowBoxStyle = false;
		FMargin ScrollBarPadding = FMargin(0.0f);
		bool bPreventThrottling = false;
	};

	template <template<typename> class ListViewT = SListView, typename UListViewBaseT>
	static TSharedRef<ListViewT<ItemType>> ConstructListView(UListViewBaseT* Implementer, 
		const TArray<ItemType>& ListItems,
		const FListViewConstructArgs& Args = FListViewConstructArgs())
	{
		static_assert(TIsDerivedFrom<ListViewT<ItemType>, SListView<ItemType>>::IsDerived, "ConstructListView can only construct instances of SListView classes");
		TSharedRef<ListViewT<ItemType>> ListView = SNew(ListViewT<ItemType>)
			.HandleGamepadEvents(true)
			.ListItemsSource(&ListItems)
			.IsFocusable(Args.bAllowFocus)
			.ClearSelectionOnClick(Args.bClearSelectionOnClick)
			.ConsumeMouseWheel(Args.ConsumeMouseWheel)
			.SelectionMode(Args.SelectionMode)
			.ReturnFocusToSelection(Args.bReturnFocusToSelection)
			.EnableProximateEntryNavigation(Args.bEnableProximateEntryNavigation)
			.ClearScrollVelocityOnSelection(Args.bClearScrollVelocityOnSelection)
			.Orientation(Args.Orientation)
			.ScrollIntoViewAlignment(Args.ScrollIntoViewAlignment)
			.bEnableShadowBoxStyle(Args.bEnableShadowBoxStyle)
			.ListViewStyle(Args.ListViewStyle)
			.ShadowBoxStyle(Args.ShadowBoxStyle)
			.ScrollBarStyle(Args.ScrollBarStyle)
			.ScrollBarPadding(Args.ScrollBarPadding)
			.PreventThrottling(Args.bPreventThrottling)
			.OnGenerateRow_UObject(Implementer, &UListViewBaseT::HandleGenerateRow)
			.OnSelectionChanged_UObject(Implementer, &UListViewBaseT::HandleSelectionChanged)
			.OnIsSelectableOrNavigable_UObject(Implementer, &UListViewBaseT::HandleIsSelectableOrNavigable)
			.OnRowReleased_UObject(Implementer, &UListViewBaseT::HandleRowReleased)
			.OnItemScrolledIntoView_UObject(Implementer, &UListViewBaseT::HandleItemScrolledIntoView)
			.OnListViewScrolled_UObject(Implementer, &UListViewBaseT::HandleListViewScrolled)
			.OnFinishedScrolling_UObject(Implementer, &UListViewBaseT::HandleFinishedScrolling)
			.OnTouchStart_UObject(Implementer, &UListViewBaseT::HandleTouchStart)
			.OnTouchMove_UObject(Implementer, &UListViewBaseT::HandleTouchMove)
			.OnTouchEnd_UObject(Implementer, &UListViewBaseT::HandleTouchEnd)
			.OnMouseButtonClick_UObject(Implementer, &UListViewBaseT::HandleItemClicked)
			.OnMouseButtonDoubleClick_UObject(Implementer, &UListViewBaseT::HandleItemDoubleClicked);
		ListView->BindToRefreshRow(TSlateDelegates< ItemType >::FOnRefreshRow::CreateUObject(Implementer, &UListViewBaseT::HandleRefreshRow));

		return ListView;
	}

	struct FTileViewConstructArgs : public FListViewConstructArgs
	{
		EListItemAlignment TileAlignment = EListItemAlignment::EvenlyDistributed;
		TAttribute<float> EntryHeight;
		TAttribute<float> EntryWidth;
		bool bWrapDirectionalNavigation = false;
		const FScrollBarStyle* ScrollBarStyle = &FUMGCoreStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar");
		EVisibility ScrollbarDisabledVisibility = EVisibility::Collapsed;
	};

	template <template<typename> class TileViewT = STileView, typename UListViewBaseT>
	static TSharedRef<TileViewT<ItemType>> ConstructTileView(UListViewBaseT* Implementer,
		const TArray<ItemType>& ListItems,
		const FTileViewConstructArgs& Args = FTileViewConstructArgs())
	{
		static_assert(TIsDerivedFrom<TileViewT<ItemType>, STileView<ItemType>>::IsDerived, "ConstructTileView can only construct instances of STileView classes");
		TSharedRef<TileViewT<ItemType>> TileView = SNew(TileViewT<ItemType>)
			.HandleGamepadEvents(true)
			.ListItemsSource(&ListItems)
			.IsFocusable(Args.bAllowFocus)
			.ClearSelectionOnClick(Args.bClearSelectionOnClick)
			.WrapHorizontalNavigation(Args.bWrapDirectionalNavigation)
			.ConsumeMouseWheel(Args.ConsumeMouseWheel)
			.SelectionMode(Args.SelectionMode)
			.EnableProximateEntryNavigation(Args.bEnableProximateEntryNavigation)
			.ItemHeight(Args.EntryHeight)
			.ItemWidth(Args.EntryWidth)
			.ItemAlignment(Args.TileAlignment)
			.Orientation(Args.Orientation)
			.ScrollBarStyle(Args.ScrollBarStyle)
			.ScrollbarDisabledVisibility(Args.ScrollbarDisabledVisibility)
			.OnGenerateTile_UObject(Implementer, &UListViewBaseT::HandleGenerateRow)
			.OnTileReleased_UObject(Implementer, &UListViewBaseT::HandleRowReleased)
			.OnSelectionChanged_UObject(Implementer, &UListViewBaseT::HandleSelectionChanged)
			.OnIsSelectableOrNavigable_UObject(Implementer, &UListViewBaseT::HandleIsSelectableOrNavigable)
			.OnItemScrolledIntoView_UObject(Implementer, &UListViewBaseT::HandleItemScrolledIntoView)
			.OnTileViewScrolled_UObject(Implementer, &UListViewBaseT::HandleListViewScrolled)
			.OnFinishedScrolling_UObject(Implementer, &UListViewBaseT::HandleFinishedScrolling)
			.OnTouchStart_UObject(Implementer, &UListViewBaseT::HandleTouchStart)
			.OnTouchMove_UObject(Implementer, &UListViewBaseT::HandleTouchMove)
			.OnTouchEnd_UObject(Implementer, &UListViewBaseT::HandleTouchEnd)
			.OnMouseButtonClick_UObject(Implementer, &UListViewBaseT::HandleItemClicked)
			.OnMouseButtonDoubleClick_UObject(Implementer, &UListViewBaseT::HandleItemDoubleClicked);
		TileView->BindToRefreshRow(TSlateDelegates< ItemType >::FOnRefreshRow::CreateUObject(Implementer, &UListViewBaseT::HandleRefreshRow));

		return TileView;
	}

	struct FTreeViewConstructArgs
	{
		ESelectionMode::Type SelectionMode = ESelectionMode::Single;
		bool bClearSelectionOnClick = false;
		EConsumeMouseWheel ConsumeMouseWheel = EConsumeMouseWheel::WhenScrollingPossible;
		bool bReturnFocusToSelection = false;
		const FTableViewStyle* TreeViewStyle = &FUMGCoreStyle::Get().GetWidgetStyle<FTableViewStyle>("TreeView");
		const FScrollBarStyle* ScrollBarStyle = &FUMGCoreStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar");
	};

	template <template<typename> class TreeViewT = STreeView, typename UListViewBaseT>
	static TSharedRef<TreeViewT<ItemType>> ConstructTreeView(UListViewBaseT* Implementer,
		const TArray<ItemType>& ListItems,
		const FTreeViewConstructArgs& Args = FTreeViewConstructArgs())
	{
		static_assert(TIsDerivedFrom<TreeViewT<ItemType>, STreeView<ItemType>>::IsDerived, "ConstructTreeView can only construct instances of STreeView classes");
		TSharedRef<TreeViewT<ItemType>> TreeView = SNew(TreeViewT<ItemType>)
			.HandleGamepadEvents(true)
			.TreeItemsSource(&ListItems)
			.ClearSelectionOnClick(Args.bClearSelectionOnClick)
			.ConsumeMouseWheel(Args.ConsumeMouseWheel)
			.SelectionMode(Args.SelectionMode)
			.ReturnFocusToSelection(Args.bReturnFocusToSelection)
			.TreeViewStyle(Args.TreeViewStyle)
			.ScrollBarStyle(Args.ScrollBarStyle)
			.OnGenerateRow_UObject(Implementer, &UListViewBaseT::HandleGenerateRow)
			.OnSelectionChanged_UObject(Implementer, &UListViewBaseT::HandleSelectionChanged)
			.OnIsSelectableOrNavigable_UObject(Implementer, &UListViewBaseT::HandleIsSelectableOrNavigable)
			.OnRowReleased_UObject(Implementer, &UListViewBaseT::HandleRowReleased)
			.OnItemScrolledIntoView_UObject(Implementer, &UListViewBaseT::HandleItemScrolledIntoView)
			.OnTreeViewScrolled_UObject(Implementer, &UListViewBaseT::HandleListViewScrolled)
			.OnFinishedScrolling_UObject(Implementer, &UListViewBaseT::HandleFinishedScrolling)
			.OnTouchStart_UObject(Implementer, &UListViewBaseT::HandleTouchStart)
			.OnTouchMove_UObject(Implementer, &UListViewBaseT::HandleTouchMove)
			.OnTouchEnd_UObject(Implementer, &UListViewBaseT::HandleTouchEnd)
			.OnMouseButtonClick_UObject(Implementer, &UListViewBaseT::HandleItemClicked)
			.OnMouseButtonDoubleClick_UObject(Implementer, &UListViewBaseT::HandleItemDoubleClicked)
			.OnGetChildren_UObject(Implementer, &UListViewBaseT::HandleGetChildren)
			.OnExpansionChanged_UObject(Implementer, &UListViewBaseT::HandleExpansionChanged);
		TreeView->BindToRefreshRow(TSlateDelegates< ItemType >::FOnRefreshRow::CreateUObject(Implementer, &UListViewBaseT::HandleRefreshRow));

		return TreeView;
	}

protected:
	/** Gets the SObjectTableRow underlying the UMG EntryWidget that represents the given item (if one exists) */
	template <template<typename> class ObjectRowT = SObjectTableRow>
	TSharedPtr<ObjectRowT<ItemType>> GetObjectRowFromItem(const ItemType& Item) const
	{
		static_assert(TIsDerivedFrom<ObjectRowT<ItemType>, SObjectTableRow<ItemType>>::IsDerived, "All UMG table rows must be or derive from SObjectTableRow.");

		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			TSharedPtr<ITableRow> RowWidget = MyListView->WidgetFromItem(Item);
			return StaticCastSharedPtr<ObjectRowT<ItemType>>(RowWidget);
		}
		return nullptr;
	}

	/**
	 * Generates the actual entry widget that represents the given item.
	 * Expected to be used in concert with UListViewBase::GenerateTypedEntry().
	 */
	virtual UUserWidget& OnGenerateEntryWidgetInternal(ItemType Item, TSubclassOf<UUserWidget> DesiredEntryClass, const TSharedRef<STableViewBase>& OwnerTable) = 0;

	/** Gets the desired padding for the entry representing the given item */
	virtual FMargin GetDesiredEntryPadding(ItemType Item) const { return FMargin(0.f); }

	/** TreeViews only. Gets the items to consider children of the given item when generating child entries. */
	virtual void OnGetChildrenInternal(ItemType Item, TArray<ItemType>& OutChildren) const {}

	/** ListView events - implement these instead of binding handlers directly to a list */
	virtual void OnItemClickedInternal(ItemType Item) {}
	virtual void OnItemDoubleClickedInternal(ItemType Item) {}
	virtual void OnSelectionChangedInternal(NullableItemType FirstSelectedItem) {}
	virtual bool OnIsSelectableOrNavigableInternal(ItemType FirstSelectedItem) { return OnIsItemSelectableOrNavigable().IsBound() ? OnIsItemSelectableOrNavigable().Execute(FirstSelectedItem) : true; }
	virtual void OnItemScrolledIntoViewInternal(ItemType Item, UUserWidget& EntryWidget) {}
	virtual void OnListViewScrolledInternal(float ItemOffset, float DistanceRemaining) {}
	virtual void OnListViewFinishedScrollingInternal() {}
	virtual void OnListViewTouchStartInternal() {}
	virtual void OnListViewTouchMoveInternal() {}
	virtual void OnListViewTouchEndInternal() {}
	virtual void OnItemExpansionChangedInternal(ItemType Item, bool bIsExpanded) {}

private:
	TSharedRef<ITableRow> HandleGenerateRow(ItemType Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSubclassOf<UUserWidget> DesiredEntryClass = GetDesiredEntryClassForItem(Item);

		UUserWidget& EntryWidget = OnGenerateEntryWidgetInternal(Item, DesiredEntryClass, OwnerTable);
		
		// Combine the desired entry padding with the padding the widget wants natively on the CDO.
		const FMargin DefaultPadding = EntryWidget.GetClass()->GetDefaultObject<UUserWidget>()->GetPadding();
		EntryWidget.SetPadding(DefaultPadding + GetDesiredEntryPadding(Item));

		TSharedPtr<SWidget> CachedWidget = EntryWidget.GetCachedWidget();
		CachedWidget->SetCanTick(true); // this is a hack to force ticking to true so selection works (which should NOT require ticking! but currently does)
		return StaticCastSharedPtr<SObjectTableRow<ItemType>>(CachedWidget).ToSharedRef();
	}

	void HandleRefreshRow(ItemType Item)
	{
		UUserWidget* RowWidget = GetEntryWidgetFromItem(Item);
		if (RowWidget)
		{
			const FMargin DefaultPadding = RowWidget->GetClass()->GetDefaultObject<UUserWidget>()->GetPadding();
			RowWidget->SetPadding(DefaultPadding + GetDesiredEntryPadding(Item));
		}
	}

	void HandleItemClicked(ItemType Item)
	{
		OnItemClickedInternal(Item);
		OnItemClicked().Broadcast(Item);
	}

	void HandleItemDoubleClicked(ItemType Item)
	{
		OnItemDoubleClickedInternal(Item);
		OnItemDoubleClicked().Broadcast(Item);
	}

	void HandleSelectionChanged(NullableItemType Item, ESelectInfo::Type SelectInfo)
	{
		//@todo DanH ListView: This really isn't the event that many will expect it to be - is it worth having at all? 
		//		It only works for single selection lists, and even then only broadcasts at the end - you don't get anything for de-selection
		OnSelectionChangedInternal(Item);
		OnItemSelectionChanged().Broadcast(Item);
	}

	bool HandleIsSelectableOrNavigable(ItemType Item)
	{
		return OnIsSelectableOrNavigableInternal(Item);
	}

	void HandleListViewScrolled(double OffsetInItems)
	{
		if (SListView<ItemType>* MyListView = GetMyListView())
		{
			const FVector2f DistanceRemaining = UE::Slate::CastToVector2f(MyListView->GetScrollDistanceRemaining());
			OnListViewScrolledInternal(static_cast<float>(OffsetInItems), DistanceRemaining.Y);
			OnListViewScrolled().Broadcast(static_cast<float>(OffsetInItems), DistanceRemaining.Y);
		}
	}

	void HandleFinishedScrolling()
	{
		OnListViewFinishedScrollingInternal();
		OnFinishedScrolling().Broadcast();
	}

	void HandleTouchStart()
	{
		OnListViewTouchStartInternal();
		OnTouchStart().Broadcast();
	}

	void HandleTouchMove()
	{
		OnListViewTouchMoveInternal();
		OnTouchMove().Broadcast();
	}

	void HandleTouchEnd()
	{
		OnListViewTouchEndInternal();
		OnTouchEnd().Broadcast();
	}

	void HandleItemScrolledIntoView(ItemType Item, const TSharedPtr<ITableRow>& InWidget)
	{
		UUserWidget* RowWidget = GetEntryWidgetFromItem(Item);
		if (ensure(RowWidget))
		{
			OnItemScrolledIntoViewInternal(Item, *RowWidget);
			OnItemScrolledIntoView().Broadcast(Item, *RowWidget);
		}
	}

	void HandleExpansionChanged(ItemType Item, bool bIsExpanded)
	{
		// If this item is currently visible (i.e. has a widget representing it), notify the widget of the expansion change
		auto ObjectRow = GetObjectRowFromItem(Item);
		if (ObjectRow.IsValid())
		{
			ObjectRow->NotifyItemExpansionChanged(bIsExpanded);
		}

		OnItemExpansionChangedInternal(Item, bIsExpanded);
		OnItemExpansionChanged().Broadcast(Item, bIsExpanded);
	}

	void HandleGetChildren(ItemType Item, TArray<ItemType>& OutChildren) const
	{
		OnGetChildrenInternal(Item, OutChildren);
	}
};

//////////////////////////////////////////////////////////////////////////
// UListViewBase
//////////////////////////////////////////////////////////////////////////

/**
 * Bare-bones base class to make creating custom UListView widgets easier.
 * Child classes should also inherit from ITypedUMGListView<T> to get a basic public ListView API for free.
 *
 * Child classes will own the actual SListView<T> widgets, but this provides some boilerplate functionality for generating entries.
 * To generate a row for the child list, use GenerateTypedRow with the appropriate SObjectTableRow<T> type for your list
 *
 * Additionally, the entry widget class can be filtered for a particular class and interface with the EntryClass and EntryInterface metadata arguments
 * This can be specified either on the class directly (see below) or on any BindWidget FProperty
 *
 * Example:
 * class UMyUserWidget : public UUserWidget
 * {
 *		UPROPERTY(BindWidget, meta = (EntryClass = MyListEntryWidget))
 *		UListView* ListView_InventoryItems;
 * }
 *
 */
UCLASS(Abstract, NotBlueprintable, hidedropdown, meta = (EntryInterface = UserListEntry), MinimalAPI)
class UListViewBase : public UWidget
{
	GENERATED_BODY()

public:
	UMG_API UListViewBase(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual void ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const override;
#endif

	TSubclassOf<UUserWidget> GetEntryWidgetClass() const { return EntryWidgetClass; }

	/** Gets all of the list entry widgets currently being displayed by the list */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API const TArray<UUserWidget*>& GetDisplayedEntryWidgets() const;

	/** Get the scroll offset of this view (in items) */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API float GetScrollOffset() const;

	/** Get the corresponding list object for this userwidget entry. Override this to call ITypedUMGListView::ItemFromEntryWidget in concrete widgets. */
	virtual UObject* GetListObjectFromEntry(UUserWidget& EntryWidget) { return nullptr; }

	/** Gets whether or not a drag operation is in progress. */
	UFUNCTION(BlueprintCallable, Category = ListView)
	bool GetIsDraggingListItem() const { return bIsDragging; }

	/**
	 * Full regeneration of all entries in the list. Note that the entry UWidget instances will not be destroyed, but they will be released and re-generated.
	 * In other words, entry widgets will not receive Destruct/Construct events. They will receive OnEntryReleased and IUserObjectListEntry implementations will receive OnListItemObjectSet.
	 */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void RegenerateAllEntries();

	/** Scroll the entire list up to the first item */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void ScrollToTop();

	/** Scroll the entire list down to the bottom-most item */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void ScrollToBottom();

	/** Set the scroll offset of this view (in items) */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void SetScrollOffset(const float InScrollOffset);

	/** Stops the scroll inertia */
	UFUNCTION(BlueprintCallable, Category = ListView)
	UMG_API void EndInertialScrolling();

	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void SetWheelScrollMultiplier(float NewWheelScrollMultiplier);

	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void SetScrollbarVisibility(ESlateVisibility InVisibility);

	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API ESlateVisibility GetScrollbarVisibility() const;

	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void SetAllowOverScroll(bool NewAllowOverscroll);

	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	float GetOverscroll() const { return MyTableViewBase->GetOverscroll(); }

	/** Enable/Disable the ability of the list to scroll. This should be use as a temporary disable. */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void SetIsPointerScrollingEnabled(bool bInIsPointerScrollingEnabled);

	/** Enables/Disable touch scrolling **/
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void SetIsTouchScrollingEnabled(bool bInIsTouchScrollingEnabled);

	/** Enable/Disable the ability of the list to scroll via gamepad. */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void SetIsGamepadScrollingEnabled(bool bInIsGamepadScrollingEnabled);

	/** Cancels the drag drop operation if one is in progress */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void CancelListViewDragDrop();

	// Creates the drag drop operation for the list view.
	UFUNCTION(Category = ListViewBase)
	UDragDropOperation* CreateDragDropOperation(UObject* Item);

	/**
	 * Sets the list to refresh on the next tick.
	 *
	 * Note that refreshing, from a list perspective, is limited to accounting for discrepancies between items and entries.
	 * In other words, it will only release entries that no longer have items and generate entries for new items (or newly visible items).
	 *
	 * It does NOT account for changes within existing items - that is up to the item to announce and an entry to listen to as needed.
	 * This can be onerous to set up for simple cases, so it's also reasonable (though not ideal) to call RegenerateAllEntries when changes within N list items need to be reflected.
	 */
	UFUNCTION(BlueprintCallable, Category = ListViewBase)
	UMG_API void RequestRefresh();

	DECLARE_EVENT_OneParam(UListView, FOnListEntryGenerated, UUserWidget&);
	FOnListEntryGenerated& OnEntryWidgetGenerated() { return OnListEntryGeneratedEvent; }

	DECLARE_EVENT_OneParam(UListView, FOnEntryWidgetReleased, UUserWidget&);
	FOnEntryWidgetReleased& OnEntryWidgetReleased() { return OnEntryWidgetReleasedEvent; }

protected:
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override final;
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	UMG_API virtual void SynchronizeProperties() override;

	/** Implement in child classes to construct the actual ListView Slate widget */
	UMG_API virtual TSharedRef<STableViewBase> RebuildListWidget();

	//@todo DanH: Should probably have the events for native & BP built in up here - need to update existing binds to UListView's version
	virtual void HandleListEntryHovered(UUserWidget& EntryWidget) {};
	virtual void HandleListEntryUnhovered(UUserWidget& EntryWidget) {};
	virtual UDragDropOperation* HandleListEntryDragDetected(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent, UUserWidget& EntryWidget) { return nullptr; };
	virtual TOptional<EItemDropZone> HandleListEntryCanAcceptDrop(const FDragDropEvent& DropEvent, EItemDropZone DropZone, UUserWidget& EntryWidget) { return NullOpt; };
	virtual FReply HandleListEntryAcceptDrop(const FDragDropEvent& DropEvent, EItemDropZone DropZone, UUserWidget& EntryWidget) { return FReply::Unhandled(); };
	virtual void HandleListEntryDragEnter(const FDragDropEvent& DropEvent, UUserWidget& EntryWidget) {};
	virtual void HandleListEntryDragLeave(const FDragDropEvent& DropEvent, UUserWidget& EntryWidget) {};
	virtual void HandleListEntryDragCancelled(const FDragDropEvent& DropEvent) {};
	
	UMG_API virtual	void FinishGeneratingEntry(UUserWidget& GeneratedEntry);
   
    /** Called when a row widget is generated for a list item */
    UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Entry Generated"))
    FOnListEntryGeneratedDynamic BP_OnEntryGenerated;
	virtual void NativeOnEntryGenerated(UUserWidget* EntryWidget) {};

	/** Called when all row widgets are generated for all list items */
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Entries Generated"))
	FOnListEntriesGeneratedDynamic BP_OnEntriesGenerated;
	virtual void NativeOnEntriesGenerated() {};

	/**
	* Normally these are processed by UListViewBase::FinishGeneratingEntry which uses World->GetTimerManager() to generate entries next frame
	* However when for example using a listview in editor utility widgets a world there is not reliable and an alternative is to use
	* FTSTicker::GetCoreTicker()
	*/
	TArray<TWeakObjectPtr<UUserWidget>> GeneratedEntriesToAnnounce;
	
	template <typename WidgetEntryT = UUserWidget, typename ObjectTableRowT = SObjectTableRow<UObject*>>
	WidgetEntryT& GenerateTypedEntry(TSubclassOf<WidgetEntryT> WidgetClass, const TSharedRef<STableViewBase>& OwnerTable)
	{
		static_assert(TIsDerivedFrom<ObjectTableRowT, ITableRow>::IsDerived && TIsDerivedFrom<ObjectTableRowT, SObjectWidget>::IsDerived,
			"GenerateObjectTableRow can only be used to create SObjectWidget types that also inherit from ITableRow. See SObjectTableRow.");

		WidgetEntryT* ListEntryWidget = EntryWidgetPool.GetOrCreateInstance<WidgetEntryT>(*WidgetClass,
			[this, &OwnerTable] (UUserWidget* WidgetObject, TSharedRef<SWidget> Content)
			{
				return SNew(ObjectTableRowT, OwnerTable, *WidgetObject, this)
					.bAllowDragging(bAllowDragging)
					.bAllowDragDrop(bAllowDragDrop)
					.bAllowKeepPreselectedItems(bAllowKeepPreselectedItems)
					.OnHovered_UObject(this, &UListViewBase::HandleListEntryHovered)
					.OnUnhovered_UObject(this, &UListViewBase::HandleListEntryUnhovered)
					.OnRowDragDetected_UObject(this, &UListViewBase::HandleListEntryDragDetected)
					.OnRowCanAcceptDrop_UObject(this, &UListViewBase::HandleListEntryCanAcceptDrop)
					.OnRowAcceptDrop_UObject(this, &UListViewBase::HandleListEntryAcceptDrop)
					.OnRowDragEnter_UObject(this, &UListViewBase::HandleListEntryDragEnter)
					.OnRowDragLeave_UObject(this, &UListViewBase::HandleListEntryDragLeave)
					.OnRowDragCancelled_UObject(this, &UListViewBase::HandleListEntryDragCancelled)
					[
						Content
					];
			});
		check(ListEntryWidget);

		FinishGeneratingEntry(*ListEntryWidget);

		return *ListEntryWidget;
	}

#if WITH_EDITOR
	/**
	 * Called during design time to allow lists to generate preview entries via dummy data.
	 * Since that data could be of any type, the child has to be the one to generate them.
	 * 
	 * Expected to call RefreshDesignerItems<T> with the appropriate T for your underlying list
	 * @see UListView::OnRefreshDesignerItems for a usage example
	 */
	virtual void OnRefreshDesignerItems() {}

	/**
	 * Helper intended to be called by overrides of OnRefreshDesignerItems.
	 * @see UListView::OnRefreshDesignerItems for a usage example
	 */
	template <typename PlaceholderItemT>
	void RefreshDesignerItems(TArray<PlaceholderItemT>& ListItems, TFunctionRef<PlaceholderItemT()> CreateItemFunc)
	{
		bNeedsToCallRefreshDesignerItems = false;
		bool bRefresh = false;
		if (EntryWidgetClass && NumDesignerPreviewEntries > 0 && EntryWidgetClass->ImplementsInterface(UUserListEntry::StaticClass()))
		{
			if (ListItems.Num() < NumDesignerPreviewEntries)
			{
				while (ListItems.Num() < NumDesignerPreviewEntries)
				{
					ListItems.Add(CreateItemFunc());
				}
				bRefresh = true;
			}
			else if (ListItems.Num() > NumDesignerPreviewEntries)
			{
				const int32 NumExtras = ListItems.Num() - NumDesignerPreviewEntries;
				ListItems.RemoveAtSwap(ListItems.Num() - (NumExtras + 1), NumExtras);
				bRefresh = true;
			}
		}
		else
		{
			ListItems.Reset();
			bRefresh = true;
		}

		if (bRefresh)
		{
			RequestRefresh();
		}
	}
#endif

	/** Expected to be bound to the actual ListView widget created by a child class (automatically taken care of via the construction helpers within ITypedUMGListView) */
	UMG_API void HandleRowReleased(const TSharedRef<ITableRow>& Row);

	// Note: Options for this property can be configured via class and property metadata. See class declaration comment above.
	/** The type of widget to create for each entry displayed in the list. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListEntries, meta = (DesignerRebuild, AllowPrivateAccess = true, MustImplement = "/Script/UMG.UserListEntry"))
	TSubclassOf<UUserWidget> EntryWidgetClass;

	/** The multiplier to apply when wheel scrolling */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Scrolling)
	float WheelScrollMultiplier = 1.f;

	/** True to enable lerped animation when scrolling through the list */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Scrolling)
	bool bEnableScrollAnimation = false;
	
	/** The speed to apply when lerping in the scroll animation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Scrolling)
	float ScrollingAnimationInterpolationSpeed = 12.f;

	/** True to enable lerped animation when scrolling through the list with touch*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Scrolling, DisplayName = "Enable Touch Animated Scrolling")
	bool bInEnableTouchAnimatedScrolling = false;

	/**  Disable to stop scrollbars from activating inertial overscrolling */
	UPROPERTY(EditAnywhere, Category = Scrolling)
	bool AllowOverscroll = true;

	/** True to allow right click drag scrolling. */
	UPROPERTY(EditAnywhere, Category = Scrolling)
	bool bEnableRightClickScrolling = true;

	/** True to allow scrolling using touch input. */
	UPROPERTY(EditAnywhere, Category = Scrolling)
	bool bEnableTouchScrolling = true;

	/** Enable/Disable scrolling using Touch or Mouse. */
	UPROPERTY(EditDefaultsOnly, Category = Scrolling)
	bool bIsPointerScrollingEnabled = true;

	/** Enable/Disable scrolling using Gamepad. */
	UPROPERTY(EditDefaultsOnly, Category = Scrolling)
	bool bIsGamepadScrollingEnabled = true;

	UPROPERTY(EditAnywhere, Category = Scrolling)
	bool bEnableFixedLineOffset = false;

	/** 
	 * Optional fixed offset (in lines) to always apply to the top/left (depending on orientation) of the list.
	 * If provided, all non-inertial means of scrolling will settle with exactly this offset of the topmost entry.
	 * Ex: A value of 0.25 would cause the topmost full entry to be offset down by a quarter length of the preceeding entry.
	 */
	UPROPERTY(EditAnywhere, Category = Scrolling, meta = (EditCondition = bEnableFixedLineOffset, ClampMin = 0.0f, ClampMax = 0.5f))
	float FixedLineScrollOffset = 0.f;

	/** True to allow dragging of row widgets in the list */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Scrolling")
	bool bAllowDragging = true;

	/** Allow drag and drop operations to be performed on the list. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Drag and Drop")
	bool bAllowDragDrop = false;

	/**
	 * Controls where the drag widget visual will appear when dragged relative to the pointer performing
	 * the drag operation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Drag and Drop")
	EDragPivot DragDropVisualPivot = EDragPivot::CenterCenter;

	/** A percentage offset (-1..+1) from the Pivot location, the percentage is of the desired size of the dragged visual. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Drag and Drop")
	FVector2D DragDropVisualOffset = FVector2D(0,0);

	/** The class which will be used to generate the drag drop visual.  If none is specified, the EntryWidgetClass will be used. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Drag and Drop", meta = (AllowPrivateAccess = true, MustImplement = "/Script/UMG.UserListEntry"))
	TSubclassOf<UUserWidget> DragDropVisualEntryClass;

	/** The drag drop operation class for this listview - if this is not specified, drag drop operations will not be created. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Drag and Drop", meta = (AllowPrivateAccess = true))
	TSubclassOf<UDragDropOperation> DragDropOperationClass;

	/** This is the actual drag visual widget that will be used by the drag drop operation.This can be set in blueprint via override of CreateDragDropOperation(should call parent)*/
	UPROPERTY(BlueprintReadWrite, Transient, Category = "Drag and Drop")
	TObjectPtr<class UWidget> DragVisualWidget = nullptr;

	/** This indicates whether the ListViewBase is currently in a dragging state*/
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Drag and Drop")
	bool bIsDragging = false;

	/** If true, items will be "selected" (in addition to focused) when navigating to them. If false, they will only be focused. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	bool bSelectItemOnNavigation = true;

	/** If true, when selecting an item via mouse button, we allow pre-selected items to remain selected (applicable to "Multi" Selection Mode only). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ListView)
	bool bAllowKeepPreselectedItems = true;

	/** Called when a row widget is released by the list (i.e. when it no longer represents a list item) */
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Entry Released"))
	FOnListEntryReleasedDynamic BP_OnEntryReleased;
	virtual void NativeOnEntryReleased(UUserWidget* EntryWidget) {};


private:
	UMG_API virtual void HandleAnnounceGeneratedEntries();

#if WITH_EDITORONLY_DATA
	bool bNeedsToCallRefreshDesignerItems = false;;

	/** The number of dummy item entry widgets to preview in the widget designer */
	UPROPERTY(EditAnywhere, Category = ListEntries, meta = (ClampMin = 0, ClampMax = 20))
	int32 NumDesignerPreviewEntries = 5;
#endif

	UPROPERTY(Transient)
	FUserWidgetPool EntryWidgetPool;

	FTSTicker::FDelegateHandle EntryGenAnnouncementDelegateHandle;
	
	FOnListEntryGenerated OnListEntryGeneratedEvent;
	FOnEntryWidgetReleased OnEntryWidgetReleasedEvent;

	TSharedPtr<STableViewBase> MyTableViewBase;

	friend class FListViewBaseDetails;
};


#define IMPLEMENT_TYPED_UMG_LIST(ItemType, ListPropertyName)	\
protected:	\
	virtual SListView<ItemType>* GetMyListView() const override { return ListPropertyName.Get(); }	\
	virtual uint32 GetOwningUserIndex() const override	\
	{	\
		const ULocalPlayer* LocalPlayer = GetOwningLocalPlayer();	\
		int32 SlateUserIndex = LocalPlayer ? FSlateApplication::Get().GetUserIndexForController(LocalPlayer->GetControllerId()) : 0;	\
		return SlateUserIndex >= 0 ? SlateUserIndex : 0;	\
	}	\
	virtual bool IsDesignerPreview() const override { return IsDesignTime(); }	\
private:	\
	friend class ITypedUMGListView<ItemType>;	\
	mutable FSimpleListItemEvent OnItemClickedEvent;	\
	mutable FSimpleListItemEvent OnItemDoubleClickedEvent;	\
	mutable FSimpleListItemEvent OnItemDragDetectedEvent;	\
	mutable FSimpleListItemEvent OnItemDragEnterEvent;	\
	mutable FSimpleListItemEvent OnItemDragLeaveEvent;	\
	mutable FOnItemDragCancelled OnItemDragCancelledEvent;	\
	mutable FOnItemCanAcceptDrop OnItemCanAcceptDropEvent;	\
	mutable FSimpleListItemEvent OnItemAcceptDropEvent;	\
	mutable FOnItemSelectionChanged OnItemSelectionChangedEvent;	\
	mutable FOnItemIsHoveredChanged OnItemIsHoveredChangedEvent;	\
	mutable FOnItemScrolledIntoView OnItemScrolledIntoViewEvent;	\
	mutable FOnListViewScrolled OnListViewScrolledEvent;	\
	mutable FOnFinishedScrolling OnFinishedScrollingEvent;	\
	mutable FOnTouchStart OnTouchStartEvent;	\
	mutable FOnTouchMove OnTouchMoveEvent;	\
	mutable FOnTouchEnd OnTouchEndEvent;	\
	mutable FOnItemExpansionChanged OnItemExpansionChangedEvent;	\
	mutable FOnGetEntryClassForItem OnGetEntryClassForItemDelegate;	\
	mutable FOnIsItemSelectableOrNavigable OnIsItemSelectableOrNavigableDelegate;	\
public:	\
	virtual TSubclassOf<UUserWidget> GetDefaultEntryClass() const override { return EntryWidgetClass; }	\
	virtual FSimpleListItemEvent& OnItemClicked() const override { return OnItemClickedEvent; }	\
	virtual FSimpleListItemEvent& OnItemDoubleClicked() const override { return OnItemDoubleClickedEvent; }	\
	virtual FOnItemIsHoveredChanged& OnItemIsHoveredChanged() const override { return OnItemIsHoveredChangedEvent; }	\
	virtual FSimpleListItemEvent& OnItemDragDetected() const override { return OnItemDragDetectedEvent; }	\
	virtual FSimpleListItemEvent& OnItemDragEnter() const override { return OnItemDragEnterEvent; }	\
	virtual FSimpleListItemEvent& OnItemDragLeave() const override { return OnItemDragLeaveEvent; }	\
	virtual FOnItemDragCancelled& OnItemDragCancelled() const override { return OnItemDragCancelledEvent; }	\
	virtual FOnItemCanAcceptDrop& OnItemCanAcceptDrop() const override { return OnItemCanAcceptDropEvent; }	\
	virtual FSimpleListItemEvent& OnItemAcceptDrop() const override { return OnItemAcceptDropEvent; }	\
	virtual FOnItemSelectionChanged& OnItemSelectionChanged() const override { return OnItemSelectionChangedEvent; }	\
	virtual FOnItemScrolledIntoView& OnItemScrolledIntoView() const override { return OnItemScrolledIntoViewEvent; }	\
	virtual FOnListViewScrolled& OnListViewScrolled() const override { return OnListViewScrolledEvent; }	\
	virtual FOnFinishedScrolling& OnFinishedScrolling() const override { return OnFinishedScrollingEvent; }	\
	virtual FOnTouchStart& OnTouchStart() const override { return OnTouchStartEvent; }	\
	virtual FOnTouchMove& OnTouchMove() const override { return OnTouchMoveEvent; }	\
	virtual FOnTouchEnd& OnTouchEnd() const override { return OnTouchEndEvent; }	\
	virtual FOnItemExpansionChanged& OnItemExpansionChanged() const override { return OnItemExpansionChangedEvent; }	\
	virtual FOnGetEntryClassForItem& OnGetEntryClassForItem() const override { return OnGetEntryClassForItemDelegate; }	\
	virtual FOnIsItemSelectableOrNavigable& OnIsItemSelectableOrNavigable() const override { return OnIsItemSelectableOrNavigableDelegate; }
