// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "MenuAnchor.generated.h"

class SMenuAnchor;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMenuOpenChangedEvent, bool, bIsOpen);

/**
 * The Menu Anchor allows you to specify an location that a popup menu should be anchored to, 
 * and should be summoned from.
 * * Single Child
 * * Popup
 */
UCLASS(MinimalAPI)
class UMenuAnchor : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:

	DECLARE_DYNAMIC_DELEGATE_RetVal(UUserWidget*, FGetUserWidget);

	/**
	 * The widget class to spawn when the menu is required.  Creates the widget freshly each time.  
	 * If you want to customize the creation of the popup, you should bind a function to OnGetMenuContentEvent 
	 * instead.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Menu Anchor")
	TSubclassOf<class UUserWidget> MenuClass;

	/** Deprecated. Called when the menu content is requested to allow a more customized handling over what to display */
	UE_DEPRECATED(4.26, "Use OnGetUserMenuContentEvent instead, you may need to make the previous binding return an User Widget.")
	UPROPERTY(EditAnywhere, Category = "Events", meta = (DeprecationMessage = "4.26. Use OnGetUserMenuContentEvent instead, you may need to make the previous binding return an User Widget."))
	FGetWidget OnGetMenuContentEvent;

	/** Called when the menu content is requested to allow a more customized handling over what to display */
	UPROPERTY(EditAnywhere, Category="Events")
	FGetUserWidget OnGetUserMenuContentEvent;
	
	UE_DEPRECATED(5.2, "Direct access to Placement is deprecated. Please use the getter or setter.")
	/** The placement location of the summoned widget. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, BlueprintSetter = "SetPlacement", Category = "Menu Anchor")
	TEnumAsByte<EMenuPlacement> Placement;

	UE_DEPRECATED(5.2, "Direct access to bFitInWindow is deprecated. Please use the getter or setter.")
	/** Should the menu anchor attempt to fit the menu inside the window. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = "FitInWindow", Getter = "IsFitInWindow", BlueprintSetter = "FitInWindow", Category = "Menu Anchor", meta = (ScriptName = "ShouldFitInWindow"))
	bool bFitInWindow;

	UE_DEPRECATED(5.2, "Direct access to ShouldDeferPaintingAfterWindowContent is deprecated. Please use the getter.")
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsDeferPaintingAfterWindowContent", AdvancedDisplay, Category = "Menu Anchor")
	bool ShouldDeferPaintingAfterWindowContent;

	UE_DEPRECATED(5.2, "Direct access to UseApplicationMenuStack is deprecated. Please use the getter.")
	/** Does this menu behave like a normal stacked menu? Set it to false to control the menu's lifetime yourself. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsUseApplicationMenuStack", AdvancedDisplay, Category = "Menu Anchor")
	bool UseApplicationMenuStack;

	/** For menus using the application menu stack, should the window background be visible? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsShowMenuBackground", AdvancedDisplay, Category = "Menu Anchor",  meta = (EditCondition = "UseApplicationMenuStack", EditConditionHides, AllowPrivateAccess = "true", DisplayAfter = "UseApplicationMenuStack"))
	bool ShowMenuBackground;

public:
	/** Called when the opened state of the menu changes */
	UPROPERTY(BlueprintAssignable, Category="Menu Anchor|Event")
	FOnMenuOpenChangedEvent OnMenuOpenChanged;

public:
	//TODO UMG Add Set MenuClass

	UFUNCTION(BlueprintCallable, Category = "Menu Anchor")
	UMG_API void SetPlacement(EMenuPlacement InPlacement);

	UMG_API EMenuPlacement GetPlacement() const;

	UFUNCTION(BlueprintCallable, Category = "Menu Anchor")
	UMG_API void FitInWindow(bool bFit);

	UMG_API bool IsFitInWindow() const;

	UMG_API bool IsDeferPaintingAfterWindowContent() const;

	UMG_API bool IsUseApplicationMenuStack() const;

	UMG_API bool IsShowMenuBackground() const;

public:

	/**
	 * Toggles the menus open state.
	 *
	 * @param bFocusOnOpen  Should we focus the popup as soon as it opens?
	 */
	UFUNCTION(BlueprintCallable, Category="Menu Anchor")
	UMG_API void ToggleOpen(bool bFocusOnOpen);

	/** Opens the menu if it is not already open */
	UFUNCTION(BlueprintCallable, Category="Menu Anchor")
	UMG_API void Open(bool bFocusMenu);

	/** Closes the menu if it is currently open. */
	UFUNCTION(BlueprintCallable, Category="Menu Anchor")
	UMG_API void Close();

	/** Returns true if the popup is open; false otherwise. */
	UFUNCTION(BlueprintCallable, Category="Menu Anchor")
	UMG_API bool IsOpen() const;

	/**
	 * Returns true if we should open the menu due to a click. Sometimes we should not, if
	 * the same MouseDownEvent that just closed the menu is about to re-open it because it 
	 * happens to land on the button.
	 */
	UFUNCTION(BlueprintCallable, Category="Menu Anchor")
	UMG_API bool ShouldOpenDueToClick() const;

	/** Returns the current menu position */
	UFUNCTION(BlueprintCallable, Category="Menu Anchor")
	UMG_API FVector2D GetMenuPosition() const;

	/** Returns whether this menu has open submenus */
	UFUNCTION(BlueprintCallable, Category="Menu Anchor")
	UMG_API bool HasOpenSubMenus() const;

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	// Initialize ShouldDeferPaintingAfterWindowContent in the constructor before the SWidget is constructed.
	UMG_API void InitShouldDeferPaintingAfterWindowContent(bool InShouldDeferPaintingAfterWindowContent);

	// Initialize UseApplicationMenuStack in the constructor before the SWidget is constructed.
	UMG_API void InitUseApplicationMenuStack(bool InUseApplicationMenuStack);

	// Initialize ShowMenuBackground in the constructor before the SWidget is constructed.
	UMG_API void InitShowMenuBackground(bool InShowMenuBackground);

protected:
	UMG_API TSharedRef<SWidget> HandleGetMenuContent();
	UMG_API void HandleMenuOpenChanged(bool bIsOpen);

protected:
	TSharedPtr<SMenuAnchor> MyMenuAnchor;
};
