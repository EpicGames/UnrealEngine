// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Extensions/UIComponent.h"
#include "UObject/ObjectMacros.h"

#include "MouseHoverComponent.generated.h"

/** This is a class for a Component that exposes the mouse hover state of its Owner widget. */
UCLASS(MinimalAPI)
class UMouseHoverComponent : public UUIComponent
{
	GENERATED_BODY()

public:

	bool GetIsHovered() const { return bIsHovered; }

	void OnMouseHoverChanged(bool InbIsHovered);

private:

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Getter = "GetIsHovered", FieldNotify, Category = "Mouse Hover State", meta = (AllowPrivateAccess))
	bool bIsHovered = false;

	virtual TSharedRef<SWidget> RebuildWidgetWithContent(const TSharedRef<SWidget> OwnerContent) override;
};