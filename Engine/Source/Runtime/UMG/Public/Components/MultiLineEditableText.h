// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Components/TextWidgetTypes.h"
#include "Widgets/Text/ISlateEditableTextWidget.h"
#include "MultiLineEditableText.generated.h"

class UMaterialInterface;
class SMultiLineEditableText;

/**
 * Editable text box widget
 */
UCLASS(meta=( DisplayName="Editable Text (Multi-Line)" ), MinimalAPI)
class UMultiLineEditableText : public UTextLayoutWidget
{
	GENERATED_UCLASS_BODY()

public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMultiLineEditableTextChangedEvent, const FText&, Text);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMultiLineEditableTextCommittedEvent, const FText&, Text, ETextCommit::Type, CommitMethod);

public:
	UE_DEPRECATED(5.1, "Direct access to Text is deprecated. Please use the getter or setter.")
	/** The text content for this editable text box widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter = "GetText", BlueprintSetter = "SetText", FieldNotify, Category = Content, meta = (MultiLine = "true"))
	FText Text;

	UE_DEPRECATED(5.1, "Direct access to HintText is deprecated. Please use the getter or setter.")
	/** Hint text that appears when there is no text in the text box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter = "GetHintText", BlueprintSetter = "SetHintText", Category = Content, meta = (MultiLine = "true"))
	FText HintText;

	/** A bindable delegate to allow logic to drive the hint text of the widget */
	UPROPERTY()
	FGetText HintTextDelegate;
public:

	/** The style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetWidgetStyle, Category="Style", meta=(ShowOnlyInnerProperties))
	FTextBlockStyle WidgetStyle;

	UE_DEPRECATED(5.1, "Direct access to IsReadOnly is deprecated. Please use the getter or setter.")
	/** Sets the Text as Readonly to prevent it from being modified interactively by the user */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = GetIsReadOnly, Setter = SetIsReadOnly, BlueprintSetter = "SetIsReadOnly", Category = Appearance)
	bool bIsReadOnly;

	UE_DEPRECATED(5.1, "Direct access to SelectAllTextWhenFocused is deprecated. Please use the getter or setter.")
	/** Whether to select all text when the user clicks to give focus on the widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Behavior, AdvancedDisplay)
	bool SelectAllTextWhenFocused;

	UE_DEPRECATED(5.1, "Direct access to ClearTextSelectionOnFocusLoss is deprecated. Please use the getter or setter.")
	/** Whether to clear text selection when focus is lost */
	UPROPERTY(EditAnywhere, Category = Behavior, AdvancedDisplay)
	bool ClearTextSelectionOnFocusLoss;

	UE_DEPRECATED(5.1, "Direct access to RevertTextOnEscape is deprecated. Please use the getter or setter.")
	/** Whether to allow the user to back out of changes when they press the escape key */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Behavior, AdvancedDisplay)
	bool RevertTextOnEscape;

	UE_DEPRECATED(5.1, "Direct access to ClearKeyboardFocusOnCommit is deprecated. Please use the getter or setter.")
	/** Whether to clear keyboard focus when pressing enter to commit changes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Behavior, AdvancedDisplay)
	bool ClearKeyboardFocusOnCommit;

	/** Whether the context menu can be opened */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	bool AllowContextMenu;
	
	/** Additional options for the virtual keyboard */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	FVirtualKeyboardOptions VirtualKeyboardOptions;

	/** What action should be taken when the virtual keyboard is dismissed? */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	EVirtualKeyboardDismissAction VirtualKeyboardDismissAction;

	/** Called whenever the text is changed programmatically or interactively by the user */
	UPROPERTY(BlueprintAssignable, Category="Widget Event", meta=(DisplayName="OnTextChanged (Multi-Line Editable Text)"))
	FOnMultiLineEditableTextChangedEvent OnTextChanged;

	/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event", meta=(DisplayName="OnTextCommitted (Multi-Line Editable Text)"))
	FOnMultiLineEditableTextCommittedEvent OnTextCommitted;

public:

	/**
	* Gets the widget text
	* @return The widget text
	*/
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="GetText (Multi-Line Editable Text)"))
	UMG_API FText GetText() const;

	/**
	* Directly sets the widget text.
	* @param InText The text to assign to the widget
	*/
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetText (Multi-Line Editable Text)"))
	UMG_API void SetText(FText InText);

	/** Returns the Hint text that appears when there is no text in the text box */
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="GetHintText (Multi-Line Editable Text)"))
	UMG_API FText GetHintText() const;

	/** 
	* Sets the Hint text that appears when there is no text in the text box 
	* @param InHintText The text that appears when there is no text in the text box 
	*/
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetHintText (Multi-Line Editable Text)"))
	UMG_API void SetHintText(FText InHintText);

	/** Set to true to select all text when the user clicks to give focus on the widget */
	UMG_API void SetSelectAllTextWhenFocused(bool bSelectAllTextWhenFocused);

	/** Whether to select all text when the user clicks to give focus on the widget */
	UMG_API bool GetSelectAllTextWhenFocused() const;

	/** Set to true to clear text selection when focus is lost */
	UMG_API void SetClearTextSelectionOnFocusLoss(bool bClearTextSelectionOnFocusLoss);

	/** Whether to clear text selection when focus is lost */
	UMG_API bool GetClearTextSelectionOnFocusLoss() const;

	/** Set to true to allow the user to back out of changes when they press the escape key */
	UMG_API void SetRevertTextOnEscape(bool bRevertTextOnEscape);

	/** Whether to allow the user to back out of changes when they press the escape key  */
	UMG_API bool GetRevertTextOnEscape() const;

	/** Set to true to clear keyboard focus when pressing enter to commit changes */
	UMG_API void SetClearKeyboardFocusOnCommit(bool bClearKeyboardFocusOnCommit);

	/** Whether to clear keyboard focus when pressing enter to commit changes */
	UMG_API bool GetClearKeyboardFocusOnCommit() const;	

	/** Return true when this text cannot be modified interactively by the user */
	UMG_API bool GetIsReadOnly() const;

	/** Sets the Text as Readonly to prevent it from being modified interactively by the user */
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetIsReadOnly (Multi-Line Editable Text"))
	UMG_API void SetIsReadOnly(bool bReadOnly);

	UFUNCTION(BlueprintSetter)
	UMG_API void SetWidgetStyle(const FTextBlockStyle& InWidgetStyle);

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API const FSlateFontInfo& GetFont() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetFont(FSlateFontInfo InFontInfo);

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetFontMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetFontOutlineMaterial(UMaterialInterface* InMaterial);

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	//~ Begin UTextLayoutWidget Interface
	UMG_API virtual void OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions) override;
	UMG_API virtual void OnJustificationChanged(ETextJustify::Type InJustification) override;
	UMG_API virtual void OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy) override;
	UMG_API virtual void OnAutoWrapTextChanged(bool InAutoWrapText) override;
	UMG_API virtual void OnWrapTextAtChanged(float InWrapTextAt) override;
	UMG_API virtual void OnLineHeightPercentageChanged(float InLineHeightPercentage) override;
	UMG_API virtual void OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine) override;
	UMG_API virtual void OnMarginChanged(const FMargin& InMargin) override;
	//~ End UTextLayoutWidget Interface

	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UMG_API void HandleOnTextChanged(const FText& Text);
	UMG_API void HandleOnTextCommitted(const FText& Text, ETextCommit::Type CommitMethod);

protected:
	TSharedPtr<SMultiLineEditableText> MyMultiLineEditableText;

	PROPERTY_BINDING_IMPLEMENTATION(FText, HintText);

private:
	/** @return true if the text was changed, or false if identical. */
	bool SetTextInternal(const FText& InText);
};
