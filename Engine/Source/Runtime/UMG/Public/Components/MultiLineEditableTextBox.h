// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Components/TextWidgetTypes.h"
#include "Widgets/Text/ISlateEditableTextWidget.h"
#include "MultiLineEditableTextBox.generated.h"

class SMultiLineEditableTextBox;
class USlateWidgetStyleAsset;

/**
 * Allows a user to enter multiple lines of text
 */
UCLASS(meta=(DisplayName="Text Box (Multi-Line)"), MinimalAPI)
class UMultiLineEditableTextBox : public UTextLayoutWidget
{
	GENERATED_UCLASS_BODY()

public:

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnMultiLineEditableTextBoxChangedEvent, const FText&, Text);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnMultiLineEditableTextBoxCommittedEvent, const FText&, Text, ETextCommit::Type, CommitMethod);

public:
	
	UE_DEPRECATED(5.1, "Direct access to Text is deprecated. Please use the getter or setter.")
	/** The text content for this editable text box widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter = "GetText", BlueprintSetter = "SetText", FieldNotify, Category = Content, meta=(MultiLine="true"))
	FText Text;

	UE_DEPRECATED(5.1, "Direct access to HintText is deprecated. Please use the getter or setter.")
	/** Hint text that appears when there is no text in the text box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter = "GetHintText", BlueprintSetter = "SetHintText", Category = Content, meta = (MultiLine = "true"))
	FText HintText;

	/** A bindable delegate to allow logic to drive the hint text of the widget */
	UPROPERTY()
	FGetText HintTextDelegate;
public:

	/** The style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Style", meta=(DisplayName="Style"))
	FEditableTextBoxStyle WidgetStyle;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "TextStyle has been deprecated as it was mainly duplicated information already available inside WidgetStyle. Please use the WidgetStyle.TextStyle instead.")
	/** The text style */
	UPROPERTY(meta=(DisplayName="Text Style"))
	FTextBlockStyle TextStyle_DEPRECATED;
#endif

	UE_DEPRECATED(5.1, "Direct access to IsReadOnly is deprecated. Please use the getter or setter.")
	/** Sets the Text as Readonly to prevent it from being modified interactively by the user */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = GetIsReadOnly, Setter = SetIsReadOnly, Category = Appearance)
	bool bIsReadOnly;

	/** Whether the context menu can be opened */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	bool AllowContextMenu;
	
	/** Additional options to be used by the virtual keyboard summoned from this widget */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	FVirtualKeyboardOptions VirtualKeyboardOptions;

	/** What action should be taken when the virtual keyboard is dismissed? */
	UPROPERTY(EditAnywhere, Category=Behavior, AdvancedDisplay)
	EVirtualKeyboardDismissAction VirtualKeyboardDismissAction;

	/** Called whenever the text is changed programmatically or interactively by the user */
	UPROPERTY(BlueprintAssignable, Category="Widget Event", meta=(DisplayName="OnTextChanged (Multi-Line Text Box)"))
	FOnMultiLineEditableTextBoxChangedEvent OnTextChanged;

	/** Called whenever the text is committed.  This happens when the user presses enter or the text box loses focus. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event", meta=(DisplayName="OnTextCommitted (Multi-Line Text Box)"))
	FOnMultiLineEditableTextBoxCommittedEvent OnTextCommitted;

	/** Provide a alternative mechanism for error reporting. */
	//SLATE_ARGUMENT(TSharedPtr<class IErrorReportingWidget>, ErrorReporting)

public:

	/**
	 * Gets the widget text
	 * @return The widget text
	 */
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="GetText (Multi-Line Text Box)"))
	UMG_API FText GetText() const;

	/**
	 * Directly sets the widget text.
	 * Warning: This will wipe any binding created for the Text property!
	 * @param InText The text to assign to the widget
	 */
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetText (Multi-Line Text Box)"))
	UMG_API void SetText(FText InText);

	/** Returns the Hint text that appears when there is no text in the text box */
	UFUNCTION(BlueprintCallable, Category = "Widget", meta = (DisplayName = "GetHintText (Multi-Line Text Box)"))
	UMG_API FText GetHintText() const;

	/**
	* Sets the Hint text that appears when there is no text in the text box
	* @param InHintText The text that appears when there is no text in the text box
	*/
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetHintText (Multi-Line Text Box)"))
	UMG_API void SetHintText(FText InHintText);

	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetError (Multi-Line Text Box)"))
	UMG_API void SetError(FText InError);

	/** Return true when this text cannot be modified interactively by the user */
	UMG_API bool GetIsReadOnly() const;

	/** Sets the Text as Readonly to prevent it from being modified interactively by the user */
	UFUNCTION(BlueprintCallable, Category = "Widget", meta = (DisplayName = "SetIsReadOnly (Multi-Line Text Box)"))
	UMG_API void SetIsReadOnly(UPARAM(DisplayName = "ReadyOnly") bool bReadOnly);

	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetTextStyle (Multi-Line Text Box)"))
	UMG_API void SetTextStyle(const FTextBlockStyle& InTextStyle);

	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetForegroundColor (Multi-Line Text Box)"))
	UMG_API void SetForegroundColor(FLinearColor color);

	//TODO UMG Add Set ReadOnlyForegroundColor
	//TODO UMG Add Set BackgroundColor
	//TODO UMG Add Set Font

public:
	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif
	UMG_API virtual void Serialize(FArchive& Ar) override;

protected:
	//~ Begin UTextLayoutWidget Interface
	UMG_API virtual void OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions) override;
	UMG_API virtual void OnJustificationChanged(ETextJustify::Type InJustification) override;
	UMG_API virtual void OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy) override;
	UMG_API virtual void OnAutoWrapTextChanged(bool InAutoWrapText) override;
	UMG_API virtual void OnWrapTextAtChanged(float InWrapTextAt) override;
	UMG_API virtual void OnLineHeightPercentageChanged(float InLineHeightPercentage) override;
	UMG_API virtual void OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine) override;
	UMG_API virtual void OnMarginChanged(const FMargin& InMargin) override;
	//~ End UTextLayoutWidget Interface

	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UMG_API void HandleOnTextChanged(const FText& Text);
	UMG_API void HandleOnTextCommitted(const FText& Text, ETextCommit::Type CommitMethod);

protected:
	TSharedPtr<SMultiLineEditableTextBox> MyEditableTextBlock;

	PROPERTY_BINDING_IMPLEMENTATION(FText, HintText);

private:
	/** @return true if the text was changed, or false if identical. */
	bool SetTextInternal(const FText& InText);

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bIsFontDeprecationDone;
#endif
};
