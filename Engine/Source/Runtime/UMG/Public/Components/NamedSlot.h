// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "NamedSlot.generated.h"

class SBox;

/**
 * Allows you to expose an external slot for your user widget.  When others reuse your user control, 
 * they can put whatever they want in this named slot.
 */
UCLASS(MinimalAPI)
class UNamedSlot : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:

	// UPanelWidget interface
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End of UPanelWidget interface

	// UVisual interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API FGuid GetSlotGUID() const;
#endif

public:
#if WITH_EDITORONLY_DATA
	/**
	 * Named slots exposed on the instance only follow a slightly different set of rules.  For example, they can contain
	 * some content that the user can replace after dropping it into their tree.  However, these slots can not be inherited
	 * to be filled in by a subclass.  So if you want this named slot to be extensible in a subclass of this widget, you
	 * should set or leave this value as false.
	 */
	UPROPERTY(EditAnywhere, Category="Exposing")
	bool bExposeOnInstanceOnly = false;
#endif
	
	UMG_API virtual void Serialize(FArchive& Ar) override;
	UMG_API virtual void PostLoad() override;
	
protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

protected:
	TSharedPtr<SBox> MyBox;

private:	
#if WITH_EDITORONLY_DATA

	UPROPERTY()
	FGuid SlotGuid = FGuid::NewGuid();
#endif

};
