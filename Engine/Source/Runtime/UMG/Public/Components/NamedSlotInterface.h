// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Widget.h"
#include "NamedSlotInterface.generated.h"

class UWidget;

/**
 * 
 */
UINTERFACE(meta=( CannotImplementInterfaceInBlueprint ), MinimalAPI)
class UNamedSlotInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INamedSlotInterface
{
	GENERATED_IINTERFACE_BODY()

public:

	/** Gets the names for slots that we can store widgets into. */
	virtual void GetSlotNames(TArray<FName>& SlotNames) const = 0;

	/** Gets the widget for a given slot by name, will return nullptr if no widget is in the slot. */
	virtual UWidget* GetContentForSlot(FName SlotName) const = 0;

	/** Sets the widget for a given slot by name. */
	virtual void SetContentForSlot(FName SlotName, UWidget* Content) = 0;

	/** Determines if any slot holds the given widget. */
	UMG_API bool ContainsContent(UWidget* Content) const;

	/** Determines if any slot holds the given widget and the name of that slot. */
	UMG_API FName FindSlotForContent(UWidget* Content) const;

	/** Releases named slot related resources. */
	UMG_API void ReleaseNamedSlotSlateResources(bool bReleaseChildren);

#if WITH_EDITOR
	/** Applies the designer flags to the all of the content in all the slots. */
	UMG_API void SetNamedSlotDesignerFlags(EWidgetDesignFlags NewFlags);
#endif
};
