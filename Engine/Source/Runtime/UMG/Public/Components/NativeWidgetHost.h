// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "NativeWidgetHost.generated.h"

/**
 * A NativeWidgetHost is a container widget that can contain one child slate widget.  This should
 * be used when all you need is to nest a native widget inside a UMG widget.
 */
UCLASS(MinimalAPI)
class UNativeWidgetHost : public UWidget
{
	GENERATED_UCLASS_BODY()

	UMG_API void SetContent(TSharedRef<SWidget> InContent);
	TSharedPtr< SWidget > GetContent() const { return NativeWidget; }

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	UMG_API TSharedRef<SWidget> GetDefaultContent();

protected:
	TSharedPtr<SWidget> NativeWidget;
};
