// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Widgets/SOverlay.h"
#include "Components/PanelWidget.h"
#include "Overlay.generated.h"

class UOverlaySlot;

/**
 * Allows widgets to be stacked on top of each other, uses simple flow layout for content on each layer.
 */
UCLASS(MinimalAPI)
class UOverlay : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API UOverlaySlot* AddChildToOverlay(UWidget* Content);

	/** Replace the widget at the given index it with a different widget. */
	UFUNCTION(BlueprintCallable, Category = "Panel")
	UMG_API bool ReplaceOverlayChildAt(int32 Index, UWidget* Content);

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:

	TSharedPtr<class SOverlay> MyOverlay;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
