// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Widgets/SOverlay.h"
#include "Components/PanelSlot.h"


#include "OverlaySlot.generated.h"

/** Slot for the UOverlay panel.  Allows content to be hover above other content. */
UCLASS(MinimalAPI)
class UOverlaySlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

protected:
	//TODO UMG Slots should hold weak or shared refs to slots.

	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SOverlay::FOverlaySlot* Slot;

public:
	
	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetPadding, Category="Layout|Overlay Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetHorizontalAlignment, Category="Layout|Overlay Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetVerticalAlignment, Category="Layout|Overlay Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	/** Set padding area between the slot and the content it contains. */
	UFUNCTION(BlueprintCallable, Category="Layout|Overlay Slot")
	UMG_API void SetPadding(FMargin InPadding);

	/** Get padding area between the slot and the content it contains. */
	UMG_API FMargin GetPadding() const;

	/** Set the alignment of the object horizontally. */
	UFUNCTION(BlueprintCallable, Category="Layout|Overlay Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);
	
	/** Get the alignment of the object horizontally. */
	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	/** Set the alignment of the object vertically. */
	UFUNCTION(BlueprintCallable, Category="Layout|Overlay Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	/** Get the alignment of the object vertically. */
	UMG_API EVerticalAlignment GetVerticalAlignment() const;

public:

	// UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UPanelSlot interface

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API virtual void BuildSlot(TSharedRef<SOverlay> InOverlay);

	/** Replace the slot content. */
	UMG_API void ReplaceContent(UWidget* Content);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
};
