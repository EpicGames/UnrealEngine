// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/Visual.h"
#include "PanelSlot.generated.h"

/** The base class for all Slots in UMG. */
UCLASS(BlueprintType, MinimalAPI)
class UPanelSlot : public UVisual
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(Instanced)
	TObjectPtr<class UPanelWidget> Parent;

	UPROPERTY(Instanced)
	TObjectPtr<class UWidget> Content;

	UFUNCTION(BlueprintCallable, Category = "Layout|Panel Slot")
	UMG_API UWidget* GetContent() const;
	
#if WITH_EDITOR
	UMG_API bool IsDesignTime() const;
#else
	inline bool IsDesignTime() const { return false; }
#endif

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	/** Applies all properties to the live slot if possible. */
	virtual void SynchronizeProperties()
	{
	}

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		SynchronizeProperties();
	}

	/**
	 * Called by the designer to "nudge" a widget in a direction. Returns true if the nudge had any effect, false otherwise.
	 **/
	virtual bool NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize) { return false; }

	/**
	 * Called by the designer when a design-time widget is dragged. Returns true if the drag had any effect, false otherwise.
	 **/
	virtual bool DragDropPreviewByDesigner(const FVector2D& LocalCursorPosition, const TOptional<int32>& XGridSnapSize, const TOptional<int32>& YGridSnapSize) { return false; }

	/**
	 * Called by the designer when a design-time widget needs to have changes to its associated template synchronized.
	 **/
	virtual void SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot) {};
#endif
};
