// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PanelSlot.h"
#include "Components/Widget.h"
#include "Blueprint/UserWidget.h"
#include "PanelWidget.generated.h"

/** The base class for all UMG panel widgets.  Panel widgets layout a collection of child widgets. */
UCLASS(Abstract, MinimalAPI)
class UPanelWidget : public UWidget
{
	GENERATED_UCLASS_BODY()

protected:

	/** The slots in the widget holding the child widgets of this panel. */
	UPROPERTY(Instanced)
	TArray<TObjectPtr<UPanelSlot>> Slots;

public:

	/** Gets number of child widgets in the container. */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API int32 GetChildrenCount() const;

	/**
	 * Gets the widget at an index.
	 * @param Index The index of the widget.
	 * @return The widget at the given index, or nothing if there is no widget there.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API UWidget* GetChildAt(int32 Index) const;
	
	/** Gets all widgets in the container */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API TArray<UWidget*> GetAllChildren() const;

	/** Gets the index of a specific child widget */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API int32 GetChildIndex(const UWidget* Content) const;

	/** Returns true if panel contains this widget */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API bool HasChild(UWidget* Content) const;

	/** Removes a child by it's index. */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API bool RemoveChildAt(int32 Index);

	/**
	 * Adds a new child widget to the container.  Returns the base slot type, 
	 * requires casting to turn it into the type specific to the container.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API UPanelSlot* AddChild(UWidget* Content);

	/**
	 * Adds a new child widget to the container. Creates the slot using the
	 * template passed to it. Note that a new UPanelSlot object is created
	 * and the SlotTemplate is only used as a template.
	 */
	UMG_API UPanelSlot* AddChild(UWidget* Content, UPanelSlot* SlotTemplate);

	/**
	 * Inserts a widget at a specific index.  This does not update the live slate version, it requires
	 * a rebuild of the whole UI to see a change.
	 */
	UMG_API UPanelSlot* InsertChildAt(int32 Index, UWidget* Content);

	/**
	 * Inserts a widget at a specific index. Creates the slot using the
	 * template passed to it. Note that a new UPanelSlot object is created
	 * and the SlotTemplate is only used as a template.
	 */
	UMG_API UPanelSlot* InsertChildAt(int32 Index, UWidget* Content, UPanelSlot* SlotTemplate);

	/**
	 * Moves the child widget from its current index to the new index provided.
	 */
	UMG_API void ShiftChild(int32 Index, UWidget* Child);

#if WITH_EDITOR

	/**
	 * Swaps the widget out of the slot at the given index, replacing it with a different widget.
	 *
	 * @param Index the index at which to replace the child content with this new Content.
	 * @param Content The content to replace the child with.
	 * @return true if the index existed and the child could be replaced.
	 */
	UMG_API bool ReplaceChildAt(int32 Index, UWidget* Content);

	/**
	 * Swaps the child widget out of the slot, and replaces it with the new child widget.
	 * @param CurrentChild The existing child widget being removed.
	 * @param NewChild The new child widget being inserted.
	 * @return true if the CurrentChild was found and the swap occurred, otherwise false.
	 */
	UMG_API virtual bool ReplaceChild(UWidget* CurrentChild, UWidget* NewChild);

	UMG_API virtual void SetDesignerFlags(EWidgetDesignFlags NewFlags) override;

#endif

	/**
	 * Removes a specific widget from the container.
	 * @return true if the widget was found and removed.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API bool RemoveChild(UWidget* Content);

	/** Returns true if there are any child widgets in the panel */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API bool HasAnyChildren() const;

	/** Remove all child widgets from the panel widget. */
	UFUNCTION(BlueprintCallable, Category="Widget|Panel")
	UMG_API virtual void ClearChildren();

	/** The slots in the widget holding the child widgets of this panel. */
	UMG_API const TArray<UPanelSlot*>& GetSlots() const;

	/** Returns true if the panel supports more than one child. */
	bool CanHaveMultipleChildren() const
	{
		return bCanHaveMultipleChildren;
	}

	/** Returns true if the panel can accept another child widget. */
	bool CanAddMoreChildren() const
	{
		return CanHaveMultipleChildren() || GetChildrenCount() == 0;
	}

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	virtual bool LockToPanelOnDrag() const
	{
		return false;
	}

	virtual void ConnectEditorData() override
	{
		for (UPanelSlot* PanelSlot : Slots)
		{
			if (PanelSlot)
			{
				PanelSlot->Parent = this;
				if (PanelSlot->Content)
				{
					PanelSlot->Content->Slot = PanelSlot;
				}
			}
		}
	}
#endif

	// Begin UObject
	UMG_API virtual void PostLoad() override;
	// End UObject

	virtual UClass* GetSlotClass() const
	{
		return UPanelSlot::StaticClass();
	}

protected:

#if WITH_EDITOR
	UMG_API virtual TSharedRef<SWidget> RebuildDesignWidget(TSharedRef<SWidget> Content) override;
#endif

	virtual void OnSlotAdded(UPanelSlot* InSlot)
	{

	}

	virtual void OnSlotRemoved(UPanelSlot* InSlot)
	{

	}

protected:
	/** Can this panel allow for multiple children? */
	bool bCanHaveMultipleChildren;
};
