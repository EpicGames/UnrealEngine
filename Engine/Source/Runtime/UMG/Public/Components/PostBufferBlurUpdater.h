// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Components/PostBufferUpdate.h"
#include "Rendering/SlateRendererTypes.h"
#include "Slate/SPostBufferUpdate.h"

#include "PostBufferBlurUpdater.generated.h"

/**
 * Processor updater that sets the blur strength for a blur processor intra-frame on the renderthread
 */
UCLASS(NotBlueprintable, MinimalAPI)
class UPostBufferBlurUpdater : public USlatePostBufferProcessorUpdater
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "GaussianBlur")
	float GaussianBlurStrength = 10;

public:
	virtual TSharedPtr<FSlatePostProcessorUpdaterProxy > GetRenderThreadProxy() const override;
};

/**
 * Renderthread proxy for the blur processor updater
 */
class FPostBufferBlurUpdaterProxy : public FSlatePostProcessorUpdaterProxy 
{
public:

	/** Blur strength that will be copied over to the processor mid-frame */
	float GaussianBlurStrength_RenderThread = 10;

	virtual void UpdateProcessor_RenderThread(TSharedPtr<FSlateRHIPostBufferProcessorProxy>) const override;
};