// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Rendering/SlateRendererTypes.h"

#include "PostBufferUpdate.generated.h"

class SPostBufferUpdate;
class USlatePostBufferProcessorUpdater;
class FSlatePostProcessorUpdaterProxy ;
class FSlateRHIPostBufferProcessorProxy;

/**
 * Struct containing info needed to update a particular buffer
 */
USTRUCT()
struct FSlatePostBufferUpdateInfo
{
	GENERATED_BODY()

	/** Buffers that we should update, all of these buffers will be affected by 'bPerformDefaultPostBufferUpdate' if disabled */
	UPROPERTY(EditAnywhere, Category = "Behavior", meta = (AllowPrivateAccess = "true"))
	ESlatePostRT BufferToUpdate = ESlatePostRT::None;

	/** Optional processor updater for buffer, used to update a processor within a frame */
	UPROPERTY(EditAnywhere, Instanced, Category = "Behavior", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USlatePostBufferProcessorUpdater> PostParamUpdater = nullptr;
};

/**
 * Widget that when drawn, will trigger the slate post buffer to update. Does not draw anything itself.
 * This allows for you to perform layered UI / sampling effects with the GetSlatePost material functions,
 * by placing this widget after UI you would like to be processed / sampled is drawn.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class UPostBufferUpdate : public UWidget
{
	GENERATED_BODY()

public:
	UMG_API UPostBufferUpdate();

private:

	UE_DEPRECATED(5.7, "Now always true. The value of bUpdateOnlyPaintArea is ignored")
	/**
	 * True if we should only update the buffer within the bounds of this widget
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Behavior", meta = (AllowPrivateAccess = "true"))
	bool bUpdateOnlyPaintArea;

	UE_DEPRECATED(5.7, "Default post buffer updates are no longer performed under any circumstances")
	/**
	 * True if we should do the default post buffer update of the scene before any UI.
	 * If any PostBufferUpdate widget has this set as false, be default scene copy / processing will not occur.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Behavior", meta = (AllowPrivateAccess = "true"))
	bool bPerformDefaultPostBufferUpdate;

	UE_DEPRECATED(5.5, "BuffersToUpdate is deprecated. Please use UpdateBufferInfos. This array will be ignored if UpdateBufferInfos is used")
	/** Buffers that we should update, all of these buffers will be affected by 'bPerformDefaultPostBufferUpdate' if disabled */
	UPROPERTY(EditAnywhere, Category = "Behavior", meta = (AllowPrivateAccess = "true"))
	TArray<ESlatePostRT> BuffersToUpdate;

	/** Buffer to update when this widget is drawn, along with info needed to update that buffer if desired intra-frame */
	UPROPERTY(EditAnywhere, Category = "Behavior", meta = (AllowPrivateAccess = "true"))
	TArray<FSlatePostBufferUpdateInfo> UpdateBufferInfos;

public:

#if WITH_EDITOR
	bool GetUpdateOnlyPaintArea() const { return true; }

	/** True if this widget updates any buffer indicated in SlateBufferBits */
	UMG_API bool DoesUpdateSlatePostRT(ESlatePostRT SlateBufferBits) const ;
#endif

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UMG_API virtual void SynchronizeProperties() override;
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UWidget Interface

protected:
	TSharedPtr<SPostBufferUpdate> MyPostBufferUpdate;
};

/**
 * Class that can create a FPostParamUpdaterProxy whose lifetime
 * will be managed by the renderthread. This proxy will be given a 
 * Post buffer processor to update mid-frame.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, EditInlineNew, CollapseCategories)
class USlatePostBufferProcessorUpdater : public UObject
{
	GENERATED_BODY()

public:

	UE_DEPRECATED(5.7, "Now always false. The value of bSkipBufferUpdate is ignored")
	/**
	 * True implies we will skip the buffer update & only update the processor.
	 * Previously useful to reset params for processor runs next frame.
	 * Since there are no default buffer updates anymore, setting parameters
	 * for subsequent updates is meaningless as non-default updates will set
	 * all their parameters anyway.
	 */
	UPROPERTY()
	bool bSkipBufferUpdate = false;

public:
	virtual TSharedPtr<FSlatePostProcessorUpdaterProxy> GetRenderThreadProxy() const
	{
		return nullptr;
	};
};
