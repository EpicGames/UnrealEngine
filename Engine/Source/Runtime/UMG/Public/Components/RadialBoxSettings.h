// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RadialBoxSettings.generated.h"

USTRUCT(BlueprintType)
struct FRadialBoxSettings
{
	GENERATED_BODY()

	/* At what angle will we place the first element of the wheel? */
	UPROPERTY(EditAnywhere, Category = "Items", meta = (ClampMin = 0, ClampMax = 360))
	float StartingAngle;

	/* Distribute Items evenly in the whole circle. Checking this option ignores AngleBetweenItems */
	UPROPERTY(EditAnywhere, Category = "Items")
	bool bDistributeItemsEvenly;

	/* If enabled, Places the entries in clockwise order instead of counter-clockwise order */
	UPROPERTY(EditAnywhere, Category = "Items")
	bool bClockwiseOrder;

	/* Amount of Euler degrees that separate each item. Only used when bDistributeItemsEvenly is false */
	UPROPERTY(EditAnywhere, Category = "Items", meta = (EditCondition = "!bDistributeItemsEvenly", ClampMin = 0, ClampMax = 360))
	float AngleBetweenItems;

	/** If we need a section of a radial (for example half-a-radial) we can define a central angle < 360 (180 in case of half-a-radial). Used when bDistributeItemsEvenly is enabled. */
	UPROPERTY(EditAnywhere, Category = "Items", meta = (EditCondition = "bDistributeItemsEvenly", ClampMin = 0, ClampMax = 360))
	float SectorCentralAngle;

	/** How much space to leave between the widgets and the edge of the panel */
	UPROPERTY(EditAnywhere, Category = "Items", meta = (ClampMin = 0))
	float MarginSize;

	FRadialBoxSettings()
		: StartingAngle(0.f)
		, bDistributeItemsEvenly(true)
		, bClockwiseOrder(false)
		, AngleBetweenItems(0.f)
		, SectorCentralAngle(360.f)
		, MarginSize(0.f)
	{
	}
};