// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "RetainerBox.generated.h"

class SRetainerWidget;
class UMaterialInstanceDynamic;
class UMaterialInterface;

/**
 * The Retainer Box renders children widgets to a render target first before
 * later rendering that render target to the screen.  This allows both frequency
 * and phase to be controlled so that the UI can actually render less often than the
 * frequency of the main game render.  It also has the side benefit of allow materials
 * to be applied to the render target after drawing the widgets to apply a simple post process.
 *
 * * Single Child
 * * Caching / Performance
 */
UCLASS(MinimalAPI)
class URetainerBox : public UContentWidget
{
	GENERATED_UCLASS_BODY()

protected:
	UE_DEPRECATED(5.2, "Direct access to bRetainRender is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter = "SetRetainRendering", Getter = "IsRetainRendering", BlueprintSetter = "SetRetainRendering", Category = "Render Rules")
	bool bRetainRender = true;

public:
	UE_DEPRECATED(5.2, "Direct access to RenderOnInvalidation is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/**
	 * Should this widget redraw the contents it has every time it receives an invalidation request
	 * from it's children, similar to the invalidation panel.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsRenderOnInvalidation", Category = "Render Rules", meta = (EditCondition = bRetainRender))
	bool RenderOnInvalidation;

	UE_DEPRECATED(5.2, "Direct access to RenderOnPhase is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/**
	 * Should this widget redraw the contents it has every time the phase occurs.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsRenderOnPhase", Category = "Render Rules", meta = (EditCondition = bRetainRender))
	bool RenderOnPhase;

	UE_DEPRECATED(5.2, "Direct access to Phase is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/**
	 * The Phase this widget will draw on.
	 *
	 * If the Phase is 0, and the PhaseCount is 1, the widget will be drawn fresh every frame.
	 * If the Phase were 0, and the PhaseCount were 2, this retainer would draw a fresh frame every
	 * other frame.  So in a 60Hz game, the UI would render at 30Hz.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Render Rules", meta=(UIMin=0, ClampMin=0))
	int32 Phase;

	UE_DEPRECATED(5.2, "Direct access to PhaseCount is deprecated. Please use the getter. Note that this property is only set at construction and is not modifiable at runtime.")
	/**
	 * The PhaseCount controls how many phases are possible know what to modulus the current frame 
	 * count by to determine if this is the current frame to draw the widget on.
	 * 
	 * If the Phase is 0, and the PhaseCount is 1, the widget will be drawn fresh every frame.  
	 * If the Phase were 0, and the PhaseCount were 2, this retainer would draw a fresh frame every 
	 * other frame.  So in a 60Hz game, the UI would render at 30Hz.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Render Rules", meta=(UIMin=1, ClampMin=1))
	int32 PhaseCount;

public:

	/**
	 * Requests the retainer redrawn the contents it has.
	 */
	UFUNCTION(BlueprintCallable, Category="Retainer")
	UMG_API void SetRenderingPhase(int32 RenderPhase, int32 TotalPhases);

	/**
	 * Requests the retainer redrawn the contents it has.
	 */
	UFUNCTION(BlueprintCallable, Category="Retainer")
	UMG_API void RequestRender();

	/**
	 * Get the current dynamic effect material applied to the retainer box.
	 */
	UFUNCTION(BlueprintCallable, Category="Retainer|Effect")
	UMG_API UMaterialInstanceDynamic* GetEffectMaterial() const;

	/**
	 * Set a new effect material to the retainer widget.
	 */
	UFUNCTION(BlueprintCallable, Category="Retainer|Effect")
	UMG_API void SetEffectMaterial(UMaterialInterface* EffectMaterial);

	/**
	 * Sets the name of the texture parameter to set the render target to on the material.
	 */
	UFUNCTION(BlueprintCallable, Category="Retainer|Effect")
	UMG_API void SetTextureParameter(FName TextureParameter);

	/**
	* Set the flag for if we retain the render or pass-through
	*/
	UFUNCTION(BlueprintCallable, Category = "Retainer")
	UMG_API void SetRetainRendering(bool bInRetainRendering);

	/**
	 * Get the flag for if we retain the render or pass-through.
	 */
	UMG_API bool IsRetainRendering() const;

	/**
	 * Get the phase to render on.
	 */
	UMG_API int32 GetPhase() const;

	/**
	 * Get the total number of phases.
	 */
	UMG_API int32 GetPhaseCount() const;

	/**
	 * Get whether this widget should redraw the contents it has every time it receives an invalidation request.
	 */
	UMG_API bool IsRenderOnInvalidation() const;

	/**
	 * Get whether this widget should redraw the contents it has every time the phase occurs.
	 */
	UMG_API bool IsRenderOnPhase() const;

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

	UMG_API FGeometry GetCachedAllottedGeometry() const;

protected:

	UE_DEPRECATED(5.2, "Direct access to EffectMaterial is deprecated. Please use the getter or setter.")
	/**
	 * The effect to optionally apply to the render target.  We will set the texture sampler based on the name
	 * set in the @TextureParameter property.
	 * 
	 * If you want to adjust transparency of the final image, make sure you set Blend Mode to AlphaComposite (Pre-Multiplied Alpha)
	 * and make sure to multiply the alpha you're apply across the surface to the color and the alpha of the render target, otherwise
	 * you won't see the expected color.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetEffectMaterialInterface", Setter, BlueprintSetter = "SetEffectMaterial", Category = "Effect")
	TObjectPtr<UMaterialInterface> EffectMaterial;

	UE_DEPRECATED(5.2, "Direct access to TextureParameter is deprecated. Please use the getter or setter.")
	/**
	 * The texture sampler parameter of the @EffectMaterial, that we'll set to the render target.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetTextureParameter", Category="Effect")
	FName TextureParameter;

#if WITH_EDITORONLY_DATA
	/**
	 * If true, retained rendering occurs in designer
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Effect", meta = (EditCondition = bRetainRender))
	bool bShowEffectsInDesigner;
#endif

	//~ Begin UPanelWidget interface
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	//~ End UPanelWidget interface

	//~ Begin UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UWidget interface

	//~ Begin UObject interface
#if WITH_EDITOR
	UMG_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	//~ End UObject interface

	// Initialize RenderOnInvalidation in the constructor before the SWidget is constructed.
	UMG_API void InitRenderOnInvalidation(bool InRenderOnInvalidation);

	// Initialize RenderOnPhase in the constructor before the SWidget is constructed.
	UMG_API void InitRenderOnPhase(bool InRenderOnPhase);

	// Initialize Phase in the constructor before the SWidget is constructed.
	UMG_API void InitPhase(int32 InPhase);

	// Initialize PhaseCount in the constructor before the SWidget is constructed.
	UMG_API void InitPhaseCount(int32 InPhaseCount);
public:
	/**
	* Gets the name of the texture parameter to set the render target to on the material.
	*/
	UMG_API const FName& GetTextureParameter() const;

	/**
	* Gets the current dynamic effect material applied to the retainer box.
	*/
	UMG_API const UMaterialInterface* GetEffectMaterialInterface() const;

protected:
	TSharedPtr<class SRetainerWidget> MyRetainerWidget;
};
