// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Components/TextWidgetTypes.h"
#include "Engine/DataTable.h"
#include "RichTextBlock.generated.h"

class SRichTextBlock;
class UDataTable;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class URichTextBlockDecorator;

/** Simple struct for rich text styles */
USTRUCT()
struct FRichTextStyleRow : public FTableRowBase
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Appearance)
	FTextBlockStyle TextStyle;
};

/**
 * The rich text block
 *
 * * Fancy Text
 * * No Children
 */
UCLASS(MinimalAPI)
class URichTextBlock : public UTextLayoutWidget
{
	GENERATED_BODY()
protected:
	UE_DEPRECATED(5.2, "Direct access to Text is deprecated. Please use the getter or setter.")
	/** The text to display */
	UPROPERTY(EditAnywhere, Category = Content, BlueprintSetter = "SetText", BlueprintGetter = "GetText", Getter, Setter, meta = (MultiLine = "true"))
	FText Text;

	UE_DEPRECATED(5.2, "Direct access to TextStyleSet is deprecated. Please use the getter or setter.")
	/**  */
	UPROPERTY(EditAnywhere, Category = Appearance, BlueprintSetter = "SetTextStyleSet", BlueprintGetter = "GetTextStyleSet", Getter, Setter, meta = (RequiredAssetDataTags = "RowStructure=/Script/UMG.RichTextStyleRow"))
	TObjectPtr<UDataTable> TextStyleSet;

	/**  */
	UPROPERTY(EditAnywhere, Category = Appearance)
	TArray<TSubclassOf<URichTextBlockDecorator>> DecoratorClasses;

	UE_DEPRECATED(5.2, "Direct access to TextOverflowPolicy is deprecated. Please use the getter or setter.")
	/** Text style to apply by default to text in this block */
	UPROPERTY(EditAnywhere, Category = Appearance, BlueprintSetter = "SetDefaultTextStyle", Setter = "SetDefaultTextStyle", Getter, meta = (EditCondition = bOverrideDefaultStyle))
	FTextBlockStyle DefaultTextStyleOverride;

	UE_DEPRECATED(5.2, "Direct access to MinDesiredWidth is deprecated. Please use the getter or setter.")
	/** The minimum desired size for the text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = "SetMinDesiredWidth", Getter, Setter, Category = Appearance)
	float MinDesiredWidth;
	
	/** True to specify the default text style for this rich text inline, overriding any default provided in the style set table */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool bOverrideDefaultStyle = false;

	UE_DEPRECATED(5.2, "Direct access to TextTransformPolicy is deprecated. Please use the getter or setter.")
	/** The text transformation policy to apply to this text block */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = "SetTextTransformPolicy", Getter = "GetTransformPolicy", Setter, Category = Appearance, meta = (DisplayName = "Transform Policy"))
	ETextTransformPolicy TextTransformPolicy;

	UE_DEPRECATED(5.2, "Direct access to TextOverflowPolicy is deprecated. Please use the getter or setter.")
	/** Sets what happens to text that is clipped and doesn't fit within the clip rect for this widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = "SetTextOverflowPolicy", Getter = "GetOverflowPolicy", Setter, Category = Clipping, AdvancedDisplay, meta = (DisplayName = "Overflow Policy"))
	ETextOverflowPolicy TextOverflowPolicy;

public:
	/**  
	 * Sets the color and opacity of the default text in this rich text block
	 * @param InColorAndOpacity		The new text color and opacity
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetDefaultColorAndOpacity(FSlateColor InColorAndOpacity);

	/**  
	 * Sets the color and opacity of the default text drop shadow
	 * Note: if opacity is zero no shadow will be drawn
	 * @param InShadowColorAndOpacity		The new drop shadow color and opacity
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetDefaultShadowColorAndOpacity(FLinearColor InShadowColorAndOpacity);

	/**  
	 * Sets the offset that the default text drop shadow should be drawn at
	 * @param InShadowOffset		The new offset
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetDefaultShadowOffset(FVector2D InShadowOffset);

	/**
	 * Dynamically set the default font info for this rich text block
	 * @param InFontInfo The new font info
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetDefaultFont(FSlateFontInfo InFontInfo);

	/**
	 * Dynamically set the default strike brush for this rich text block
	 * @param InStrikeBrush The new brush to use to strike through text
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetDefaultStrikeBrush(const FSlateBrush& InStrikeBrush);

	/**
	 *  Set the minimum desired width for this rich text block
	 *  @param InMinDesiredWidth new minimum desired width
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetMinDesiredWidth(float InMinDesiredWidth);
	
	/**
	 * Set the auto wrap for this rich text block
	 * @param InAutoTextWrap to turn wrap on or off
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetAutoWrapText(bool InAutoTextWrap);

	/**
	 * Set the text transformation policy for this text block.
	 * @param InTransformPolicy the new text transformation policy.
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetTextTransformPolicy(ETextTransformPolicy InTransformPolicy);

	/**
	* Set the text overflow policy for this text block.
	* @param InOverflowPolicy the new text overflow policy.
	*/
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetTextOverflowPolicy(ETextOverflowPolicy InOverflowPolicy);

	/** 
	 * Wholesale override of the currently established default text style
	 * @param InDefaultTextStyle The new text style to apply to all default (i.e. undecorated) text in the block
	 */
	UFUNCTION(BlueprintCallable, Category = Appearance)
	UMG_API void SetDefaultTextStyle(const FTextBlockStyle& InDefaultTextStyle);

	UFUNCTION(BlueprintCallable, Category = Appearance)
	UMG_API void SetDefaultMaterial(UMaterialInterface* InMaterial);

	/** Remove all overrides made to the default text style and return to the style specified in the style set data table */
	UFUNCTION(BlueprintCallable, Category = Appearance)
	UMG_API void ClearAllDefaultStyleOverrides();

	/**
	 * Creates a dynamic material for the default font or returns it if it already
	 * exists
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API UMaterialInstanceDynamic* GetDefaultDynamicMaterial();

	/**
	 * Replaces the existing decorators with the list provided
	 */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetDecorators(const TArray<TSubclassOf<URichTextBlockDecorator>>& InDecoratorClasses);

public:
	UMG_API URichTextBlock(const FObjectInitializer& ObjectInitializer);

	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	// UWidget interface
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual void OnCreationFromPalette() override;
	UMG_API virtual void ValidateCompiledDefaults(IWidgetCompilerLog& CompileLog) const override;
	// End UWidget interface
#endif

	/**
	 * Returns widgets text.
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API FText GetText() const;
	
	/**
	 * Directly sets the widget text.
	 * Warning: This will wipe any binding created for the Text property!
	 * @param InText The text to assign to the widget
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget", meta = (AutoCreateRefTerm = "InText"))
	UMG_API virtual void SetText(const FText& InText);

	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API UDataTable* GetTextStyleSet() const;

	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetTextStyleSet(UDataTable* NewTextStyleSet);

	UMG_API const FTextBlockStyle& GetDefaultTextStyle() const;
	UMG_API const FTextBlockStyle& GetCurrentDefaultTextStyle() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API URichTextBlockDecorator* GetDecoratorByClass(TSubclassOf<URichTextBlockDecorator> DecoratorClass);

	/**
	 * Causes the text to reflow it's layout and re-evaluate any decorators
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void RefreshTextLayout();

protected:
	//~ Begin UTextLayoutWidget Interface
	UMG_API virtual void OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions) override;
	UMG_API virtual void OnJustificationChanged(ETextJustify::Type InJustification) override;
	UMG_API virtual void OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy) override;
	UMG_API virtual void OnAutoWrapTextChanged(bool InAutoWrapText) override;
	UMG_API virtual void OnWrapTextAtChanged(float InWrapTextAt) override;
	UMG_API virtual void OnLineHeightPercentageChanged(float InLineHeightPercentage) override;
	UMG_API virtual void OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine) override;
	UMG_API virtual void OnMarginChanged(const FMargin& InMargin) override;
	//~ End UTextLayoutWidget Interface

	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	
	UMG_API virtual void UpdateStyleData();
	UMG_API void RebuildStyleInstance();
	UMG_API virtual void CreateDecorators(TArray< TSharedRef< class ITextDecorator > >& OutDecorators);
	UMG_API virtual TSharedPtr< class IRichTextMarkupParser > CreateMarkupParser();
	UMG_API virtual TSharedPtr< class IRichTextMarkupWriter > CreateMarkupWriter();

	UMG_API void BeginDefaultStyleOverride();
	UMG_API virtual void ApplyUpdatedDefaultTextStyle();

	UMG_API const FTextBlockStyle& GetDefaultTextStyleOverride() const;
	UMG_API float GetMinDesiredWidth() const;
	UMG_API ETextTransformPolicy GetTransformPolicy() const;
	UMG_API ETextOverflowPolicy GetOverflowPolicy() const;

protected:
	UPROPERTY(Transient)
	FTextBlockStyle DefaultTextStyle;

	UPROPERTY(Transient)
	TArray<TObjectPtr<URichTextBlockDecorator>> InstanceDecorators;
	TSharedPtr<class FSlateStyleSet> StyleInstance;
	TSharedPtr<SRichTextBlock> MyRichTextBlock;
};
