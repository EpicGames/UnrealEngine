// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ITextDecorator.h"
#include "RichTextBlockDecorator.generated.h"

class ISlateStyle;
class URichTextBlockDecorator;
class URichTextBlock;

class FRichTextDecorator : public ITextDecorator
{
public:
	UMG_API FRichTextDecorator(URichTextBlock* InOwner);

	virtual ~FRichTextDecorator() = default;

	/** Override this function to specify which types of tags are handled by this decorator */
	virtual bool Supports(const FTextRunParseResults& RunParseResult, const FString& Text) const override
	{
		return false;
	}

	UMG_API virtual TSharedRef<ISlateRun> Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style) override final;

protected:
	/** Override this function if you want to create a unique widget like an image */
	UMG_API virtual TSharedPtr<SWidget> CreateDecoratorWidget(const FTextRunInfo& RunInfo, const FTextBlockStyle& DefaultTextStyle) const;

	/** Override this function if you want to dynamically generate text, optionally changing the style. InOutString will start as the content between tags */
	UMG_API virtual void CreateDecoratorText(const FTextRunInfo& RunInfo, FTextBlockStyle& InOutTextStyle, FString& InOutString) const;

	URichTextBlock* Owner;
};

UCLASS(Abstract, Blueprintable, MinimalAPI)
class URichTextBlockDecorator : public UObject
{
	GENERATED_BODY()

public:
	UMG_API URichTextBlockDecorator(const FObjectInitializer& ObjectInitializer);

	UMG_API virtual TSharedPtr<ITextDecorator> CreateDecorator(URichTextBlock* InOwner);
};
