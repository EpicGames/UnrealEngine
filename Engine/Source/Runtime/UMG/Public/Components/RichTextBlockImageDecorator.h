// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ITextDecorator.h"
#include "Components/RichTextBlockDecorator.h"
#include "Engine/DataTable.h"
#include "RichTextBlockImageDecorator.generated.h"

class ISlateStyle;

/** Simple struct for rich text styles */
USTRUCT(Blueprintable, BlueprintType)
struct FRichImageRow : public FTableRowBase
{
	GENERATED_USTRUCT_BODY()

public:

	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush Brush;
};

/**
 * Allows you to setup an image decorator that can be configured
 * to map certain keys to certain images.  We recommend you subclass this
 * as a blueprint to configure the instance.
 *
 * Understands the format <img id="NameOfBrushInTable"></>
 */
UCLASS(Abstract, Blueprintable, MinimalAPI)
class URichTextBlockImageDecorator : public URichTextBlockDecorator
{
	GENERATED_BODY()

public:
	UMG_API URichTextBlockImageDecorator(const FObjectInitializer& ObjectInitializer);

	UMG_API virtual TSharedPtr<ITextDecorator> CreateDecorator(URichTextBlock* InOwner) override;

	UMG_API virtual const FSlateBrush* FindImageBrush(FName TagOrId, bool bWarnIfMissing);

protected:

	UMG_API FRichImageRow* FindImageRow(FName TagOrId, bool bWarnIfMissing);

	UPROPERTY(EditAnywhere, Category=Appearance, meta = (RequiredAssetDataTags = "RowStructure=/Script/UMG.RichImageRow"))
	TObjectPtr<class UDataTable> ImageSet;
};
