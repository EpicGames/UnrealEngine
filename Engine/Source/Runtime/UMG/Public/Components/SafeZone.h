// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "Widgets/Layout/SSafeZone.h"
#include "SafeZone.generated.h"

/**
 * The Safe-Zone widget is an essential part of developing a game UI that can run on lots of different non-PC platforms.
 * While a modern flat panel computer monitor may not have over scan issues, this is a common occurrence for Consoles.  
 * It's common for TVs to have extra pixels under the bezel, in addition to projectors and projection TVs having potentially
 * several vertical and horizontal columns of pixels hidden behind or against a black border of the projection screen.
 * 
 * Useful testing console commands to help, simulate the safe zone on PC,
 *   r.DebugSafeZone.TitleRatio 0.96
 *   r.DebugActionZone.ActionRatio 0.96
 * 
 * To enable a border to visualize the safe zone, use this console command,
 * r.DebugSafeZone.Mode controls the debug visualization overlay (0..1, default 0).
 *   0: Do not display the safe zone overlay.
 *   1: Display the overlay for the title safe zone.
 */
UCLASS(MinimalAPI)
class USafeZone : public UContentWidget
{
	GENERATED_BODY()
public:
	UMG_API USafeZone();

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;

	UMG_API virtual void OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs) override;
#endif

	UMG_API virtual void OnSlotAdded( UPanelSlot* Slot ) override;
	UMG_API virtual void OnSlotRemoved( UPanelSlot* Slot ) override;
	UMG_API virtual UClass* GetSlotClass() const override;

	UMG_API void UpdateWidgetProperties();

	UFUNCTION(BlueprintCallable, Category = "SafeZone")
	UMG_API void SetSidesToPad(bool InPadLeft, bool InPadRight, bool InPadTop, bool InPadBottom);

public:

	UE_DEPRECATED(5.2, "Direct access to PadLeft is deprecated. Please use the getter or setter.")
	/** If this safe zone should pad for the left side of the screen's safe zone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "SafeZone")
	bool PadLeft;

	UE_DEPRECATED(5.2, "Direct access to PadRight is deprecated. Please use the getter or setter.")
	/** If this safe zone should pad for the right side of the screen's safe zone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "SafeZone")
	bool PadRight;

	UE_DEPRECATED(5.2, "Direct access to PadTop is deprecated. Please use the getter or setter.")
	/** If this safe zone should pad for the top side of the screen's safe zone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "SafeZone")
	bool PadTop;

	UE_DEPRECATED(5.2, "Direct access to PadBottom is deprecated. Please use the getter or setter.")
	/** If this safe zone should pad for the bottom side of the screen's safe zone */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "SafeZone")
	bool PadBottom;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	UMG_API void SetPadLeft(bool InPadLeft);
	UMG_API bool GetPadLeft() const;

	UMG_API void SetPadRight(bool InPadRight);
	UMG_API bool GetPadRight() const;

	UMG_API void SetPadTop(bool InPadTop);
	UMG_API bool GetPadTop() const;

	UMG_API void SetPadBottom(bool InPadBottom);
	UMG_API bool GetPadBottom() const;

	TSharedPtr< class SSafeZone > MySafeZone;

#if WITH_EDITOR
	TOptional<FVector2D> DesignerSize;
	TOptional<float> DesignerDpi;
#endif
};
