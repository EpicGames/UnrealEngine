// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PanelSlot.h"
#include "Layout/Margin.h"

#include "SafeZoneSlot.generated.h"

class SSafeZone;

UCLASS(MinimalAPI)
class USafeZoneSlot : public UPanelSlot
{
	GENERATED_BODY()
public:

	UE_DEPRECATED(5.2, "Direct access to bIsTitleSafe is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter = "IsTitleSafe", Setter = "SetIsTitleSafe", Category = SafeZone)
	bool bIsTitleSafe;

	UE_DEPRECATED(5.2, "Direct access to SafeAreaScale is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = SafeZone )
	FMargin SafeAreaScale;

	UE_DEPRECATED(5.2, "Direct access to HAlign is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter = "GetHorizontalAlignment", Setter = "SetHorizontalAlignment", Category = SafeZone)
	TEnumAsByte< EHorizontalAlignment > HAlign;

	UE_DEPRECATED(5.2, "Direct access to VAlign is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter = "GetVerticalAlignment", Setter = "SetVerticalAlignment", Category = SafeZone)
	TEnumAsByte< EVerticalAlignment > VAlign;

	UE_DEPRECATED(5.2, "Direct access to Padding is deprecated. Please use the getter or setter.")
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = SafeZone )
	FMargin Padding;

	UMG_API USafeZoneSlot();

	UMG_API void SetIsTitleSafe(bool InIsTitleSafe);
	UMG_API bool IsTitleSafe() const;

	UMG_API void SetSafeAreaScale(const FMargin& InSafeAreaScale);
	UMG_API const FMargin& GetSafeAreaScale() const;

	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHAlign);
	UMG_API const EHorizontalAlignment GetHorizontalAlignment() const;

	UMG_API void SetVerticalAlignment(EVerticalAlignment InVAlign);
	UMG_API const EVerticalAlignment GetVerticalAlignment() const;

	UMG_API void SetPadding(const FMargin& InPadding);
	UMG_API const FMargin& GetPadding() const;

	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying slot for the slate button. */
	UMG_API void BuildSlot(TSharedRef<SSafeZone> InSafeZone);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

private:

	/** A pointer to the button to allow us to adjust the alignment, padding...etc at runtime. */
	TWeakPtr<SSafeZone> SafeZone;
};
