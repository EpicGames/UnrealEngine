// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "Widgets/Layout/SScaleBox.h"
#include "ScaleBox.generated.h"

/**
 * Allows you to place content with a desired size and have it scale to meet the constraints placed on this box's alloted area.  If
 * you needed to have a background image scale to fill an area but not become distorted with different aspect ratios, or if you need
 * to auto fit some text to an area, this is the control for you.
 *
 * * Single Child
 * * Aspect Ratio
 */
UCLASS(config=Engine, MinimalAPI)
class UScaleBox : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.2, "Direct access to Stretch is deprecated. Please use the getter or setter.")
	/** The stretching rule to apply when content is stretched */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, BlueprintSetter = "SetStretch", Category = "Stretching")
	TEnumAsByte<EStretch::Type> Stretch;

	UE_DEPRECATED(5.2, "Direct access to StretchDirection is deprecated. Please use the getter or setter.")
	/** Controls in what direction content can be scaled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, BlueprintSetter = "SetStretchDirection", Category = "Stretching")
	TEnumAsByte<EStretchDirection::Type> StretchDirection;

	UE_DEPRECATED(5.2, "Direct access to UserSpecifiedScale is deprecated. Please use the getter or setter.")
	/** Optional scale that can be specified by the User. Used only for UserSpecified stretching. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, BlueprintSetter = "SetUserSpecifiedScale", Category = "Stretching")
	float UserSpecifiedScale;

	UE_DEPRECATED(5.2, "Direct access to IgnoreInheritedScale is deprecated. Please use the getter or setter.")
	/** Optional bool to ignore the inherited scale. Applies inverse scaling to counteract parents before applying the local scale operation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter = "IsIgnoreInheritedScale", BlueprintSetter = "SetIgnoreInheritedScale", Category = "Stretching")
	bool IgnoreInheritedScale;

public:
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetStretch(EStretch::Type InStretch);

	UMG_API EStretch::Type GetStretch() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetStretchDirection(EStretchDirection::Type InStretchDirection);

	UMG_API EStretchDirection::Type GetStretchDirection() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetUserSpecifiedScale(float InUserSpecifiedScale);

	UMG_API float GetUserSpecifiedScale() const;

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetIgnoreInheritedScale(bool bInIgnoreInheritedScale);

	UMG_API bool IsIgnoreInheritedScale() const;

public:

	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UMG_API virtual void OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs) override;
#endif

	UMG_API virtual void Serialize(FArchive& Ar) override;

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:
	TSharedPtr<SScaleBox> MyScaleBox;

#if WITH_EDITOR
	TOptional<FVector2D> DesignerSize;
#endif

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
