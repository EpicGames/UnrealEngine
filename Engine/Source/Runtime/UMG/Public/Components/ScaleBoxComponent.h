// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ContentWidget.h"
#include "CoreMinimal.h"
#include "Extensions/UIComponent.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SWidget.h"

#include "ScaleBoxComponent.generated.h"

class SScaleBox;

/** This is a class for a Component that wraps the Owner widget with a Scale Box */
UCLASS(MinimalAPI, Experimental)
class UScaleBoxComponent : public UUIComponent
{
	GENERATED_BODY()

public:
	UMG_API void SetStretch(EStretch::Type InStretch);

	UMG_API EStretch::Type GetStretch() const;

	UMG_API void SetStretchDirection(EStretchDirection::Type InStretchDirection);

	UMG_API EStretchDirection::Type GetStretchDirection() const;

	UMG_API void SetUserSpecifiedScale(float InUserSpecifiedScale);

	UMG_API float GetUserSpecifiedScale() const;

	UMG_API void SetIgnoreInheritedScale(bool bInIgnoreInheritedScale);

	UMG_API bool IsIgnoreInheritedScale() const;

	// Child slot functions:

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

#if WITH_EDITOR
	UMG_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	// TODO vinz: Implement OnDesignerChanged to work for Components. This currently is just a virtual function for UWidgets.
	//UMG_API virtual void OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs) override;

// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif // WITH_EDITOR
	
	virtual TSharedRef<SWidget> RebuildWidgetWithContent(const TSharedRef<SWidget> OwnerContent) override;

// TODO vinz: Implement OnDesignerChanged to work for Components. This currently is just a virtual function for UWidgets.
//#if WITH_EDITOR
//	TOptional<FVector2D> DesignerSize;
//#endif

private:

	void SynchronizeProperties(TSharedRef<SScaleBox> ScaleBox);

	// Child slot properties

	/** This property is for the Widget that owns this Component. The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Scale Box Slot", meta = (AllowPrivateAccess = true))
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = HAlign_Center;

	/** This property is for the Widget that owns this Component. The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Scale Box Slot", meta = (AllowPrivateAccess = true))
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = VAlign_Center;

	// Scale Box properties:

	/** The stretching rule to apply when content is stretched */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Stretching", meta = (AllowPrivateAccess = true))
	TEnumAsByte<EStretch::Type> Stretch = EStretch::ScaleToFit;

	/** Controls in what direction content can be scaled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Stretching", meta = (AllowPrivateAccess = true))
	TEnumAsByte<EStretchDirection::Type> StretchDirection = EStretchDirection::Both;

	/** Optional scale that can be specified by the User. Used only for UserSpecified stretching. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Stretching", meta = (AllowPrivateAccess = true))
	float UserSpecifiedScale = 1.0f;

	/** Optional bool to ignore the inherited scale. Applies inverse scaling to counteract parents before applying the local scale operation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="IsIgnoreInheritedScale", Setter, Category = "Stretching", meta = (AllowPrivateAccess = true))
	bool IgnoreInheritedScale = false;

	TWeakPtr<SScaleBox> MyScaleBox;
};