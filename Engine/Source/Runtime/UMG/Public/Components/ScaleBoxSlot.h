// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/PanelSlot.h"

#include "ScaleBoxSlot.generated.h"

class SScaleBox;

/** The Slot for the UScaleBoxSlot, contains the widget displayed in a button's single slot */
UCLASS(MinimalAPI)
class UScaleBoxSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "Padding is not used.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY()
	FMargin Padding_DEPRECATED;
#endif

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category="Layout|ScaleBox Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Setter, Category="Layout|ScaleBox Slot" )
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:
	UE_DEPRECATED(5.1, "SetPadding is deprecated")
	UFUNCTION(BlueprintCallable, Category="Layout|ScaleBox Slot", meta = (DeprecatedFunction, DeprecatedMessage = "The function is deprecated - There is no padding on ScaleBox."))
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|ScaleBox Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);


	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|ScaleBox Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying slot for the slate ScaleBox. */
	UMG_API void BuildSlot(TSharedRef<SScaleBox> InScaleBox);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

private:
	/** A pointer to the button to allow us to adjust the size, padding...etc at runtime. */
	TWeakPtr<SScaleBox> ScaleBox;
};
