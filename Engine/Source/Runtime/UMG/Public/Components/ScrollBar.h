// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Components/Widget.h"
#include "ScrollBar.generated.h"

/** */
UCLASS(Experimental, MinimalAPI)
class UScrollBar : public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.2, "Direct access to WidgetStyle is deprecated. Please use the getter or setter.")
	/** Style of the scrollbar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Style", meta=( DisplayName="Style" ))
	FScrollBarStyle WidgetStyle;

	UE_DEPRECATED(5.2, "Direct access to bAlwaysShowScrollbar is deprecated. Please use the getter or setter.")
	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsAlwaysShowScrollbar", Setter = "SetAlwaysShowScrollbar", Category = "Behavior")
	bool bAlwaysShowScrollbar;

	UE_DEPRECATED(5.2, "Direct access to bAlwaysShowScrollbarTrack is deprecated. Please use the getter or setter.")
	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsAlwaysShowScrollbarTrack", Setter = "SetAlwaysShowScrollbarTrack", Category = "Behavior")
	bool bAlwaysShowScrollbarTrack;

	UE_DEPRECATED(5.2, "Direct access to Orientation is deprecated. Please use the getter. Note that the orientation of a scrollbar is only set at construction and is not modifiable at runtime.")
	/**  */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Behavior")
	TEnumAsByte<EOrientation> Orientation;

	UE_DEPRECATED(5.2, "Direct access to Thickness is deprecated. Please use the getter or setter.")
	/** The thickness of the scrollbar thumb */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Behavior")
	FVector2D Thickness;

	UE_DEPRECATED(5.2, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The margin around the scrollbar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Behavior")
	FMargin Padding;

public:

	/**
	* Set the offset and size of the track's thumb.
	* Note that the maximum offset is 1.0-ThumbSizeFraction.
	* If the user can view 1/3 of the items in a single page, the maximum offset will be ~0.667f
	*
	* @param InOffsetFraction     Offset of the thumbnail from the top as a fraction of the total available scroll space.
	* @param InThumbSizeFraction  Size of thumbnail as a fraction of the total available scroll space.
	*/
	UFUNCTION(BlueprintCallable, Category="Scrolling")
	UMG_API void SetState(float InOffsetFraction, float InThumbSizeFraction);

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	//~ Begin UObject Interface
#if WITH_EDITORONLY_DATA
	UMG_API virtual void Serialize(FArchive& Ar) override;
#endif // if WITH_EDITORONLY_DATA
	//~ End UObject Interface

#if WITH_EDITOR
	//~ Begin UWidget Interface
	UMG_API virtual const FText GetPaletteCategory() override;
	//~ End UWidget Interface
#endif

	/** @return the style of scrollbar. */
	UMG_API const FScrollBarStyle& GetWidgetStyle() const;
	/** Sets the style of scrollbar. */
	UMG_API void SetWidgetStyle(const FScrollBarStyle& InWidgetStyle);

	/** @return True if the scrollbar should always show. */
	UMG_API bool IsAlwaysShowScrollbar() const;
	/** Sets whether the scrollbar should always show. */
	UMG_API void SetAlwaysShowScrollbar(bool bNewValue);

	/** @return True if the scrollbar track should always show. */
	UMG_API bool IsAlwaysShowScrollbarTrack() const;
	/** Sets whether the scrollbar track should always show */
	UMG_API void SetAlwaysShowScrollbarTrack(bool bNewValue);

	/** @return the orientation of the scrollbar. */
	UMG_API EOrientation GetOrientation() const;

	/** @return the thickness of the scrollbar. */
	UMG_API FVector2D GetThickness() const;
	/** Sets the thickness of the scrollbar. */
	UMG_API void SetThickness(const FVector2D& InThickness);

	/** Sets the padding of the scrollbar. */
	UMG_API FMargin GetPadding() const;
	/** @return the padding of the scrollbar. */
	UMG_API void SetPadding(const FMargin& InPadding);

protected:

	TSharedPtr<class SScrollBar> MyScrollBar;

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

	/** Initialize the orientation of the scrollbar in the constructor before the SWidget is constructed. */
	UMG_API void InitOrientation(EOrientation InOrientation);
};
