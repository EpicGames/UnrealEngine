// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/WidgetPath.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Components/PanelWidget.h"
#include "Containers/Ticker.h"
#include "ScrollBox.generated.h"

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnScrollBoxFocusReceivedEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnScrollBoxFocusLostEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnScrollBoxFocusUpdatedEvent, const UWidget*, ScrollBox, const bool, bFocused);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnUserScrolledEvent, float, CurrentOffset);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnScrollBarVisibilityChangedEvent, ESlateVisibility , NewVisibility);

/**
 * An arbitrary scrollable collection of widgets.  Great for presenting 10-100 widgets in a list.  Doesn't support virtualization.
 */
UCLASS(MinimalAPI)
class UScrollBox : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

private:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Scroll", meta = (AllowPrivateAccess = "true", DisplayAfter = "bAnimateWheelScrolling"))
	float ScrollAnimationInterpolationSpeed = 15.f;

	/** True to allow scrolling using touch input. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="GetIsTouchScrollingEnabled", Setter="SetIsTouchScrollingEnabled", Category = "Scroll", meta = (AllowPrivateAccess = "true", DisplayAfter = "WheelScrollMultiplier"))
	bool bEnableTouchScrolling = true;
	
	/** Option to pass touch input events to widgets behind the ScrollBox. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetConsumePointerInput", Setter = "SetConsumePointerInput", Category = "Scroll", meta = (AllowPrivateAccess = "true", DisplayAfter = "bEnableTouchScrolling"))
	bool bConsumePointerInput = true;
		
	/** The virtual mouse wheel key for gamepad input */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetAnalogMouseWheelKey", Setter = "SetAnalogMouseWheelKey", Category = "Scroll", meta = (AllowPrivateAccess = "true", DisplayAfter = "bConsumePointerInput"))
	FKey AnalogMouseWheelKey;

	/** If true, scrollbox can receive focus. On gamepad, ensure AnalogMouseWheelKey is set to allow user interaction */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetIsFocusable", Setter = "SetIsFocusable", Category = "Scroll", meta = (AllowPrivateAccess = "true", DisplayAfter = "AnalogMouseWheelKey"))
	bool bIsFocusable;

public:

	UE_DEPRECATED(5.2, "Direct access to WidgetStyle is deprecated. Please use the getter or setter.")
	/** The style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Style", meta=( DisplayName="Style" ))
	FScrollBoxStyle WidgetStyle;

	UE_DEPRECATED(5.2, "Direct access to WidgetBarStyle is deprecated. Please use the getter or setter.")
	/** The bar style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Style", meta=( DisplayName="Bar Style" ))
	FScrollBarStyle WidgetBarStyle;

	UE_DEPRECATED(5.2, "Direct access to Orientation is deprecated. Please use the getter or setter.")
	/** The orientation of the scrolling and stacking in the box. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetOrientation", Category="Scroll")
	TEnumAsByte<EOrientation> Orientation;

	UE_DEPRECATED(5.2, "Direct access to ScrollBarVisibility is deprecated. Please use the getter or setter.")
	/** Visibility */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetScrollBarVisibility", Category="Scroll")
	ESlateVisibility ScrollBarVisibility;

	UE_DEPRECATED(5.2, "Direct access to ConsumeMouseWheel is deprecated. Please use the getter or setter.")
	/** When mouse wheel events should be consumed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetConsumeMouseWheel", Category = "Scroll")
	EConsumeMouseWheel ConsumeMouseWheel;

	UE_DEPRECATED(5.2, "Direct access to ScrollbarThickness is deprecated. Please use the getter or setter.")
	/** The thickness of the scrollbar thumb */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetScrollbarThickness", Category="Scroll")
	FVector2D ScrollbarThickness;

	UE_DEPRECATED(5.2, "Direct access to ScrollbarPadding is deprecated. Please use the getter or setter.")
	/** The margin around the scrollbar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetScrollbarPadding", Category = "Scroll")
	FMargin ScrollbarPadding;

	/**  */
	UE_DEPRECATED(5.2, "Direct access to AlwaysShowScrollbar is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsAlwaysShowScrollbar", Setter, BlueprintSetter = "SetAlwaysShowScrollbar", Category = "Scroll")
	bool AlwaysShowScrollbar;

	/**  */
	UE_DEPRECATED(5.2, "Direct access to AlwaysShowScrollbarTrack is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsAlwaysShowScrollbarTrack", Setter, Category = "Scroll")
	bool AlwaysShowScrollbarTrack;

	UE_DEPRECATED(5.2, "Direct access to AllowOverscroll is deprecated. Please use the getter or setter.")
	/**  Disable to stop scrollbars from activating inertial overscrolling */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsAllowOverscroll", Setter, BlueprintSetter = "SetAllowOverscroll", Category = "Scroll")
	bool AllowOverscroll;

	UE_DEPRECATED(5.2, "Direct access to BackPadScrolling is deprecated. Please use the getter. Note that BackPadScrolling is only set at construction and is not modifiable at runtime.")
	/** Whether to back pad this scroll box, allowing user to scroll backward until child contents are no longer visible */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsBackPadScrolling", Category = "Scroll")
	bool BackPadScrolling;

	UE_DEPRECATED(5.2, "Direct access to FrontPadScrolling is deprecated. Please use the getter. Note that FrontPadScrolling is only set at construction and is not modifiable at runtime.")
	/** Whether to front pad this scroll box, allowing user to scroll forward until child contents are no longer visible */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter = "IsFrontPadScrolling", Category = "Scroll")
	bool FrontPadScrolling;
	
	UE_DEPRECATED(5.2, "Direct access to bAnimateWheelScrolling is deprecated. Please use the getter or setter.")
	/** True to lerp smoothly when wheel scrolling along the scroll box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsAnimateWheelScrolling", Setter = "SetAnimateWheelScrolling", BlueprintSetter = "SetAnimateWheelScrolling", Category = "Scroll")
	bool bAnimateWheelScrolling = false;

	UE_DEPRECATED(5.2, "Direct access to NavigationDestination is deprecated. Please use the getter or setter.")
	/** Sets where to scroll a widget to when using explicit navigation or if ScrollWhenFocusChanges is enabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetNavigationDestination", Getter, Setter, Category = "Scroll")
	EDescendantScrollDestination NavigationDestination;

	UE_DEPRECATED(5.2, "Direct access to NavigationScrollPadding is deprecated. Please use the getter. Note that NavigationScrollPadding is only set at construction and is not modifiable at runtime.")
	/**
	 * The amount of padding to ensure exists between the item being navigated to, at the edge of the
	 * scrollbox.  Use this if you want to ensure there's a preview of the next item the user could scroll to.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Getter, Category="Scroll")
	float NavigationScrollPadding;

	UE_DEPRECATED(5.2, "Direct access to ScrollWhenFocusChanges is deprecated. Please use the getter or setter.")
	/** Scroll behavior when user focus is given to a child widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetScrollWhenFocusChanges", Category="Scroll", meta=(DisplayName="Scroll When Focus Changes"))
	EScrollWhenFocusChanges ScrollWhenFocusChanges;
	
	UE_DEPRECATED(5.2, "Direct access to bAllowRightClickDragScrolling is deprecated. Please use the getter or setter.")
	/** Option to disable right-click-drag scrolling */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsAllowRightClickDragScrolling", Setter = "SetAllowRightClickDragScrolling", Category = "Scroll")
	bool bAllowRightClickDragScrolling;

	UE_DEPRECATED(5.2, "Direct access to WheelScrollMultiplier is deprecated. Please use the getter or setter.")
	/** The multiplier to apply when wheel scrolling */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetWheelScrollMultiplier", Category = "Scroll")
	float WheelScrollMultiplier = 1.f;

	UMG_API void SetWidgetStyle(const FScrollBoxStyle& NewWidgetStyle);

	UMG_API const FScrollBoxStyle& GetWidgetStyle() const;

	UMG_API void SetWidgetBarStyle(const FScrollBarStyle& NewWidgetBarStyle);

	UMG_API const FScrollBarStyle& GetWidgetBarStyle() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetConsumeMouseWheel(EConsumeMouseWheel NewConsumeMouseWheel);

	UMG_API EConsumeMouseWheel GetConsumeMouseWheel() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetOrientation(EOrientation NewOrientation);

	UMG_API EOrientation GetOrientation() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetScrollBarVisibility(ESlateVisibility NewScrollBarVisibility);

	UMG_API ESlateVisibility GetScrollBarVisibility() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetScrollbarThickness(const FVector2D& NewScrollbarThickness);

	UMG_API FVector2D GetScrollbarThickness() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetScrollbarPadding(const FMargin& NewScrollbarPadding);

	UMG_API FMargin GetScrollbarPadding() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetAlwaysShowScrollbar(bool NewAlwaysShowScrollbar);
	
	UMG_API bool IsAlwaysShowScrollbar() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetAllowOverscroll(bool NewAllowOverscroll);

	UMG_API bool IsAllowOverscroll() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetAnimateWheelScrolling(bool bShouldAnimateWheelScrolling);

	UMG_API bool IsAnimateWheelScrolling() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetScrollAnimationInterpolationSpeed(float NewScrollAnimationInterpolationSpeed);

	UMG_API float GetScrollAnimationInterpolationSpeed() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetWheelScrollMultiplier(float NewWheelScrollMultiplier);

	UMG_API float GetWheelScrollMultiplier() const;

	UFUNCTION(Category = "Scroll")
	UMG_API void SetIsTouchScrollingEnabled(bool bInEnableTouchScrolling);

	UMG_API bool GetIsTouchScrollingEnabled() const;
	
	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API bool GetConsumePointerInput() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetConsumePointerInput(bool bInConsumePointerInput);

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetScrollWhenFocusChanges(EScrollWhenFocusChanges NewScrollWhenFocusChanges);

	UMG_API EScrollWhenFocusChanges GetScrollWhenFocusChanges() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetNavigationDestination(const EDescendantScrollDestination NewNavigationDestination);

	UMG_API EDescendantScrollDestination GetNavigationDestination() const;

	UMG_API void SetAlwaysShowScrollbarTrack(bool NewAlwaysShowScrollbarTrack);

	UMG_API bool IsAlwaysShowScrollbarTrack() const;

	UMG_API float GetNavigationScrollPadding() const;

	UMG_API void SetAllowRightClickDragScrolling(bool bShouldAllowRightClickDragScrolling);

	UMG_API bool IsAllowRightClickDragScrolling() const;

	UMG_API bool IsFrontPadScrolling() const;

	UMG_API bool IsBackPadScrolling() const;

	/** Instantly stops any inertial scrolling that is currently in progress */
	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void EndInertialScrolling();

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API FKey GetAnalogMouseWheelKey() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetAnalogMouseWheelKey(FKey InMouseWheelKey);

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API bool GetIsFocusable() const;

	UFUNCTION(BlueprintCallable, Category = "Scroll")
	UMG_API void SetIsFocusable(bool bInIsFocusable);

public:

	/** Called when the scroll has changed */
	UPROPERTY(BlueprintAssignable, Category = "Button|Event")
	FOnUserScrolledEvent OnUserScrolled;
	
	/** Called when the scrollbar visibility has changed */
	UPROPERTY(BlueprintAssignable, Category = "Button|Event")
	FOnScrollBarVisibilityChangedEvent OnScrollBarVisibilityChanged;

	/** Called when the scroll has changed */
	UPROPERTY(BlueprintAssignable, Category = "Scroll|Event")
	FOnScrollBoxFocusReceivedEvent OnFocusReceived;

	/** Called when the scroll has changed */
	UPROPERTY(BlueprintAssignable, Category = "Scroll|Event")
	FOnScrollBoxFocusLostEvent OnFocusLost;

	/** Called when the focus of the scrollbox has changed */
	UPROPERTY(BlueprintAssignable, Category = "Scroll|Event")
	FOnScrollBoxFocusUpdatedEvent OnFocusUpdated;

	/**
	 * Updates the scroll offset of the scrollbox.
	 * @param NewScrollOffset is in Slate Units.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetScrollOffset(float NewScrollOffset);
	
	/**
	 * Gets the scroll offset of the scrollbox in Slate Units.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API float GetScrollOffset() const;

	/**
	 * Gets the overscroll offset of the scrollbox in Slate Units.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API float GetOverscrollOffset() const;
	/**
	 * Gets the overscroll offset of the scrollbox in percentage from its view size.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API float GetOverscrollPercentage() const;

	/** Gets the scroll offset of the bottom of the ScrollBox in Slate Units. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API float GetScrollOffsetOfEnd() const;

	/** Gets the fraction currently visible in the scrollbox */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API float GetViewFraction() const;

	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API float GetViewOffsetFraction() const;

	/** Scrolls the ScrollBox to the top instantly */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void ScrollToStart();

	/** Scrolls the ScrollBox to the bottom instantly during the next layout pass. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void ScrollToEnd();

	/** Scrolls the ScrollBox to the widget during the next layout pass. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void ScrollWidgetIntoView(UWidget* WidgetToFind, bool AnimateScroll = true, EDescendantScrollDestination ScrollDestination = EDescendantScrollDestination::IntoView, float Padding = 0);

	/** Returns true when the widget is currently inertial scrolling */
	UFUNCTION(BlueprintPure, Category="Widget")
	UMG_API bool GetIsScrolling() const;

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	//~ Begin UObject Interface
#if WITH_EDITORONLY_DATA
	UMG_API virtual void Serialize(FArchive& Ar) override;
#endif // if WITH_EDITORONLY_DATA
	//~ End UObject Interface

#if WITH_EDITOR
	//~ Begin UWidget Interface
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual void OnDescendantSelectedByDesigner( UWidget* DescendantWidget ) override;
	UMG_API virtual void OnDescendantDeselectedByDesigner( UWidget* DescendantWidget ) override;
	//~ End UWidget Interface
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

	UMG_API void SlateHandleUserScrolled(float CurrentOffset);

	UMG_API void SlateHandleScrollBarVisibilityChanged(EVisibility NewVisibility);

	UMG_API void SlateHandleFocusReceived();
	
	UMG_API void SlateHandleFocusLost();

	UMG_API void SlateHandleFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath);

	// Initialize IsFocusable in the constructor before the SWidget is constructed.
	UMG_API void InitBackPadScrolling(bool InBackPadScrolling);
	// Initialize IsFocusable in the constructor before the SWidget is constructed.
	UMG_API void InitFrontPadScrolling(bool InFrontPadScrolling);
	// Initialize IsFocusable in the constructor before the SWidget is constructed.
	UMG_API void InitNavigationScrollPadding(float InNavigationScrollPadding);

protected:
	/** The desired scroll offset for the underlying scrollbox.  This is a cache so that it can be set before the widget is constructed. */
	float DesiredScrollOffset;

	TSharedPtr<class SScrollBox> MyScrollBox;

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

#if WITH_EDITOR
	FTSTicker::FDelegateHandle TickHandle;
#endif //WITH_EDITOR
};
