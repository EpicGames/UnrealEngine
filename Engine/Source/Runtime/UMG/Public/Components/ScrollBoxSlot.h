// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/SlateWrapperTypes.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Components/PanelSlot.h"

#include "ScrollBoxSlot.generated.h"

/** The Slot for the UScrollBox, contains the widget that are scrollable */
UCLASS(MinimalAPI)
class UScrollBoxSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

private:

	/** How much space this slot should occupy in the direction of the panel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Layout|ScrollBox Slot", meta = (AllowPrivateAccess = "true", DisplayAfter = "Padding"))
	FSlateChildSize Size;

public:
	
	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Layout|ScrollBox Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Layout|ScrollBox Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Layout|ScrollBox Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|ScrollBox Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API FSlateChildSize GetSize() const;

	UMG_API void SetSize(FSlateChildSize InSize);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|ScrollBox Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category = "Layout|ScrollBox Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

public:

	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SScrollBox> ScrollBox);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

private:

	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SScrollBox::FSlot* Slot;
};
