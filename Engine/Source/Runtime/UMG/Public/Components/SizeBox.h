// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"
#include "SizeBox.generated.h"

class SBox;

/**
 * A widget that allows you to specify the size it reports to have and desire.  Not all widgets report a desired size
 * that you actually desire.  Wrapping them in a SizeBox lets you have the Size Box force them to be a particular size.
 *
 * * Single Child
 * * Fixed Size
 */
UCLASS(MinimalAPI)
class USizeBox : public UContentWidget
{
	GENERATED_UCLASS_BODY()

protected:
	TSharedPtr<SBox> MySizeBox;

public:

	UE_DEPRECATED(5.1, "Direct access to WidthOverride is deprecated. Please use the getter or setter.")
	/** When specified, ignore the content's desired size and report the WidthOverride as the Box's desired width. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetWidthOverride", Category="Child Layout", meta=( editcondition="bOverride_WidthOverride" ))
	float WidthOverride;

	UE_DEPRECATED(5.1, "Direct access to HeightOverride is deprecated. Please use the getter or setter.")
	/** When specified, ignore the content's desired size and report the HeightOverride as the Box's desired height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetHeightOverride", Category="Child Layout", meta=( editcondition="bOverride_HeightOverride" ))
	float HeightOverride;

	UE_DEPRECATED(5.1, "Direct access to MinDesiredWidth is deprecated. Please use the getter or setter.")
	/** When specified, will report the MinDesiredWidth if larger than the content's desired width. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMinDesiredWidth", Category="Child Layout", meta=( editcondition="bOverride_MinDesiredWidth" ))
	float MinDesiredWidth;

	UE_DEPRECATED(5.1, "Direct access to MinDesiredHeight is deprecated. Please use the getter or setter.")
	/** When specified, will report the MinDesiredHeight if larger than the content's desired height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMinDesiredHeight", Category="Child Layout", meta=( editcondition="bOverride_MinDesiredHeight" ))
	float MinDesiredHeight;

	UE_DEPRECATED(5.1, "Direct access to MaxDesiredWidth is deprecated. Please use the getter or setter.")
	/** When specified, will report the MaxDesiredWidth if smaller than the content's desired width. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMaxDesiredWidth", Category="Child Layout", meta=( editcondition="bOverride_MaxDesiredWidth" ))
	float MaxDesiredWidth;

	UE_DEPRECATED(5.1, "Direct access to MaxDesiredHeight is deprecated. Please use the getter or setter.")
	/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMaxDesiredHeight", Category="Child Layout", meta=( editcondition="bOverride_MaxDesiredHeight" ))
	float MaxDesiredHeight;

	UE_DEPRECATED(5.1, "Direct access to MinAspectRatio is deprecated. Please use the getter or setter.")
	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMinAspectRatio", Category = "Child Layout", meta = (editcondition = "bOverride_MinAspectRatio"))
	float MinAspectRatio;

	UE_DEPRECATED(5.1, "Direct access to MaxAspectRatio is deprecated. Please use the getter or setter.")
	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMaxAspectRatio", Category="Child Layout", meta=( editcondition="bOverride_MaxAspectRatio" ))
	float MaxAspectRatio;

	UE_DEPRECATED(5.1, "Direct access to bOverride_WidthOverride is deprecated. Please use the SetWidthOverride or ClearWidthOverride.")
	/**  */
	UPROPERTY(EditAnywhere, Category="Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_WidthOverride : 1;

	UE_DEPRECATED(5.1, "Direct access to bOverride_HeightOverride is deprecated. Please use the SetHeightOverride or ClearHeightOverride.")
	/**  */
	UPROPERTY(EditAnywhere, Category="Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_HeightOverride : 1;

	UE_DEPRECATED(5.1, "Direct access to bOverride_MinDesiredWidth is deprecated. Please use the SetMinDesiredWidth or ClearMinDesiredWidth.")
	/**  */
	UPROPERTY(EditAnywhere, Category="Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_MinDesiredWidth : 1;

	UE_DEPRECATED(5.1, "Direct access to bOverride_MinDesiredHeight is deprecated. Please use the SetMinDesiredHeight or ClearMinDesiredHeight.")
	/**  */
	UPROPERTY(EditAnywhere, Category="Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_MinDesiredHeight : 1;

	UE_DEPRECATED(5.1, "Direct access to bOverride_MaxDesiredWidth is deprecated. Please use the SetMaxDesiredWidth or ClearMaxDesiredWidth.")
	/**  */
	UPROPERTY(EditAnywhere, Category="Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_MaxDesiredWidth : 1;

	UE_DEPRECATED(5.1, "Direct access to bOverride_MaxDesiredHeight is deprecated. Please use the SetMaxDesiredHeight or ClearMaxDesiredHeight.")
	/**  */
	UPROPERTY(EditAnywhere, Category="Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_MaxDesiredHeight : 1;

	UE_DEPRECATED(5.1, "Direct access to bOverride_MinAspectRatio is deprecated. Please use the SetMinAspectRatio or ClearMinAspectRatio.")
	/**  */
	UPROPERTY(EditAnywhere, Category="Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_MinAspectRatio : 1;

	UE_DEPRECATED(5.1, "Direct access to bOverride_MaxAspectRatio is deprecated. Please use the SetMaxAspectRatio  or ClearMaxAspectRatio .")
	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta=(InlineEditConditionToggle))
	uint8 bOverride_MaxAspectRatio : 1;

	
public:

	/** */
	UMG_API float GetWidthOverride() const;

	/** */
	UMG_API bool IsWidthOverride() const;

	/** When specified, ignore the content's desired size and report the WidthOverride as the Box's desired width. */
	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetWidthOverride(float InWidthOverride);

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void ClearWidthOverride();

	/** */
	UMG_API float GetHeightOverride() const;

	/** */
	UMG_API bool IsHeightOverride() const;

	/** When specified, ignore the content's desired size and report the HeightOverride as the Box's desired height. */
	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetHeightOverride(float InHeightOverride);

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void ClearHeightOverride();

	/** */
	UMG_API float GetMinDesiredWidth() const;

	/** */
	UMG_API bool IsMinDesiredWidthOverride() const;

	/** When specified, will report the MinDesiredWidth if larger than the content's desired width. */
	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetMinDesiredWidth(float InMinDesiredWidth);

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void ClearMinDesiredWidth();

	/** */
	UMG_API float GetMinDesiredHeight() const;

	/** */
	UMG_API bool IsMinDesiredHeightOverride() const;

	/** When specified, will report the MinDesiredHeight if larger than the content's desired height. */
	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetMinDesiredHeight(float InMinDesiredHeight);

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void ClearMinDesiredHeight();

	/** */
	UMG_API float GetMaxDesiredWidth() const;

	/** */
	UMG_API bool IsMaxDesiredWidthOverride() const;

	/** When specified, will report the MaxDesiredWidth if smaller than the content's desired width. */
	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetMaxDesiredWidth(float InMaxDesiredWidth);

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void ClearMaxDesiredWidth();

	/** */
	UMG_API float GetMaxDesiredHeight() const;

	/** */
	UMG_API bool IsMaxDesiredHeightOverride() const;

	/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height. */
	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetMaxDesiredHeight(float InMaxDesiredHeight);

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void ClearMaxDesiredHeight();

	/** */
	UMG_API float GetMinAspectRatio() const;

	/** */
	UMG_API bool IsMinAspectRatioOverride() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetMinAspectRatio(float InMinAspectRatio);

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void ClearMinAspectRatio();

	/** */
	UMG_API float GetMaxAspectRatio() const;

	/** */
	UMG_API bool IsMaxAspectRatioOverride() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Size Box")
	UMG_API void SetMaxAspectRatio(float InMaxAspectRatio);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearMaxAspectRatio();


	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
