// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Extensions/UIComponent.h"
#include "Layout/Margin.h"
#include "UObject/ObjectMacros.h"

#include "SizeBoxComponent.generated.h"

class SBox;

/** This is a class for a Component that wraps the Owner widget with a Size Box. */
UCLASS(MinimalAPI, Experimental)
class USizeBoxComponent : public UUIComponent
{
	GENERATED_BODY()

public:

	/** */
	UMG_API float GetWidthOverride() const;

	/** */
	UMG_API bool IsWidthOverride() const;

	/** When specified, ignore the content's desired size and report the WidthOverride as the Box's desired width. */
	UMG_API void SetWidthOverride(float InWidthOverride);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearWidthOverride();

	/** */
	UMG_API float GetHeightOverride() const;

	/** */
	UMG_API bool IsHeightOverride() const;

	/** When specified, ignore the content's desired size and report the HeightOverride as the Box's desired height. */
	UMG_API void SetHeightOverride(float InHeightOverride);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearHeightOverride();

	/** */
	UMG_API float GetMinDesiredWidth() const;

	/** */
	UMG_API bool IsMinDesiredWidthOverride() const;

	/** When specified, will report the MinDesiredWidth if larger than the content's desired width. */
	UMG_API void SetMinDesiredWidth(float InMinDesiredWidth);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearMinDesiredWidth();

	/** */
	UMG_API float GetMinDesiredHeight() const;

	/** */
	UMG_API bool IsMinDesiredHeightOverride() const;

	/** When specified, will report the MinDesiredHeight if larger than the content's desired height. */
	UMG_API void SetMinDesiredHeight(float InMinDesiredHeight);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearMinDesiredHeight();

	/** */
	UMG_API float GetMaxDesiredWidth() const;

	/** */
	UMG_API bool IsMaxDesiredWidthOverride() const;

	/** When specified, will report the MaxDesiredWidth if smaller than the content's desired width. */
	UMG_API void SetMaxDesiredWidth(float InMaxDesiredWidth);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearMaxDesiredWidth();

	/** */
	UMG_API float GetMaxDesiredHeight() const;

	/** */
	UMG_API bool IsMaxDesiredHeightOverride() const;

	/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height. */
	UMG_API void SetMaxDesiredHeight(float InMaxDesiredHeight);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearMaxDesiredHeight();

	/** */
	UMG_API float GetMinAspectRatio() const;

	/** */
	UMG_API bool IsMinAspectRatioOverride() const;

	UMG_API void SetMinAspectRatio(float InMinAspectRatio);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearMinAspectRatio();

	/** */
	UMG_API float GetMaxAspectRatio() const;

	/** */
	UMG_API bool IsMaxAspectRatioOverride() const;

	UMG_API void SetMaxAspectRatio(float InMaxAspectRatio);

	UFUNCTION(BlueprintCallable, Category = "Layout|Size Box")
	UMG_API void ClearMaxAspectRatio();

	// Child slot functions:

	UMG_API FMargin GetPadding() const;

	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

#if WITH_EDITOR
	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif // WITH_EDITOR
	
	virtual TSharedRef<SWidget> RebuildWidgetWithContent(const TSharedRef<SWidget> OwnerContent) override;

private:
		
	void SynchronizeProperties(TSharedRef<SBox> SizeBox);
	
	// Child slot properties:

	/** This property is for the Widget that owns this Component. The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Size Box Slot", meta = (AllowPrivateAccess = true))
	FMargin Padding = FMargin(0.f, 0.f);;

	/** This property is for the Widget that owns this Component. The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Size Box Slot", meta = (AllowPrivateAccess = true))
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = HAlign_Fill;;

	/** This property is for the Widget that owns this Component. The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Size Box Slot", meta = (AllowPrivateAccess = true))
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = VAlign_Fill;

	// Size Box Properties:

	/** When specified, ignore the content's desired size and report the WidthOverride as the Box's desired width. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_WidthOverride", AllowPrivateAccess = true))
	float WidthOverride = 0.f;

	/** When specified, ignore the content's desired size and report the HeightOverride as the Box's desired height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_HeightOverride", AllowPrivateAccess = true))
	float HeightOverride = 0.f;

	/** When specified, will report the MinDesiredWidth if larger than the content's desired width. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_MinDesiredWidth", AllowPrivateAccess = true))
	float MinDesiredWidth = 0.f;

	/** When specified, will report the MinDesiredHeight if larger than the content's desired height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_MinDesiredHeight", AllowPrivateAccess = true))
	float MinDesiredHeight = 0.f;

	/** When specified, will report the MaxDesiredWidth if smaller than the content's desired width. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_MaxDesiredWidth", AllowPrivateAccess = true))
	float MaxDesiredWidth = 0.f;

	/** When specified, will report the MaxDesiredHeight if smaller than the content's desired height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_MaxDesiredHeight", AllowPrivateAccess = true))
	float MaxDesiredHeight = 0.f;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_MinAspectRatio", AllowPrivateAccess = true))
	float MinAspectRatio = 1.f;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Child Layout", meta = (editcondition = "bOverride_MaxAspectRatio", AllowPrivateAccess = true))
	float MaxAspectRatio = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_WidthOverride : 1;	

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_HeightOverride : 1;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_MinDesiredWidth : 1;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_MinDesiredHeight : 1;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_MaxDesiredWidth : 1;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_MaxDesiredHeight : 1;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_MinAspectRatio : 1;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Child Layout", meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	uint8 bOverride_MaxAspectRatio : 1;

	TWeakPtr<SBox> MySizeBox;
};