// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/PanelSlot.h"

#include "SizeBoxSlot.generated.h"

class SBox;

/** The Slot for the USizeBoxSlot, contains the widget displayed in a button's single slot */
UCLASS(MinimalAPI)
class USizeBoxSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetPadding", Category="Layout|SizeBox Slot")
	FMargin Padding;

private:
	/** A pointer to the button to allow us to adjust the size, padding...etc at runtime. */
	TWeakPtr<SBox> SizeBox;

public:
	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|SizeBox Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|SizeBox Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|SizeBox Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|SizeBox Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|SizeBox Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

public:

	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying slot for the slate SizeBox. */
	UMG_API void BuildSlot(TSharedRef<SBox> InSizeBox);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
};
