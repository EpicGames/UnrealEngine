// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#include "SlateWrapperTypes.generated.h"


#define BIND_UOBJECT_ATTRIBUTE(Type, Function) \
	TAttribute<Type>::Create( TAttribute<Type>::FGetter::CreateUObject( this, &ThisClass::Function ) )

#define BIND_UOBJECT_DELEGATE(Type, Function) \
	Type::CreateUObject( this, &ThisClass::Function )

/** Is an entity visible? */
UENUM(BlueprintType)
enum class ESlateVisibility : uint8
{
	/** Visible and hit-testable (can interact with cursor). Default value. */
	Visible,
	/** Not visible and takes up no space in the layout (obviously not hit-testable). */
	Collapsed,
	/** Not visible but occupies layout space (obviously not hit-testable). */
	Hidden,
	/** Visible but not hit-testable (cannot interact with cursor) and children in the hierarchy (if any) are also not hit-testable. */
	HitTestInvisible UMETA(DisplayName = "Not Hit-Testable (Self & All Children)"),
	/** Visible but not hit-testable (cannot interact with cursor) and doesn't affect hit-testing on children (if any). */
	SelfHitTestInvisible UMETA(DisplayName = "Not Hit-Testable (Self Only)")
};

/** Whether a widget should be included in accessibility, and if so, how its text should be retrieved. */
UENUM(BlueprintType)
enum class ESlateAccessibleBehavior : uint8
{
	/** Not accessible. */
	NotAccessible,
	/**
	 * Accessible, first checking to see if there's any custom default text assigned for widgets of this type.
	 * If not, then it will attempt to use the alternate behavior (ie AccessibleSummaryBehavior instead of AccessibleBehavior)
	 * and return that value instead. This acts as a reference so that you only need to set one value for both of them
	 * to return the same thing.
	 */
	Auto,
	/** Accessible, and traverse all child widgets and concat their AccessibleSummaryText together. */
	Summary,
	/** Accessible, and retrieve manually-assigned text from a TAttribute. */
	Custom,
	/** Accessible, and use the tooltip's accessible text. */
	ToolTip
};

/**
 * A container for all accessible properties for a UWidget that will be passed to the underlying SWidget.
 * Any property here must also be added to UWidget.h and synchronized. See UWidget for more information.
 */
UCLASS(DefaultToInstanced)
class USlateAccessibleWidgetData : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_DELEGATE_RetVal(FText, FGetText);

	USlateAccessibleWidgetData()
	{
		AccessibleBehavior = ESlateAccessibleBehavior::NotAccessible;
		AccessibleSummaryBehavior = ESlateAccessibleBehavior::Auto;
		bCanChildrenBeAccessible = true;
	}

	TAttribute<FText> CreateAccessibleTextAttribute()
	{
		return AccessibleTextDelegate.IsBound() ?
			TAttribute<FText>::Create(AccessibleTextDelegate.GetUObject(), AccessibleTextDelegate.GetFunctionName()) :
			TAttribute<FText>(AccessibleText);
	}

	TAttribute<FText> CreateAccessibleSummaryTextAttribute()
	{
		return AccessibleSummaryTextDelegate.IsBound() ?
			TAttribute<FText>::Create(AccessibleSummaryTextDelegate.GetUObject(), AccessibleSummaryTextDelegate.GetFunctionName()) :
			TAttribute<FText>(AccessibleSummaryText);
	}

	UPROPERTY()
	bool bCanChildrenBeAccessible;

	UPROPERTY()
	ESlateAccessibleBehavior AccessibleBehavior;

	UPROPERTY()
	ESlateAccessibleBehavior AccessibleSummaryBehavior;

	UPROPERTY()
	FText AccessibleText;

	UPROPERTY()
	FGetText AccessibleTextDelegate;

	UPROPERTY()
	FText AccessibleSummaryText;

	UPROPERTY()
	FGetText AccessibleSummaryTextDelegate;
};

/** The sizing options of UWidgets */
UENUM(BlueprintType)
namespace ESlateSizeRule
{
	enum Type : int
	{
		/** Only requests as much room as it needs based on the widgets desired size. */
		Automatic,
		/** Greedily attempts to fill all available room based on the percentage value 0..1 */
		Fill
	};
}

/**
 * Allows users to handle events and return information to the underlying UI layer.
 */
USTRUCT(BlueprintType)
struct FEventReply
{
	GENERATED_USTRUCT_BODY()

public:

	FEventReply(bool IsHandled = false)
	: NativeReply(IsHandled ? FReply::Handled() : FReply::Unhandled())
	{
	}

	bool operator==(const FEventReply& Other) const { return NativeReply.IsEventHandled() == Other.NativeReply.IsEventHandled(); }

	FReply NativeReply;
};

template<>
struct TStructOpsTypeTraits<FEventReply> : public TStructOpsTypeTraitsBase2<FEventReply>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

/** A struct exposing size param related properties to UMG. */
USTRUCT(BlueprintType)
struct FSlateChildSize
{
	GENERATED_USTRUCT_BODY()

	/** The parameter of the size rule. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, meta=( UIMin="0", UIMax="1" ))
	float Value;

	/** The sizing rule of the content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	TEnumAsByte<ESlateSizeRule::Type> SizeRule;

	FSlateChildSize()
		: Value(1.0f)
		, SizeRule(ESlateSizeRule::Fill)
	{
	}

	FSlateChildSize(ESlateSizeRule::Type InSizeRule)
		: Value(1.0f)
		, SizeRule(InSizeRule)
	{
	}
};


UENUM( BlueprintType )
namespace EVirtualKeyboardType
{
	enum Type : int
	{
		Default,
		Number,
		Web,
		Email,
		Password,
		AlphaNumeric
	};
}

namespace EVirtualKeyboardType
{
	inline EKeyboardType AsKeyboardType( Type InType )
	{
		switch ( InType )
		{
		case Type::Default:
			return EKeyboardType::Keyboard_Default;
		case Type::Number:
			return EKeyboardType::Keyboard_Number;
		case Type::Web:
			return EKeyboardType::Keyboard_Web;
		case Type::Email:
			return EKeyboardType::Keyboard_Email;
		case Type::Password:
			return EKeyboardType::Keyboard_Password;
		case Type::AlphaNumeric:
			return EKeyboardType::Keyboard_AlphaNumeric;
		}

		return EKeyboardType::Keyboard_Default;
	}
}
