// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Slider.generated.h"

class SSlider;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnMouseCaptureBeginEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnMouseCaptureEndEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnControllerCaptureBeginEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnControllerCaptureEndEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnFloatValueChangedEvent, float, Value);

/**
 * A simple widget that shows a sliding bar with a handle that allows you to control the value in a user define range (between 0..1 by default).
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class USlider : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.1, "Direct access to Value is deprecated. Please use the getter or setter.")
	/** The volume value to display. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, FieldNotify, BlueprintGetter="GetValue", BlueprintSetter="SetValue", Category="Appearance", meta=(UIMin="0", UIMax="1"))
	float Value;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueDelegate;

	UE_DEPRECATED(5.1, "Direct access to MinValue is deprecated. Please use the getter or setter.")
	/** The minimum value the slider can be set to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMinValue", Category="Appearance")
	float MinValue;

	UE_DEPRECATED(5.1, "Direct access to MaxValue is deprecated. Please use the getter or setter.")
	/** The maximum value the slider can be set to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMaxValue", Category="Appearance")
	float MaxValue;

public:
	UE_DEPRECATED(5.1, "Direct access to WidgetStyle is deprecated. Please use the getter or setter.")
	/** The progress bar style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Style", meta=( DisplayName="Style" ))
	FSliderStyle WidgetStyle;

	UE_DEPRECATED(5.1, "Direct access to Orientation is deprecated. Please use the getter or setter.")
	/** The slider's orientation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Appearance)
	TEnumAsByte<EOrientation> Orientation;

	UE_DEPRECATED(5.1, "Direct access to SliderBarColor is deprecated. Please use the getter or setter.")
	/** The color to draw the slider bar in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetSliderBarColor", Category="Appearance")
	FLinearColor SliderBarColor;

	UE_DEPRECATED(5.1, "Direct access to SliderHandleColor is deprecated. Please use the getter or setter.")
	/** The color to draw the slider handle in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetSliderHandleColor", Category="Appearance")
	FLinearColor SliderHandleColor;

	UE_DEPRECATED(5.1, "Direct access to IndentHandle is deprecated. Please use the getter or setter.")
	/** Whether the slidable area should be indented to fit the handle. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="HasIndentHandle", Setter, BlueprintSetter="SetIndentHandle", Category="Appearance", AdvancedDisplay)
	bool IndentHandle;

	UE_DEPRECATED(5.1, "Direct access to Locked is deprecated. Please use the getter or setter.")
	/** Whether the handle is interactive or fixed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="IsLocked", Setter, BlueprintSetter="SetLocked", Category="Appearance", AdvancedDisplay)
	bool Locked;

	/** Sets new value if mouse position is greater/less than half the step size. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, AdvancedDisplay)
	bool MouseUsesStep;

	/** Sets whether we have to lock input to change the slider value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, AdvancedDisplay)
	bool RequiresControllerLock;

	UE_DEPRECATED(5.1, "Direct access to StepSize is deprecated. Please use the getter or setter.")
	/** The amount to adjust the value by, when using a controller or keyboard */
	UPROPERTY(EditAnywhere,BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetStepSize", Category="Appearance", meta=(UIMin="0", UIMax="1"))
	float StepSize;

	/** Should the slider be focusable? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Interaction")
	bool IsFocusable;

	/** If refresh requests for the viewport should happen for all value changes **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Interaction")
	bool bPreventThrottling = false;

public:

	/** Invoked when the mouse is pressed and a capture begins. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Invoked when the mouse is released and a capture ends. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureEndEvent OnMouseCaptureEnd;

	/** Invoked when the controller capture begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureBeginEvent OnControllerCaptureBegin;

	/** Invoked when the controller capture ends. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureEndEvent OnControllerCaptureEnd;

	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnFloatValueChangedEvent OnValueChanged;

	/** Gets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API float GetValue() const;

	/** Get the current value scaled from 0 to 1 */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API float GetNormalizedValue() const;

	/** Sets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API void SetValue(float InValue);

	/** Gets the minimum value of the slider. */
	UMG_API float GetMinValue() const;

	/** Sets the minimum value of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void SetMinValue(float InValue);

	/** Gets the maximum value of the slider. */
	UMG_API float GetMaxValue() const;

	/** Sets the maximum value of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void SetMaxValue(float InValue);

	/** Get the style used by the widget. */
	UMG_API const FSliderStyle& GetWidgetStyle() const;

	/** Set the style used by the widget. */
	UMG_API void SetWidgetStyle(const FSliderStyle& InStyle);

	/** Getg the slider's orientation. */
	UMG_API EOrientation GetOrientation() const;

	/** Sets the slider's orientation. */
	UMG_API void SetOrientation(EOrientation InOrientation);

	/** Gets if the slidable area should be indented to fit the handle. */
	UMG_API bool HasIndentHandle() const;

	/** Sets if the slidable area should be indented to fit the handle. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API void SetIndentHandle(bool InValue);

	/** Returns true when the handle is fixed. */
	UMG_API bool IsLocked() const;

	/** Sets the handle to be interactive or fixed. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API void SetLocked(bool InValue);

	/** Gets the amount to adjust the value by. */
	UMG_API float GetStepSize() const;

	/** Sets the amount to adjust the value by, when using a controller or keyboard. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API void SetStepSize(float InValue);

	/** Gets the color of the slider bar. */
	UMG_API FLinearColor GetSliderBarColor() const;

	/** Sets the color of the slider bar. */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetSliderBarColor(FLinearColor InValue);

	/** Gets the color of the handle bar */
	UMG_API FLinearColor GetSliderHandleColor() const;

	/** Sets the color of the handle bar */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetSliderHandleColor(FLinearColor InValue);
	
	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	/** Native Slate Widget */
	TSharedPtr<SSlider> MySlider;

	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	UMG_API void HandleOnValueChanged(float InValue);
	UMG_API void HandleOnMouseCaptureBegin();
	UMG_API void HandleOnMouseCaptureEnd();
	UMG_API void HandleOnControllerCaptureBegin();
	UMG_API void HandleOnControllerCaptureEnd();

#if WITH_ACCESSIBILITY
	UMG_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const override;
#endif

	PROPERTY_BINDING_IMPLEMENTATION(float, Value);
};
