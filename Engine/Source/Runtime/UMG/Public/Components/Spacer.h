// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Spacer.generated.h"

class SSpacer;

/**
 * A spacer widget; it does not have a visual representation, and just provides padding between other widgets.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class USpacer : public UWidget
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.1, "Direct access to Size is deprecated. Please use the getter or setter.")
	/** The size of the spacer */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetSize", Category="Appearance")
	FVector2D Size;

public:

	/** */
	UMG_API FVector2D GetSize() const;

	/** Sets the size of the spacer */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetSize(FVector2D InSize);
	
	//~ UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End of UVisual interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	//~ UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End of UWidget interface

protected:
	TSharedPtr<SSpacer> MySpacer;
};
