// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SSpinBox.h"
#include "Components/Widget.h"
#include "SpinBox.generated.h"

/**
 * A numerical entry box that allows for direct entry of the number or allows the user to click and slide the number.
 */
UCLASS(MinimalAPI)
class USpinBox : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSpinBoxValueChangedEvent, float, InValue);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnSpinBoxValueCommittedEvent, float, InValue, ETextCommit::Type, CommitMethod);
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnSpinBoxBeginSliderMovement);

public:

	UE_DEPRECATED(5.2, "Direct access to Value is deprecated. Please use the getter or setter.")
	/** Value stored in this spin box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetValue", BlueprintGetter = "GetValue", FieldNotify, Category = Content)
	float Value;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueDelegate;

public:
	UE_DEPRECATED(5.2, "Direct access to WidgetStyle is deprecated. Please use the getter or setter.")
	/** The Style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Style", meta=( DisplayName="Style" ))
	FSpinBoxStyle WidgetStyle;

	UE_DEPRECATED(5.2, "Direct access to MinFractionalDigits is deprecated. Please use the getter or setter.")
	/** The minimum required fractional digits - default 1 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetMinFractionalDigits, BlueprintGetter = GetMinFractionalDigits, Category = "Slider", meta = (ClampMin = 0, UIMin = 0))
	int32 MinFractionalDigits;

	UE_DEPRECATED(5.2, "Direct access to MaxFractionalDigits is deprecated. Please use the getter or setter.")
	/** The maximum required fractional digits - default 6 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetMaxFractionalDigits, BlueprintGetter = GetMaxFractionalDigits, Category = "Slider", meta = (ClampMin = 0, UIMin = 0))
	int32 MaxFractionalDigits;

	UE_DEPRECATED(5.2, "Direct access to bAlwaysUsesDeltaSnap is deprecated. Please use the getter or setter.")
	/** Whether this spin box should use the delta snapping logic for typed values - default false */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = GetAlwaysUsesDeltaSnap, Setter = SetAlwaysUsesDeltaSnap, BlueprintSetter = SetAlwaysUsesDeltaSnap, BlueprintGetter = GetAlwaysUsesDeltaSnap, Category = "Slider")
	bool bAlwaysUsesDeltaSnap;

	UE_DEPRECATED(5.2, "Direct access to bEnableSlider is deprecated. Please use the getter or setter.")
	/** Whether this spin box should have slider feature enabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetEnableSlider", Setter = "SetEnableSlider", Category = "Slider")
	bool bEnableSlider = true;

	UE_DEPRECATED(5.2, "Direct access to Delta is deprecated. Please use the getter or setter.")
	/** The amount by which to change the spin box value as the slider moves. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = SetDelta, BlueprintGetter = GetDelta, Category = "Slider")
	float Delta;

	UE_DEPRECATED(5.2, "Direct access to SliderExponent is deprecated. Please use the getter or setter.")
	/** The exponent by which to increase the delta as the mouse moves. 1 is constant (never increases the delta). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Slider")
	float SliderExponent;
	
	UE_DEPRECATED(5.2, "Direct access to Font is deprecated. Please use the getter or setter.")
	/** Font color and opacity (overrides style) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Display")
	FSlateFontInfo Font;

	UE_DEPRECATED(5.2, "Direct access to Justification is deprecated. Please use the getter or setter.")
	/** The justification the value text should appear as. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Display")
	TEnumAsByte<ETextJustify::Type> Justification;

	UE_DEPRECATED(5.2, "Direct access to MinDesiredWidth is deprecated. Please use the getter or setter.")
	/** The minimum width of the spin box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Display", AdvancedDisplay, DisplayName = "Minimum Desired Width")
	float MinDesiredWidth;
	
	/** If we're on a platform that requires a virtual keyboard, what kind of keyboard should this widget use? */
	UPROPERTY(EditAnywhere, Category = "Input", AdvancedDisplay)
	TEnumAsByte<EVirtualKeyboardType::Type> KeyboardType;

	/** What action should be taken when the virtual keyboard is dismissed? */
	UPROPERTY(EditAnywhere, Category = Behavior, AdvancedDisplay)
	EVirtualKeyboardDismissAction VirtualKeyboardDismissAction;

	UE_DEPRECATED(5.2, "Direct access to ClearKeyboardFocusOnCommit is deprecated. Please use the getter or setter.")
	/** Whether to remove the keyboard focus from the spin box when the value is committed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Input", AdvancedDisplay)
	bool ClearKeyboardFocusOnCommit;

	UE_DEPRECATED(5.2, "Direct access to SelectAllTextOnCommit is deprecated. Please use the getter or setter.")
	/** Whether to select the text in the spin box when the value is committed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Input", AdvancedDisplay)
	bool SelectAllTextOnCommit;

	UE_DEPRECATED(5.2, "Direct access to ForegroundColor is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetForegroundColor", Category = "Style")
	FSlateColor ForegroundColor;

public:
	/** Called when the value is changed interactively by the user */
	UPROPERTY(BlueprintAssignable, Category="SpinBox|Events")
	FOnSpinBoxValueChangedEvent OnValueChanged;

	/** Called when the value is committed. Occurs when the user presses Enter or the text box loses focus. */
	UPROPERTY(BlueprintAssignable, Category="SpinBox|Events")
	FOnSpinBoxValueCommittedEvent OnValueCommitted;

	/** Called right before the slider begins to move */
	UPROPERTY(BlueprintAssignable, Category="SpinBox|Events")
	FOnSpinBoxBeginSliderMovement OnBeginSliderMovement;

	/** Called right after the slider handle is released by the user */
	UPROPERTY(BlueprintAssignable, Category="SpinBox|Events")
	FOnSpinBoxValueChangedEvent OnEndSliderMovement;

public:

	/** Get the current value of the spin box. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API float GetValue() const;

	/** Set the value of the spin box. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API void SetValue(float NewValue);

public:

	/** Set the style for the spin box. */
	UMG_API const FSpinBoxStyle& GetWidgetStyle() const;

	/** Get the style for the spin box. */
	UMG_API void SetWidgetStyle(const FSpinBoxStyle& InWidgetStyle);

	/** Get the current Min Fractional Digits for the spin box. */
	UFUNCTION(BlueprintCallable, BlueprintGetter, Category = "Behavior")
	UMG_API int32 GetMinFractionalDigits() const;

	/** Set the Min Fractional Digits for the spin box. */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Behavior")
	UMG_API void SetMinFractionalDigits(int32 NewValue);

	/** Get the current Max Fractional Digits for the spin box. */
	UFUNCTION(BlueprintCallable, BlueprintGetter, Category = "Behavior")
	UMG_API int32 GetMaxFractionalDigits() const;

	/** Set the Max Fractional Digits for the spin box. */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Behavior")
	UMG_API void SetMaxFractionalDigits(int32 NewValue);

	/** Get whether the spin box uses delta snap on type. */
	UFUNCTION(BlueprintCallable, BlueprintGetter, Category = "Behavior")
	UMG_API bool GetAlwaysUsesDeltaSnap() const;

	/** Set whether the spin box uses delta snap on type. */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Behavior")
	UMG_API void SetAlwaysUsesDeltaSnap(bool bNewValue);

	/** Get whether the spin box uses the slider feature. */
	UMG_API bool GetEnableSlider() const;

	/** Set whether the spin box uses the slider feature. */
	UMG_API void SetEnableSlider(bool bNewValue);

	/** Get the current delta for the spin box. */
	UFUNCTION(BlueprintCallable, BlueprintGetter, Category = "Behavior")
	UMG_API float GetDelta() const;

	/** Set the delta for the spin box. */
	UFUNCTION(BlueprintCallable, BlueprintSetter, Category = "Behavior")
	UMG_API void SetDelta(float NewValue);

	/** Get the current slider exponent for the spin box. */
	UMG_API float GetSliderExponent() const;

	/** Set the slider exponent for the spin box. */
	UMG_API void SetSliderExponent(float NewValue);

	/**  Get the font color and opacity that overrides the style font. */
	UMG_API const FSlateFontInfo& GetFont() const;

	/** Set the font color and opacity that overrides the style font. */
	UMG_API void SetFont(const FSlateFontInfo& InFont);

	/** Get the justification for value text. */
	UMG_API const ETextJustify::Type GetJustification() const;

	/** Set the justification for value text. */
	UMG_API void SetJustification(ETextJustify::Type InJustification);

	/** Get the minimum width of the spin box. */
	UMG_API float GetMinDesiredWidth() const;

	/** Set the minimum width of the spin box. */
	UMG_API void SetMinDesiredWidth(float NewValue);

	/** Get whether the keyboard focus is removed from the spin box when the value is committed. */
	UMG_API bool GetClearKeyboardFocusOnCommit() const;

	/** Set whether the keyboard focus is removed from the spin box when the value is committed. */
	UMG_API void SetClearKeyboardFocusOnCommit(bool bNewValue);

	/** Get whether to select the text in the spin box when the value is committed. */
	UMG_API bool GetSelectAllTextOnCommit() const;

	/** Set whether to select the text in the spin box when the value is committed. */
	UMG_API void SetSelectAllTextOnCommit(bool bNewValue);

	/** Get the current minimum value that can be manually set in the spin box. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	UMG_API float GetMinValue() const;

	/** Set the minimum value that can be manually set in the spin box. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void SetMinValue(float NewValue);

	/** Clear the minimum value that can be manually set in the spin box. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void ClearMinValue();

	/** Get the current maximum value that can be manually set in the spin box. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API float GetMaxValue() const;

	/** Set the maximum value that can be manually set in the spin box. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void SetMaxValue(float NewValue);

	/** Clear the maximum value that can be manually set in the spin box. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void ClearMaxValue();

	/** Get the current minimum value that can be specified using the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API float GetMinSliderValue() const;

	/** Set the minimum value that can be specified using the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void SetMinSliderValue(float NewValue);

	/** Clear the minimum value that can be specified using the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void ClearMinSliderValue();

	/** Get the current maximum value that can be specified using the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API float GetMaxSliderValue() const;

	/** Set the maximum value that can be specified using the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void SetMaxSliderValue(float NewValue);

	/** Clear the maximum value that can be specified using the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	UMG_API void ClearMaxSliderValue();

	/**  */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetForegroundColor(FSlateColor InForegroundColor);

	/** Get the foreground color of the spin box. */
	UMG_API FSlateColor GetForegroundColor() const;

public:

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif
	//~ End UWidget Interface

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UMG_API void HandleOnValueChanged(float InValue);
	UMG_API void HandleOnValueCommitted(float InValue, ETextCommit::Type CommitMethod);
	UMG_API void HandleOnBeginSliderMovement();
	UMG_API void HandleOnEndSliderMovement(float InValue);

protected:
	/** Whether the optional MinValue attribute of the widget is set */
	UPROPERTY(EditAnywhere, Category = Content, meta = (InlineEditConditionToggle))
	uint32 bOverride_MinValue : 1;

	/** Whether the optional MaxValue attribute of the widget is set */
	UPROPERTY(EditAnywhere, Category = Content, meta=(InlineEditConditionToggle))
	uint32 bOverride_MaxValue : 1;

	/** Whether the optional MinSliderValue attribute of the widget is set */
	UPROPERTY(EditAnywhere, Category = Content, meta = (InlineEditConditionToggle))
	uint32 bOverride_MinSliderValue : 1;

	/** Whether the optional MaxSliderValue attribute of the widget is set */
	UPROPERTY(EditAnywhere, Category = Content, meta=(InlineEditConditionToggle))
	uint32 bOverride_MaxSliderValue : 1;

	UE_DEPRECATED(5.2, "Direct access to MinValue is deprecated. Please use the getter or setter.")
	/** The minimum allowable value that can be manually entered into the spin box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetMinValue", BlueprintGetter = "GetMinValue", Category = Content, DisplayName = "Minimum Value", meta = (editcondition = "bOverride_MinValue"))
	float MinValue;

	UE_DEPRECATED(5.2, "Direct access to MaxValue is deprecated. Please use the getter or setter.")
	/** The maximum allowable value that can be manually entered into the spin box */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetMaxValue", BlueprintGetter = "GetMaxValue", Category = Content, DisplayName = "Maximum Value", meta = (editcondition = "bOverride_MaxValue"))
	float MaxValue;

	UE_DEPRECATED(5.2, "Direct access to MinSliderValue is deprecated. Please use the getter or setter.")
	/** The minimum allowable value that can be specified using the slider */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetMinSliderValue", BlueprintGetter = "GetMinSliderValue", Category = Content, DisplayName = "Minimum Slider Value", meta = (editcondition = "bOverride_MinSliderValue"))
	float MinSliderValue;

	UE_DEPRECATED(5.2, "Direct access to MaxSliderValue is deprecated. Please use the getter or setter.")
	/** The maximum allowable value that can be specified using the slider */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetMaxSliderValue", BlueprintGetter = "GetMaxSliderValue", Category = Content, DisplayName = "Maximum Slider Value", meta = (editcondition = "bOverride_MaxSliderValue"))
	float MaxSliderValue;

protected:
	TSharedPtr<SSpinBox<float>> MySpinBox;

	PROPERTY_BINDING_IMPLEMENTATION(float, Value);
};
