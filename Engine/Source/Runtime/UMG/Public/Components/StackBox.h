// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/PanelWidget.h"
#include "Types/SlateEnums.h"
#include "StackBox.generated.h"

class UStackBoxSlot;
class SStackBox;

/**
 * A stack box widget is a layout panel allowing child widgets to be automatically laid out
 * vertically or horizontally.
 *
 * * Many Children
 * * Flows Vertical or Horizontal
 */
UCLASS(meta = (ShortTooltip = "A layout panel for automatically laying child widgets out vertically or horizontally"), MinimalAPI)
class UStackBox : public UPanelWidget
{
	GENERATED_BODY()

public:
	UMG_API UStackBox();

private:
	/** The orientation of the stack box. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category = "Behavior", meta = (AllowPrivateAccess = "true"))
	TEnumAsByte<EOrientation> Orientation;

public:
	/** Get the orientation of the stack box. */
	UMG_API EOrientation GetOrientation() const;
	/** Set the orientation of the stack box. The existing elements will be rearranged. */
	UMG_API void SetOrientation(EOrientation InType);

	/** Adds a new child widget to the container. */
	UFUNCTION(BlueprintCallable, Category="Panel")
	UMG_API UStackBoxSlot* AddChildToStackBox(UWidget* Content);

	/** Replace the widget at the given index it with a different widget. */
	UFUNCTION(BlueprintCallable, Category = "Panel")
	UMG_API bool ReplaceStackBoxChildAt(int32 Index, UWidget* Content);

#if WITH_EDITOR
	//~ UWidget interface
	UMG_API virtual const FText GetPaletteCategory() override;
	//~ End UWidget interface
#endif

protected:
	//~ UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UPanelWidget

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

private:
	TSharedPtr<SStackBox> MyBox;
};
