// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Components/SlateWrapperTypes.h"
#include "Components/PanelSlot.h"
#include "Widgets/SBoxPanel.h"

#include "StackBoxSlot.generated.h"

class SPanel;

/** The Slot for the UStackBox, contains the widget that is flowed vertically or horizontally. */
UCLASS(MinimalAPI)
class UStackBoxSlot : public UPanelSlot
{
	GENERATED_BODY()

private:
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Layout|Stack Box Slot", meta = (AllowPrivateAccess = "true"))
	FMargin Padding;

	/** How much space this slot should occupy in the direction of the panel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category = "Layout|Stack Box Slot", meta = (AllowPrivateAccess = "true"))
	FSlateChildSize Size = FSlateChildSize(ESlateSizeRule::Automatic);

	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Layout|Stack Box Slot", meta = (AllowPrivateAccess = "true"))
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;

	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Layout|Stack Box Slot", meta = (AllowPrivateAccess = "true"))
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = EVerticalAlignment::VAlign_Fill;

private:
	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SStackBox::FSlot* Slot = nullptr;

public:
	UMG_API FMargin GetPadding() const;
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API FSlateChildSize GetSize() const;
	UMG_API void SetSize(FSlateChildSize InSize);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

public:
	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SStackBox> InBox);

	/** Replace the slot content. */
	UMG_API void ReplaceContent(UWidget* Content);

#if WITH_EDITOR
	UMG_API virtual bool NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize) override;
	UMG_API virtual void SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot) override;
#endif //WITH_EDITOR
};
