// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Components/TextWidgetTypes.h"
#include "TextBlock.generated.h"

class STextBlock;
class UMaterialInterface;

/**
 * A simple static text widget.
 *
 * * No Children
 * * Text
 */
UCLASS(meta=(DisplayName="Text"), MinimalAPI)
class UTextBlock : public UTextLayoutWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.1, "Direct access to Text is deprecated. Please use the getter or setter.")
	/** The text to display */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetText", BlueprintSetter="SetText", Category="Content", meta = (MultiLine = "true"))
	FText Text;
	
	/** A bindable delegate to allow logic to drive the text of the widget */
	UPROPERTY()
	FGetText TextDelegate;

	UE_DEPRECATED(5.1, "Direct access to ColorAndOpacity is deprecated. Please use the getter or setter.")
	/** The color of the text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetColorAndOpacity", Category="Appearance")
	FSlateColor ColorAndOpacity;

	/** A bindable delegate for the ColorAndOpacity. */
	UPROPERTY()
	FGetSlateColor ColorAndOpacityDelegate;
	
	UE_DEPRECATED(5.1, "Direct access to MinDesiredWidth is deprecated. Please use the getter or setter.")
	/** The minimum desired size for the text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMinDesiredWidth", Category="Appearance")
	float MinDesiredWidth;
		
	UE_DEPRECATED(5.1, "Direct access to Font is deprecated. Please use the getter or setter.")
	/** The font to render the text with */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetFont", Category="Appearance")
	FSlateFontInfo Font;

	UE_DEPRECATED(5.1, "Direct access to StrikeBrush is deprecated. Please use the getter or setter.")
	/** The brush to strike through text with */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetStrikeBrush", Category="Appearance")
	FSlateBrush StrikeBrush;

	UE_DEPRECATED(5.1, "Direct access to ShadowOffset is deprecated. Please use the getter or setter.")
	/** The direction the shadow is cast */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetShadowOffset", Category="Appearance")
	FVector2D ShadowOffset;

	UE_DEPRECATED(5.1, "Direct access to ShadowColorAndOpacity is deprecated. Please use the getter or setter.")
	/** The color of the shadow */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetShadowColorAndOpacity", Category="Appearance", meta=( DisplayName="Shadow Color" ))
	FLinearColor ShadowColorAndOpacity;

	/** A bindable delegate for the ShadowColorAndOpacity. */
	UPROPERTY()
	FGetLinearColor ShadowColorAndOpacityDelegate;

	/** If true, it will automatically wrap this text widget with an invalidation panel */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Performance", AdvancedDisplay)
	bool bWrapWithInvalidationPanel;

	UE_DEPRECATED(5.1, "Direct access to TextTransformPolicy is deprecated. Please use the getter or setter.")
	/** The text transformation policy to apply to this text block. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetTextTransformPolicy", Category="Appearance", meta=(DisplayName="Transform Policy"))
	ETextTransformPolicy TextTransformPolicy;

	UE_DEPRECATED(5.1, "Direct access to TextOverflowPolicy is deprecated. Please use the getter or setter.")
	/** Sets what happens to text that is clipped and doesn't fit within the clip rect for this widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetTextOverflowPolicy", Category="Clipping", AdvancedDisplay, meta = (DisplayName="Overflow Policy"))
	ETextOverflowPolicy TextOverflowPolicy;

public:
	/** 
	 * Gets the widget text
	 * @return The widget text
	 */
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="GetText (Text)"))
	UMG_API FText GetText() const;

	/**
	 * Directly sets the widget text.
	 * Warning: This will wipe any binding created for the Text property!
	 * @param InText The text to assign to the widget
	 */
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="SetText (Text)"))
	UMG_API virtual void SetText(FText InText);

	/** @return the color and opacity of the text in this text block. */
	UMG_API FSlateColor GetColorAndOpacity() const;

	/**  
	 * Sets the color and opacity of the text in this text block
	 *
	 * @param InColorAndOpacity		The new text color and opacity
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetColorAndOpacity(FSlateColor InColorAndOpacity);

	/**
	* Sets the opacity of the text in this text block
	*
	* @param InOpacity		The new text opacity
	*/
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetOpacity(float InOpacity);

	/** @return the color and opacity of the text drop shadow. */
	UMG_API FLinearColor GetShadowColorAndOpacity() const;

	/**  
	 * Sets the color and opacity of the text drop shadow
	 * Note: if opacity is zero no shadow will be drawn
	 *
	 * @param InShadowColorAndOpacity		The new drop shadow color and opacity
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetShadowColorAndOpacity(FLinearColor InShadowColorAndOpacity);

	/** @return the offset that the text drop shadow should be drawn at. */
	UMG_API FVector2D GetShadowOffset() const;

	/**  
	 * Sets the offset that the text drop shadow should be drawn at
	 *
	 * @param InShadowOffset		The new offset
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetShadowOffset(FVector2D InShadowOffset);

	/** @return the font info for this text block. */
	UMG_API const FSlateFontInfo& GetFont() const;

	/**
	 * Dynamically set the font info for this text block
	 * 
	 * @param InFontInfo The new font info
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetFont(FSlateFontInfo InFontInfo);

	/** @return the brush to use to strike through text.  */
	UMG_API const FSlateBrush& GetStrikeBrush() const;

	/**
	 * Dynamically set the strike brush for this text block
	 *
	 * @param InStrikeBrush The new brush to use to strike through text
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetStrikeBrush(FSlateBrush InStrikeBrush);

	/** @return the minimum desired width for this text block */
	UMG_API float GetMinDesiredWidth() const;

	/**
	 *  Set the minimum desired width for this text block
	 *
	 *  @param InMinDesiredWidth new minimum desired width
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetMinDesiredWidth(float InMinDesiredWidth);
	
	/**
	 * Set the auto wrap for this text block.
	 *
	 * @param InAutoTextWrap to turn wrap on or off.
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetAutoWrapText(bool InAutoTextWrap);

	/** @return the text transformation policy for this text block. */
	UMG_API ETextTransformPolicy GetTextTransformPolicy() const;

	/**
	 * Set the text transformation policy for this text block.
	 *
	 * @param InTransformPolicy the new text transformation policy.
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetTextTransformPolicy(ETextTransformPolicy InTransformPolicy);

	/** @return the text overflow policy for this text block. */
	UMG_API ETextOverflowPolicy GetTextOverflowPolicy() const;

	/**
	 * Set the text overflow policy for this text block.
	 *
	 * @param InOverflowPolicy the new text overflow policy.
	 */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetTextOverflowPolicy(ETextOverflowPolicy InOverflowPolicy);

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetFontMaterial(UMaterialInterface* InMaterial);

	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetFontOutlineMaterial(UMaterialInterface* InMaterial);

	/** 
	* Set the font size of this text block directly, respecting project-specified font resolution DPI
	* 
	* @param The display font size, at the intended DPI (generally 72)
	*/
	UFUNCTION(BlueprintCallable, Category="Appearance", meta=(ToolTip="DisplayFontSize should be set with respect to your Project settings for Font Resolution DPI.  Updates to this value in your project will impact display of text blocks set with this value accordingly."))
	UMG_API void SetFontSize(float DisplayFontSize);

	/** 
	* Get the font size of this text block directly, respecting project-specified font resolution DPI
	* 
	* @return The display font size, at the intended DPI (generally 72)
	*/
	UFUNCTION(BlueprintCallable, Category="Appearance", meta=(ToolTip="Gets the font size of this text block, with respect to the current project settings for Font Resolution DPI."))
	UMG_API float GetFontSize() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API UMaterialInstanceDynamic* GetDynamicFontMaterial();

	/**  */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API UMaterialInstanceDynamic* GetDynamicOutlineMaterial();

protected:
	virtual void OnTextChanged()
	{
	}

	virtual void OnFontChanged()
	{
	}

public:
	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	//~ Begin UWidget Interface
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual void OnCreationFromPalette() override;
	UMG_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	//~ End UWidget Interface

	UMG_API virtual FString GetLabelMetadata() const override;

	UMG_API void HandleTextCommitted(const FText& InText, ETextCommit::Type CommitteType);
#endif

protected:
	//~ Begin UTextLayoutWidget Interface
	UMG_API virtual void OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions) override;
	UMG_API virtual void OnJustificationChanged(ETextJustify::Type InJustification) override;
	UMG_API virtual void OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy) override;
	UMG_API virtual void OnAutoWrapTextChanged(bool InAutoWrapText) override;
	UMG_API virtual void OnWrapTextAtChanged(float InWrapTextAt) override;
	UMG_API virtual void OnLineHeightPercentageChanged(float InLineHeightPercentage) override;
	UMG_API virtual void OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine) override;
	UMG_API virtual void OnMarginChanged(const FMargin& InMargin) override;
	//~ End UTextLayoutWidget Interface

	/**
	 * If this is enabled, text shaping, wrapping, justification are disabled in favor of much faster text layout and measurement.
	 * This feature is only suitable for "simple" text (ie, text containing only numbers or basic ASCII) as it disables the complex text rendering support required for certain languages (such as Arabic and Thai).
	 * It is significantly faster for text that can take advantage of it (particularly if that text changes frequently), but shouldn't be used for localized user-facing text.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Performance, AdvancedDisplay, meta=(AllowPrivateAccess = "true", DesignerRebuild))
	bool bSimpleTextMode;

	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	UMG_API virtual void OnBindingChanged(const FName& Property) override;
	//~ End UWidget Interface

	/** Get the text that should be displayed in the internal Slate widget (allows flags to mutate the display text without modifying the persistent designer property data) */
	UMG_API virtual TAttribute<FText> GetDisplayText();

	UMG_API EVisibility GetTextWarningImageVisibility() const;

#if WITH_ACCESSIBILITY
	UMG_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const override;
#endif

protected:

	TSharedPtr<STextBlock> MyTextBlock;
	
	PROPERTY_BINDING_IMPLEMENTATION(FText, Text);
	PROPERTY_BINDING_IMPLEMENTATION(FSlateColor, ColorAndOpacity);
	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, ShadowColorAndOpacity);
};
