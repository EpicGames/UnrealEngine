// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Components/Widget.h"
#include "Framework/Text/TextLayout.h"
#include "TextWidgetTypes.generated.h"

enum class ETextShapingMethod : uint8;

/**
 * Common data for all widgets that use shaped text.
 * Contains the common options that should be exposed for the underlying Slate widget.
 */
USTRUCT(BlueprintType)
struct FShapedTextOptions
{
	GENERATED_USTRUCT_BODY()

	UMG_API FShapedTextOptions();

	/** Synchronize the properties with the given widget. A template as the Slate widgets conform to the same API, but don't derive from a common base. */
	template <typename TWidgetType>
	void SynchronizeShapedTextProperties(TWidgetType& InWidget) const
	{
		InWidget.SetTextShapingMethod(bOverride_TextShapingMethod ? TOptional<ETextShapingMethod>(TextShapingMethod) : TOptional<ETextShapingMethod>());
		InWidget.SetTextFlowDirection(bOverride_TextFlowDirection ? TOptional<ETextFlowDirection>(TextFlowDirection) : TOptional<ETextFlowDirection>());
	}

	inline bool operator==(const FShapedTextOptions& Other) const
	{
		return bOverride_TextShapingMethod == Other.bOverride_TextShapingMethod
			&& bOverride_TextFlowDirection == Other.bOverride_TextFlowDirection
			&& (!bOverride_TextShapingMethod || TextShapingMethod == Other.TextShapingMethod)
			&& (!bOverride_TextFlowDirection || TextFlowDirection == Other.TextFlowDirection);
	}

	inline bool operator!=(const FShapedTextOptions& Other) const
	{
		return !operator==(Other);
	}

	/**  */
	UPROPERTY(EditAnywhere, Category=Localization, meta=(InlineEditConditionToggle))
	uint8 bOverride_TextShapingMethod : 1;

	/**  */
	UPROPERTY(EditAnywhere, Category=Localization, meta=(InlineEditConditionToggle))
	uint8 bOverride_TextFlowDirection : 1;

	/** Which text shaping method should the text within this widget use? (unset to use the default returned by GetDefaultTextShapingMethod) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Localization, AdvancedDisplay, meta=(EditCondition="bOverride_TextShapingMethod"))
	ETextShapingMethod TextShapingMethod;
		
	/** Which text flow direction should the text within this widget use? (unset to use the default returned by GetDefaultTextFlowDirection) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Localization, AdvancedDisplay, meta=(EditCondition="bOverride_TextFlowDirection"))
	ETextFlowDirection TextFlowDirection;
};

enum class EFontFacesLoadingPaintPolicy : uint8;


/**
 * Base class for all widgets that use a text layout.
 * Contains the common options that should be exposed for the underlying Slate widget.
 */
UCLASS(Abstract, BlueprintType, MinimalAPI)
class UTextLayoutWidget : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(FOnAllFontFacesFinishLoading_TextLayoutWidget, UTextLayoutWidget, OnAllFontFacesFinishLoading);

	bool GetAutoWrapText() const { return AutoWrapText; }
	float GetWrapTextAt() const { return WrapTextAt; }

	UFUNCTION(BlueprintSetter)
	UMG_API virtual void SetJustification(ETextJustify::Type InJustification);

	UMG_API void SetShapedTextOptions(FShapedTextOptions InShapedTextOptions);
	UMG_API void SetWrappingPolicy(ETextWrappingPolicy InWrappingPolicy);
	UMG_API void SetAutoWrapText(bool InAutoWrapText);
	UMG_API void SetWrapTextAt(float InWrapTextAt);
	UMG_API void SetLineHeightPercentage(float InLineHeightPercentage);
	UMG_API void SetApplyLineHeightToBottomLine(bool InApplyLineHeightToBottomLine);
	UMG_API void SetFontFacesLoadingPaintPolicy(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy);
	UMG_API void SetMargin(const FMargin& InMargin);

protected:
	virtual void OnShapedTextOptionsChanged(FShapedTextOptions InShapedTextOptions) {};
	virtual void OnJustificationChanged(ETextJustify::Type InJustification) {};
	virtual void OnWrappingPolicyChanged(ETextWrappingPolicy InWrappingPolicy) {};
	virtual void OnAutoWrapTextChanged(bool InAutoWrapText) {};
	virtual void OnWrapTextAtChanged(float InWrapTextAt) {};
	virtual void OnLineHeightPercentageChanged(float InLineHeightPercentage) {};
	virtual void OnApplyLineHeightToBottomLineChanged(bool InApplyLineHeightToBottomLine) {};
	virtual void OnFontFacesLoadingPaintPolicyChange(EFontFacesLoadingPaintPolicy InFontFacesLoadingPaintPolicy) {};
	virtual void OnMarginChanged(const FMargin& InMargin) {};

	/** Synchronize the properties with the given widget. A template as the Slate widgets conform to the same API, but don't derive from a common base. */
	template <typename TWidgetType>
	void SynchronizeTextLayoutProperties(TWidgetType& InWidget)
	{
		ShapedTextOptions.SynchronizeShapedTextProperties(InWidget);

		InWidget.SetJustification(Justification);
		InWidget.SetAutoWrapText(!!AutoWrapText);
		InWidget.SetWrapTextAt(WrapTextAt != 0 ? WrapTextAt : TAttribute<float>());
		InWidget.SetWrappingPolicy(WrappingPolicy);
		InWidget.SetMargin(Margin);
		InWidget.SetLineHeightPercentage(LineHeightPercentage);
		InWidget.SetApplyLineHeightToBottomLine(ApplyLineHeightToBottomLine);
		InWidget.SetFontFacesLoadingPaintPolicy(FontFacesLoadingPaintPolicy);
	}

	void SlateHandleAllFontFacesFinishLoading()
	{
		OnAllFontFacesFinishLoading.Broadcast();
	}

	/** Controls how the text within this widget should be shaped. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category=Localization, AdvancedDisplay, meta=(ShowOnlyInnerProperties))
	FShapedTextOptions ShapedTextOptions;

	/** Called when all font faces for the text has finished async loading. Is not called if font faces are synchronously loaded. */
	UPROPERTY(BlueprintAssignable, Category = "Font|Event")
	FOnAllFontFacesFinishLoading_TextLayoutWidget OnAllFontFacesFinishLoading;

	/** How the text should be aligned with the margin. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetJustification, Setter, Category=Appearance)
	TEnumAsByte<ETextJustify::Type> Justification;

	/** The wrapping policy to use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category=Wrapping, AdvancedDisplay)
	ETextWrappingPolicy WrappingPolicy;

	/** True if we're wrapping text automatically based on the computed horizontal space for this widget. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category=Wrapping)
	uint8 AutoWrapText:1;

	/** Whether to leave extra space below the last line due to line height. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category=Appearance, AdvancedDisplay)
	bool ApplyLineHeightToBottomLine;

	/** Controls how painting the text in the widget should behave when font faces for the text is still loading. */
	UPROPERTY(EditAnywhere, Setter, Category = Font)
	EFontFacesLoadingPaintPolicy FontFacesLoadingPaintPolicy;

	/** Whether text wraps onto a new line when it's length exceeds this width; if this value is zero or negative, no wrapping occurs. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category=Wrapping)
	float WrapTextAt;

	/** The amount of blank space left around the edges of text area. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category=Appearance, AdvancedDisplay)
	FMargin Margin;

	/** The amount to scale each lines height by. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category=Appearance, AdvancedDisplay)
	float LineHeightPercentage;
};
