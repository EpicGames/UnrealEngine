// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateBrush.h"
#include "Widgets/SWidget.h"
#include "Components/Widget.h"
#include "Widgets/Images/SThrobber.h"
#include "Throbber.generated.h"

class USlateBrushAsset;

/**
 * A Throbber widget that shows several zooming circles in a row.
 */
UCLASS(MinimalAPI)
class UThrobber : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	
	UE_DEPRECATED(5.1, "Direct access to NumberOfPieces is deprecated. Please use the getter or setter.")
	/** How many pieces there are */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetNumberOfPieces", Category=Appearance, meta=( ClampMin="1", ClampMax="25", UIMin="1", UIMax="25" ))
	int32 NumberOfPieces;

	UE_DEPRECATED(5.1, "Direct access to bAnimateHorizontally is deprecated. Please use the getter or setter.")
	/** Should the pieces animate horizontally? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="IsAnimateHorizontally", Setter="SetAnimateHorizontally", BlueprintSetter = "SetAnimateHorizontally", Category = Appearance)
	bool bAnimateHorizontally;

	UE_DEPRECATED(5.1, "Direct access to bAnimateVertically is deprecated. Please use the getter or setter.")
	/** Should the pieces animate vertically? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="IsAnimateVertically", Setter="SetAnimateVertically", BlueprintSetter="SetAnimateVertically", Category = Appearance)
	bool bAnimateVertically;

	UE_DEPRECATED(5.1, "Direct access to bAnimateOpacity is deprecated. Please use the getter or setter.")
	/** Should the pieces animate their opacity? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="IsAnimateOpacity", Setter="SetAnimateOpacity", BlueprintSetter="SetAnimateOpacity", Category = Appearance)
	bool bAnimateOpacity;

	UE_DEPRECATED(5.1, "Direct access to Image is deprecated. Please use the getter or setter.")
	/** The animated pieces. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category=Appearance)
	FSlateBrush Image;

public:

	/** Sets how many pieces there are */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetNumberOfPieces(int32 InNumberOfPieces);
	UMG_API int32 GetNumberOfPieces() const;

	/** Sets whether the pieces animate horizontally. */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetAnimateHorizontally(bool bInAnimateHorizontally);
	UMG_API bool IsAnimateHorizontally() const;

	/** Sets whether the pieces animate vertically. */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	UMG_API void SetAnimateVertically(bool bInAnimateVertically);
	UMG_API bool IsAnimateVertically() const;

	/** Sets whether the pieces animate their opacity. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetAnimateOpacity(bool bInAnimateOpacity);
	UMG_API bool IsAnimateOpacity() const;

	UMG_API void SetImage(const FSlateBrush& Brush);
	UMG_API const FSlateBrush& GetImage() const;

	//~ Begin UWidget Interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End UWidget Interface

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

private:
	/** Gets the combined value of the animation properties as a single SThrobber::EAnimation value. */
	SThrobber::EAnimation GetAnimation() const;

private:
	/** The Throbber widget managed by this object. */
	TSharedPtr<SThrobber> MyThrobber;
};
