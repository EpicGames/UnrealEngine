// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ListView.h"
#include "Widgets/Views/STileView.h"
#include "TileView.generated.h"

/**
 * A ListView that presents the contents as a set of tiles all uniformly sized.
 *
 * To make a widget usable as an entry in a TileView, it must inherit from the IUserObjectListEntry interface.
 */
UCLASS(MinimalAPI)
class UTileView : public UListView
{
	GENERATED_BODY()

public:
	UMG_API UTileView(const FObjectInitializer& ObjectInitializer);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	/** Sets the height of every tile entry */
	UFUNCTION(BlueprintCallable, Category = TileView)
	UMG_API void SetEntryHeight(float NewHeight);

	/** Sets the width of every tile entry */
	UFUNCTION(BlueprintCallable, Category = TileView)
	UMG_API void SetEntryWidth(float NewWidth);

	/** Gets the height of tile entries */
	UFUNCTION(BlueprintPure, Category = TileView)
	float GetEntryHeight() const { return EntryHeight; }

	/** Gets the width of tile entries */
	UFUNCTION(BlueprintPure, Category = TileView)
	float GetEntryWidth() const { return EntryWidth; }

protected:
	UMG_API virtual TSharedRef<STableViewBase> RebuildListWidget() override;
	UMG_API virtual FMargin GetDesiredEntryPadding(UObject* Item) const override;

	UMG_API float GetTotalEntryHeight() const;
	UMG_API float GetTotalEntryWidth() const;

	/** STileView construction helper - useful if using a custom STileView subclass */
	template <template<typename> class TileViewT = STileView>
	TSharedRef<TileViewT<UObject*>> ConstructTileView()
	{
		FTileViewConstructArgs Args;
		Args.bAllowFocus = bIsFocusable;
		Args.SelectionMode = SelectionMode;
		Args.bClearSelectionOnClick = bClearSelectionOnClick;
		Args.ConsumeMouseWheel = ConsumeMouseWheel;
		Args.bReturnFocusToSelection = bReturnFocusToSelection;
		Args.bEnableProximateEntryNavigation = bEnableProximateEntryNavigation;
		Args.TileAlignment = TileAlignment;
		Args.bWrapDirectionalNavigation = bWrapHorizontalNavigation;
		Args.Orientation = Orientation;
		Args.ScrollBarStyle = &ScrollBarStyle;
		Args.ScrollbarDisabledVisibility = UWidget::ConvertSerializedVisibilityToRuntime(ScrollbarDisabledVisibility);

		if (IsAligned() && !bEntrySizeIncludesEntrySpacing)
		{
			// This tells the underlying ListView to expect entries with the final width/height equal to EntryWidth/EntryHeight + spacing
			// It allows the entry widget to always occupy the entire EntryWidth/EntryHeight.
			Args.EntryWidth = GetTotalEntryWidth();
			Args.EntryHeight = GetTotalEntryHeight();
		}
		else
		{
			// This tells the underlying ListView to expect entries with the final width/height equal to EntryWidth/EntryHeight.
			// It forces the entry widget size to adjust so that the summation of entry widget width/height and spacing does not exceed EntryWidth/EntryHeight.
			Args.EntryWidth = EntryWidth;
			Args.EntryHeight = EntryHeight;
		}

		MyListView = MyTileView = ITypedUMGListView<UObject*>::ConstructTileView<TileViewT>(this, ListItems, Args);
		MyTileView->SetOnEntryInitialized(SListView<UObject*>::FOnEntryInitialized::CreateUObject(this, &UTileView::HandleOnEntryInitializedInternal));
		return StaticCastSharedRef<TileViewT<UObject*>>(MyTileView.ToSharedRef());
	}

private:
	/** Returns whether the TileView is left, right or center aligned. */
	UFUNCTION()
	UMG_API bool IsAligned() const;

protected:
	/** The height of each tile */
	UPROPERTY(EditAnywhere, Category = ListEntries)
	float EntryHeight = 128.f;

	/** The width of each tile */
	UPROPERTY(EditAnywhere, Category = ListEntries)
	float EntryWidth = 128.f;

	/** The method by which to align the tile entries in the available space for the tile view */
	UPROPERTY(EditAnywhere, Category = ListEntries)
	EListItemAlignment TileAlignment;

	/** True to allow left/right navigation to wrap back to the tile on the opposite edge */
	UPROPERTY(EditAnywhere, Category = Navigation)
	bool bWrapHorizontalNavigation = false;

	/** Set the visibility of the Scrollbar when it's not needed */
	UPROPERTY(EditAnywhere, Category = ListView, meta=(ValidEnumValues="Collapsed, Hidden, Visible"))
	ESlateVisibility ScrollbarDisabledVisibility = ESlateVisibility::Collapsed;

	TSharedPtr<STileView<UObject*>> MyTileView;

private:
	/**
	 * True if entry dimensions should be the sum of the entry widget dimensions and the spacing.
	 * This means the size of the entry widget will be adjusted so that the summation of the widget size and entry spacing always equals entry size.
	 */
	UPROPERTY(EditAnywhere, Category = ListEntries, meta = (EditCondition = "IsAligned"))
	bool bEntrySizeIncludesEntrySpacing = true;
};
