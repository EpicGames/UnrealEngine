// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ListView.h"
#include "Widgets/Views/STreeView.h"
#include "TreeView.generated.h"

DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnGetItemChildrenDynamic, UObject*, Item, TArray<UObject*>&, Children);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnItemExpansionChangedDynamic, UObject*, Item, bool, bIsExpanded);

/**
 * Similar to ListView, but can display a hierarchical tree of elements.
 * The base items source for the tree identifies the root items, each of which can have n associated child items.
 * There is no hard limit to the nesting - child items can have children and so on
 *
 * To make a widget usable as an entry in a TreeView, it must inherit from the IUserObjectListEntry interface.
 */
UCLASS(MinimalAPI)
class UTreeView : public UListView
{
	GENERATED_BODY()

public:
	UMG_API UTreeView(const FObjectInitializer& ObjectInitializer);
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	
	/** Attempts to expand/collapse the given item (only relevant if the item has children) */
	UFUNCTION(BlueprintCallable, Category = TreeView)
	UMG_API void SetItemExpansion(UObject* Item, bool bExpandItem);

	/** Expands all items with children */
	UFUNCTION(BlueprintCallable, Category = TreeView)
	UMG_API void ExpandAll();

	/** Collapses all currently expanded items */
	UFUNCTION(BlueprintCallable, Category = TreeView)
	UMG_API void CollapseAll();

	template <typename ObjectT>
	void SetOnGetItemChildren(ObjectT* InUserObject, typename TSlateDelegates<UObject*>::FOnGetChildren::TMethodPtr<ObjectT> InMethodPtr)
	{
		static_assert(TIsDerivedFrom<ObjectT, UObject>::IsDerived, "Only UObject ptrs can be passed directly when binding to SetOnGetItemChildren. Pass a shared ref for non-UObject binders.");
		OnGetItemChildren = TSlateDelegates<UObject*>::FOnGetChildren::CreateUObject(InUserObject, InMethodPtr);
	}
	template <typename ObjectT>
	void SetOnGetItemChildren(TSharedRef<ObjectT> InUserObject, typename TSlateDelegates<UObject*>::FOnGetChildren::TMethodPtr<ObjectT> InMethodPtr)
	{
		OnGetItemChildren = TSlateDelegates<UObject*>::FOnGetChildren::CreateSP(InUserObject, InMethodPtr);
	}

protected:
	UMG_API virtual TSharedRef<STableViewBase> RebuildListWidget() override;
	UMG_API virtual void OnItemClickedInternal(UObject* ListItem) override;

	UMG_API virtual void OnItemExpansionChangedInternal(UObject* Item, bool bIsExpanded) override;
	UMG_API virtual void OnGetChildrenInternal(UObject* Item, TArray<UObject*>& OutChildren) const override;

	/** STreeView construction helper - useful if using a custom STreeView subclass */
	template <template<typename> class TreeViewT = STreeView>
	TSharedRef<TreeViewT<UObject*>> ConstructTreeView()
	{
		FTreeViewConstructArgs Args;
		Args.bClearSelectionOnClick = bClearSelectionOnClick;
		Args.SelectionMode = SelectionMode;
		Args.ConsumeMouseWheel = ConsumeMouseWheel;
		Args.bReturnFocusToSelection = bReturnFocusToSelection;
		Args.TreeViewStyle = &WidgetStyle;
		Args.ScrollBarStyle = &ScrollBarStyle;

		MyListView = MyTreeView = ITypedUMGListView<UObject*>::ConstructTreeView<TreeViewT>(this, ListItems, Args);
		MyTreeView->SetOnEntryInitialized(SListView<UObject*>::FOnEntryInitialized::CreateUObject(this, &UTreeView::HandleOnEntryInitializedInternal));
		return StaticCastSharedRef<TreeViewT<UObject*>>(MyTreeView.ToSharedRef());
	}


	TSharedPtr<STreeView<UObject*>> MyTreeView;

private:
	/** Called to get the list of children (if any) that correspond to the given item. Only called if the native C++ version of the event is not bound. */
	UPROPERTY(EditAnywhere, Category = Events, meta = (IsBindableEvent, AllowPrivateAccess = true, DisplayName = "On Get Item Children"))
	FOnGetItemChildrenDynamic BP_OnGetItemChildren;

	UPROPERTY(BlueprintAssignable, Category = Events, meta = (AllowPrivateAccess = true, DisplayName = "On Item Expansion Changed"))
	FOnItemExpansionChangedDynamic BP_OnItemExpansionChanged;

	TSlateDelegates<UObject*>::FOnGetChildren OnGetItemChildren;
	FOnItemExpansionChanged OnItemExpansionChangedEvent;
};
