// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Widgets/SWidget.h"
#include "Components/PanelWidget.h"
#include "UniformGridPanel.generated.h"

class SUniformGridPanel;
class UUniformGridSlot;

/**
 * A panel that evenly divides up available space between all of its children.
 */
UCLASS(MinimalAPI)
class UUniformGridPanel : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.1, "Direct access to SlotPadding is deprecated. Please use the getter or setter.")
	/** Padding given to each slot */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetSlotPadding", Category="Child Layout")
	FMargin SlotPadding;

	UE_DEPRECATED(5.1, "Direct access to MinDesiredSlotWidth is deprecated. Please use the getter or setter.")
	/** The minimum desired width of the slots */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMinDesiredSlotWidth", Category="Child Layout")
	float MinDesiredSlotWidth;

	UE_DEPRECATED(5.1, "Direct access to MinDesiredSlotHeight is deprecated. Please use the getter or setter.")
	/** The minimum desired height of the slots */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetMinDesiredSlotHeight", Category="Child Layout")
	float MinDesiredSlotHeight;

public:

	/** */
	UMG_API FMargin GetSlotPadding() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Child Layout")
	UMG_API void SetSlotPadding(FMargin InSlotPadding);

	/** */
	UMG_API float GetMinDesiredSlotWidth() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Child Layout")
	UMG_API void SetMinDesiredSlotWidth(float InMinDesiredSlotWidth);

	/** */
	UMG_API float GetMinDesiredSlotHeight() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Child Layout")
	UMG_API void SetMinDesiredSlotHeight(float InMinDesiredSlotHeight);

public:

	/**  */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API UUniformGridSlot* AddChildToUniformGrid(UWidget* Content, int32 InRow = 0, int32 InColumn = 0);

public:

	//~ UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif
	//~ End of UWidget interface

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:

	TSharedPtr<SUniformGridPanel> MyUniformGridPanel;

protected:
	//~ UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End of UWidget interface
};
