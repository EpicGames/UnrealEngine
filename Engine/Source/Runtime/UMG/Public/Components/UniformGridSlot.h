// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Components/PanelSlot.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#include "UniformGridSlot.generated.h"

/**
 * A slot for UUniformGridPanel, these slots all share the same size as the largest slot
 * in the grid.
 */
UCLASS(MinimalAPI)
class UUniformGridSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Uniform Grid Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Uniform Grid Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;
	
	UE_DEPRECATED(5.1, "Direct access to Row is deprecated. Please use the getter or setter.")
	/** The row index of the cell this slot is in */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetRow", meta=( UIMin = "0" ), Category="Layout|Uniform Grid Slot")
	int32 Row;
	
	UE_DEPRECATED(5.1, "Direct access to Column is deprecated. Please use the getter or setter.")
	/** The column index of the cell this slot is in */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetColumn", meta=( UIMin = "0" ), Category="Layout|Uniform Grid Slot")
	int32 Column;

public:

	/** Gets the row index of the slot */
	UMG_API int32 GetRow() const;

	/** Sets the row index of the slot, this determines what cell the slot is in the panel */
	UFUNCTION(BlueprintCallable, Category="Layout|Uniform Grid Slot")
	UMG_API void SetRow(int32 InRow);

	/** Gets the column index of the slot. */
	UMG_API int32 GetColumn() const;

	/** Sets the column index of the slot, this determines what cell the slot is in the panel */
	UFUNCTION(BlueprintCallable, Category="Layout|Uniform Grid Slot")
	UMG_API void SetColumn(int32 InColumn);

	/**  */
	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Layout|Uniform Grid Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	/**  */
	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	/**  */
	UFUNCTION(BlueprintCallable, Category="Layout|Uniform Grid Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

public:

	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SUniformGridPanel> GridPanel);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual bool NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize) override;
	UMG_API virtual void SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot) override;
#endif //WITH_EDITOR

private:
	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SUniformGridPanel::FSlot* Slot;
};
