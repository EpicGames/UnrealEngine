// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Components/PanelWidget.h"
#include "VerticalBox.generated.h"

class UVerticalBoxSlot;

/**
 * A vertical box widget is a layout panel allowing child widgets to be automatically laid out
 * vertically.
 *
 * * Many Children
 * * Flows Vertical
 */
UCLASS(meta = (ShortTooltip = "A layout panel for automatically laying child widgets out vertically"), MinimalAPI)
class UVerticalBox : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:

	/**  */
	UFUNCTION(BlueprintCallable, Category="Panel")
	UMG_API UVerticalBoxSlot* AddChildToVerticalBox(UWidget* Content);

#if WITH_EDITOR
	// UWidget interface
	UMG_API virtual const FText GetPaletteCategory() override;
	// End UWidget interface
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

protected:

	TSharedPtr<class SVerticalBox> MyVerticalBox;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
