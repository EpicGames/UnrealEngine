// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/SlateWrapperTypes.h"
#include "Components/PanelSlot.h"
#include "Widgets/SBoxPanel.h"

#include "VerticalBoxSlot.generated.h"

/** The Slot for the UVerticalBox, contains the widget that is flowed vertically */
UCLASS(MinimalAPI)
class UVerticalBoxSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:
	
	UE_DEPRECATED(5.1, "Direct access to Size is deprecated. Please use the getter or setter.")
	/** How much space this slot should occupy in the direction of the panel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetSize", Category="Layout|Vertical Box Slot", meta = (DisplayAfter = "Padding"))
	FSlateChildSize Size;

	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetPadding", Category="Layout|Vertical Box Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Vertical Box Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Vertical Box Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

private:

	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SVerticalBox::FSlot* Slot;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Vertical Box Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API FSlateChildSize GetSize() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Vertical Box Slot")
	UMG_API void SetSize(FSlateChildSize InSize);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Vertical Box Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Vertical Box Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

public:

	// UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UPanelSlot interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SVerticalBox> InVerticalBox);

#if WITH_EDITOR
	UMG_API virtual bool NudgeByDesigner(const FVector2D& NudgeDirection, const TOptional<int32>& GridSnapSize) override;
	UMG_API virtual void SynchronizeFromTemplate(const UPanelSlot* const TemplateSlot) override;
#endif //WITH_EDITOR
};
