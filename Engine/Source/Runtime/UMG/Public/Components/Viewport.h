// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "ShowFlags.h"
#include "Widgets/SWidget.h"
#include "UnrealClient.h"
#include "SceneTypes.h"
#include "Camera/CameraTypes.h"
#include "SceneManagement.h"
#include "Components/ContentWidget.h"
#include "ViewportClient.h"

#include "Viewport.generated.h"

class FCanvas;
class FPreviewScene;
class SAutoRefreshViewport;

/**
* Stores the transformation data for the viewport camera
*/
struct FUMGViewportCameraTransform
{
public:
	UMG_API FUMGViewportCameraTransform();

	/** Sets the transform's location */
	UMG_API void SetLocation(const FVector& Position);

	/** Sets the transform's rotation */
	void SetRotation(const FRotator& Rotation)
	{
		ViewRotation = Rotation;
	}

	/** Sets the location to look at during orbit */
	void SetLookAt(const FVector& InLookAt)
	{
		LookAt = InLookAt;
	}

	/** Set the ortho zoom amount */
	void SetOrthoZoom(float InOrthoZoom)
	{
		OrthoZoom = InOrthoZoom;
	}


	/** @return The transform's location */
	inline const FVector& GetLocation() const { return ViewLocation; }

	/** @return The transform's rotation */
	inline const FRotator& GetRotation() const { return ViewRotation; }

	/** @return The look at point for orbiting */
	inline const FVector& GetLookAt() const { return LookAt; }

	/** @return The ortho zoom amount */
	inline float GetOrthoZoom() const { return OrthoZoom; }

	/**
	* Animates from the current location to the desired location
	*
	* @param InDesiredLocation	The location to transition to
	* @param bInstant			If the desired location should be set instantly rather than transitioned to over time
	*/
	UMG_API void TransitionToLocation(const FVector& InDesiredLocation, bool bInstant);

	/**
	* Updates any current location transitions
	*
	* @return true if there is currently a transition
	*/
	UMG_API bool UpdateTransition();

	/**
	* Computes a matrix to use for viewport location and rotation when orbiting
	*/
	UMG_API FMatrix ComputeOrbitMatrix() const;
private:
	/** The time when a transition to the desired location began */
	double TransitionStartTime;
	/** Current viewport Position. */
	FVector	ViewLocation;
	/** Current Viewport orientation; valid only for perspective projections. */
	FRotator ViewRotation;
	/** Desired viewport location when animating between two locations */
	FVector	DesiredLocation;
	/** When orbiting, the point we are looking at */
	FVector LookAt;
	/** Viewport start location when animating to another location */
	FVector StartLocation;
	/** Ortho zoom amount */
	float OrthoZoom;
};


class FUMGViewportClient : public FCommonViewportClient, public FViewElementDrawer
{
public:
	UMG_API FUMGViewportClient(FPreviewScene* InPreviewScene = nullptr);
	UMG_API virtual ~FUMGViewportClient();

	using FViewElementDrawer::Draw;

	// FViewportClient interface
	UMG_API virtual UWorld* GetWorld() const override;
	UMG_API virtual void Draw(FViewport* InViewport, FCanvas* Canvas) override;

	// FUMGViewportClient

	UMG_API virtual void Tick(float InDeltaTime);

	UMG_API virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily);


	/**
	 * @return The scene being rendered in this viewport
	 */
	UMG_API virtual FSceneInterface* GetScene() const;


	UMG_API bool IsAspectRatioConstrained() const;

	UMG_API void SetBackgroundColor(FLinearColor InBackgroundColor);
	UMG_API FLinearColor GetBackgroundColor() const;

	/** Sets the location of the viewport's camera */
	void SetViewLocation(const FVector& NewLocation)
	{
		ViewTransform.SetLocation(NewLocation);
	}

	/** Sets the location of the viewport's camera */
	void SetViewRotation(const FRotator& NewRotation)
	{
		ViewTransform.SetRotation(NewRotation);
	}

	/**
	* Sets the look at location of the viewports camera for orbit *
	*
	* @param LookAt The new look at location
	* @param bRecalulateView	If true, will recalculate view location and rotation to look at the new point immediatley
	*/
	void SetLookAtLocation(const FVector& LookAt, bool bRecalculateView = false)
	{
		ViewTransform.SetLookAt(LookAt);

		if ( bRecalculateView )
		{
			FMatrix OrbitMatrix = ViewTransform.ComputeOrbitMatrix();
			OrbitMatrix = OrbitMatrix.InverseFast();

			ViewTransform.SetRotation(OrbitMatrix.Rotator());
			ViewTransform.SetLocation(OrbitMatrix.GetOrigin());
		}
	}

	/** Sets ortho zoom amount */
	void SetOrthoZoom(float InOrthoZoom)
	{
		// A zero ortho zoom is not supported and causes NaN/div0 errors
		check(InOrthoZoom != 0);
		ViewTransform.SetOrthoZoom(InOrthoZoom);
	}

	/** @return the current viewport camera location */
	const FVector& GetViewLocation() const
	{
		return ViewTransform.GetLocation();
	}

	/** @return the current viewport camera rotation */
	const FRotator& GetViewRotation() const
	{
		return ViewTransform.GetRotation();
	}

	/** @return the current look at location */
	const FVector& GetLookAtLocation() const
	{
		return ViewTransform.GetLookAt();
	}

	/** @return the current ortho zoom amount */
	float GetOrthoZoom() const
	{
		return ViewTransform.GetOrthoZoom();
	}

	/** @return The number of units per pixel displayed in this viewport */
	UMG_API float GetOrthoUnitsPerPixel(const FViewport* Viewport) const;

	void SetEngineShowFlags(FEngineShowFlags InEngineShowFlags)
	{
		EngineShowFlags = InEngineShowFlags;
	}

	FMatrix GetViewProjectionMatrix() const;

protected:

	/** The scene used for the viewport. Owned externally */
	FPreviewScene* PreviewScene;
	
	FMinimalViewInfo ViewInfo;

	FLinearColor BackgroundColor;

	/** Viewport camera transform data */
	FUMGViewportCameraTransform ViewTransform;

	FViewport* Viewport;

	/** The viewport's scene view state. */
	FSceneViewStateReference ViewState;

	/** A set of flags that determines visibility for various scene elements. */
	FEngineShowFlags EngineShowFlags;

	FMatrix CachedViewProjectionMatrix;
};

/**
 * 
 */
UCLASS(Experimental, MinimalAPI)
class UViewport : public UContentWidget
{
	GENERATED_UCLASS_BODY()

	UE_DEPRECATED(5.2, "Direct access to BackgroundColor is deprecated. Please use the getter or setter.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = Appearance)
	FLinearColor BackgroundColor;

	/**
	 * True = WorldType in the new viewport will be set to "EditorPreview". This is advised when spawning a UMG viewport for Editor Tooling
	 * as the default World is WorldType=GamePreview and can cause unneeded subsystems to respond to the creation of the new UWorld.
	 */
	UPROPERTY(EditAnywhere, Category = "World")
	bool bIsEditorPreview = false;

	UFUNCTION(BlueprintCallable, Category="Viewport")
	UMG_API UWorld* GetViewportWorld() const;

	UFUNCTION(BlueprintCallable, Category="Camera")
	UMG_API FVector GetViewLocation() const;

	UFUNCTION(BlueprintCallable, Category="Camera")
	UMG_API void SetViewLocation(FVector Location);

	UFUNCTION(BlueprintCallable, Category="Camera")
	UMG_API FRotator GetViewRotation() const;

	UFUNCTION(BlueprintCallable, Category="Camera")
	UMG_API void SetViewRotation(FRotator Rotation);

	UFUNCTION(BlueprintCallable, Category="Viewport")
	UMG_API AActor* Spawn(TSubclassOf<AActor> ActorClass);

	UMG_API void SetBackgroundColor(const FLinearColor& InColor);

	UMG_API const FLinearColor& GetBackgroundColor() const;

	UFUNCTION(BlueprintCallable, Category = "Viewport")
	UMG_API void SetEnableAdvancedFeatures(bool InEnableAdvancedFeatures);

	UFUNCTION(BlueprintCallable, Category = "Viewport")
	UMG_API void SetLightIntensity(float LightIntensity);

	UFUNCTION(BlueprintCallable, Category = "Viewport")
	UMG_API void SetSkyIntensity(float LightIntensity);

	UFUNCTION(BlueprintCallable, Category = "Viewport")
	UMG_API void SetShowFlag(FString InShowFlagName, bool InValue);

	UFUNCTION(BlueprintCallable, Category = "Viewport")
	UMG_API FMatrix GetViewProjectionMatrix() const;
	

	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
#endif

protected:

	// UPanelWidget
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:
	TSharedPtr<class SAutoRefreshViewport> ViewportWidget;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	/** Show flags for the engine for this viewport */
	FEngineShowFlags ShowFlags;
};
