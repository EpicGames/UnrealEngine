// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Visual.generated.h"

/** The base class for elements in UMG: slots and widgets. */
UCLASS(DefaultToInstanced, MinimalAPI)
class UVisual : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren);

	//~ Begin UObject Interface
	UMG_API virtual void BeginDestroy() override;
	UMG_API virtual bool NeedsLoadForServer() const override;
	//~ End UObject Interface

private:
	// Hide this to avoid confusion with UI selection state
	using UObject::IsSelected;
};
