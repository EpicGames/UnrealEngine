// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Binding/States/WidgetStateBitfield.h"
#include "UObject/ObjectMacros.h"
#include "FieldNotificationDeclaration.h"
#include "INotifyFieldValueChanged.h"
#include "Misc/Attribute.h"
#include "Templates/SubclassOf.h"
#include "UObject/ScriptMacros.h"
#include "Styling/SlateColor.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Widgets/SWidget.h"
#include "Types/SlateStructs.h"
#include "Components/Visual.h"
#include "Styling/SlateBrush.h"
#include "UObject/TextProperty.h"
#include "Components/SlateWrapperTypes.h"
#include "Slate/WidgetTransform.h"
#include "UObject/UObjectThreadContext.h"
#include "GameFramework/PlayerController.h"
#include "Blueprint/WidgetNavigation.h"
#include "Widgets/WidgetPixelSnapping.h"

#include "Widget.generated.h"

#ifndef UE_HAS_WIDGET_GENERATED_BY_CLASS
  #define UE_HAS_WIDGET_GENERATED_BY_CLASS (!UE_BUILD_SHIPPING || WITH_EDITOR)
#endif

#ifndef WIDGET_INCLUDE_RELFECTION_METADATA
#define WIDGET_INCLUDE_RELFECTION_METADATA  (!UE_BUILD_SHIPPING || WITH_EDITOR)
#endif

class ULocalPlayer;
class SObjectWidget;
class UGameViewportSubsystem;
class UPanelSlot;
class UPropertyBinding;
class UUserWidget;
struct FDynamicPropertyPath;
struct FNavigationMethod;
enum class ECheckBoxState : uint8;

namespace UMWidget
{
	// valid keywords for the UCLASS macro
	enum
	{
		// [ClassMetadata] [PropertyMetadata] Specifies the base class by which to filter available entry classes within DynamicEntryBox and any ListViewBase.
		EntryClass,

		// [ClassMetadata] [PropertyMetadata] Specifies the base class by which to filter available entry classes within DynamicEntryBox and any ListViewBase.
		EntryInterface,
	};

	// valid metadata keywords for the UPROPERTY macro
	enum
	{
		// [PropertyMetadata] This property if changed will rebuild the widget designer preview.  Use sparingly, try to update most properties by
		// setting them in the SynchronizeProperties function.
		// UPROPERTY(meta=(DesignerRebuild))
		DesignerRebuild,

		// [PropertyMetadata] This property requires a widget be bound to it in the designer.  Allows easy native access to designer defined controls.
		// UPROPERTY(meta=(BindWidget))
		BindWidget,

		// [PropertyMetadata] This property optionally allows a widget be bound to it in the designer.  Allows easy native access to designer defined controls.
		// UPROPERTY(meta=(BindWidgetOptional))
		BindWidgetOptional,

		// [PropertyMetadata] This property optionally allows a widget be bound to it in the designer.  Allows easy native access to designer defined controls.
		// UPROPERTY(meta=(BindWidget, OptionalWidget=true))
		OptionalWidget,

		// [PropertyMetadata] This property requires a widget animation be bound to it in the designer.  Allows easy native access to designer defined animations.
		// UPROPERTY(meta=(BindWidgetAnim))
		BindWidgetAnim,

		// [PropertyMetadata] This property optionally allows a animation widget be bound to it in the designer.  Allows easy native access to designer defined animation.
		// UPROPERTY(meta=(BindWidgetAnimOptional))
		BindWidgetAnimOptional,

		// [PropertyMetadata] Exposes a dynamic delegate property in the details panel for the widget.
		// UPROPERTY(meta=(IsBindableEvent))
		IsBindableEvent,

		// [ClassMetadata] [PropertyMetadata] Specifies the base class by which to filter available entry classes within DynamicEntryBox and any ListViewBase.
		// EntryClass, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the property section)

		// [ClassMetadata] [PropertyMetadata] Specifies the base class by which to filter available entry classes within DynamicEntryBox and any ListViewBase.
		//EntryInterface, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the property section)
	};
}




#if WITH_EDITOR

/**
 * Helper macro for binding to a delegate or using the constant value when constructing the underlying SWidget.
 * These macros create a binding that has a layer of indirection that allows blueprint debugging to work more effectively.
 */
#define PROPERTY_BINDING(ReturnType, MemberName)					\
	( MemberName ## Delegate.IsBound() && !IsDesignTime() )			\
	?																\
		BIND_UOBJECT_ATTRIBUTE(ReturnType, K2_Gate_ ## MemberName)	\
	:																\
		TAttribute< ReturnType >(MemberName)

#define BITFIELD_PROPERTY_BINDING(MemberName)						\
	( MemberName ## Delegate.IsBound() && !IsDesignTime() )			\
	?																\
		BIND_UOBJECT_ATTRIBUTE(bool, K2_Gate_ ## MemberName)		\
	:																\
		TAttribute< bool >(MemberName != 0)

#define PROPERTY_BINDING_IMPLEMENTATION(ReturnType, MemberName)			\
	ReturnType K2_Cache_ ## MemberName;									\
	ReturnType K2_Gate_ ## MemberName()									\
	{																	\
		if (CanSafelyRouteEvent())										\
		{																\
			K2_Cache_ ## MemberName = TAttribute< ReturnType >::Create(MemberName ## Delegate.GetUObject(), MemberName ## Delegate.GetFunctionName()).Get(); \
		}																\
																		\
		return K2_Cache_ ## MemberName;									\
	}

#else

#define PROPERTY_BINDING(ReturnType, MemberName)				\
	( MemberName ## Delegate.IsBound() && !IsDesignTime() )		\
	?															\
		TAttribute< ReturnType >::Create(MemberName ## Delegate.GetUObject(), MemberName ## Delegate.GetFunctionName()) \
	:															\
		TAttribute< ReturnType >(MemberName)

#define BITFIELD_PROPERTY_BINDING(MemberName)					\
	( MemberName ## Delegate.IsBound() && !IsDesignTime() )		\
	?															\
		TAttribute< bool >::Create(MemberName ## Delegate.GetUObject(), MemberName ## Delegate.GetFunctionName()) \
	:															\
		TAttribute< bool >(MemberName != 0)

#define PROPERTY_BINDING_IMPLEMENTATION(Type, MemberName) 

#endif

#define GAME_SAFE_OPTIONAL_BINDING(ReturnType, MemberName) PROPERTY_BINDING(ReturnType, MemberName)
#define GAME_SAFE_BINDING_IMPLEMENTATION(ReturnType, MemberName) PROPERTY_BINDING_IMPLEMENTATION(ReturnType, MemberName)

/**
 * Helper macro for binding to a delegate or using the constant value when constructing the underlying SWidget,
 * also allows a conversion function to be provided to convert between the SWidget value and the value exposed to UMG.
 */
#define OPTIONAL_BINDING_CONVERT(ReturnType, MemberName, ConvertedType, ConversionFunction) \
		( MemberName ## Delegate.IsBound() && !IsDesignTime() )								\
		?																					\
			TAttribute< ConvertedType >::Create(TAttribute< ConvertedType >::FGetter::CreateUObject(this, &ThisClass::ConversionFunction, TAttribute< ReturnType >::Create(MemberName ## Delegate.GetUObject(), MemberName ## Delegate.GetFunctionName()))) \
		:																					\
			ConversionFunction(TAttribute< ReturnType >(MemberName))



/**
 * Flags used by the widget designer.
 */
UENUM()
enum class EWidgetDesignFlags : uint8
{
	None				= 0,
	Designing			= 1 << 0,
	ShowOutline			= 1 << 1,
	ExecutePreConstruct	= 1 << 2,
	Previewing			= 1 << 3
};

ENUM_CLASS_FLAGS(EWidgetDesignFlags);


#if WITH_EDITOR

/**
 * Event args that are sent whenever the designer is changed in some big way, allows for more accurate previews for
 * widgets that need to anticipate things about the size of the screen, or other similar device factors.
 */
struct FDesignerChangedEventArgs
{
public:
	FDesignerChangedEventArgs()
		: bScreenPreview(false)
		, Size(0, 0)
		, DpiScale(1.0f)
	{
	}

public:
	bool bScreenPreview;
	FVector2D Size;
	float DpiScale;
};

#endif


/**
 * This is the base class for all wrapped Slate controls that are exposed to UObjects.
 */
UCLASS(Abstract, BlueprintType, Blueprintable, CustomFieldNotify, MinimalAPI)
class UWidget : public UVisual, public INotifyFieldValueChanged
{
	GENERATED_UCLASS_BODY()

	friend UGameViewportSubsystem;

public:
	UE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_BASE_BEGIN(UMG_API)
		UE_FIELD_NOTIFICATION_DECLARE_FIELD(ToolTipText, UMG_API)
		UE_FIELD_NOTIFICATION_DECLARE_FIELD(Visibility, UMG_API)
		UE_FIELD_NOTIFICATION_DECLARE_FIELD(bIsEnabled, UMG_API)
		UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD_BEGIN(ToolTipText)
		UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD(Visibility)
		UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD(bIsEnabled)
		UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD_END()
	UE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_BASE_END();

	// Common Bindings - If you add any new common binding, you must provide a UPropertyBinding for it.
	//                   all primitive binding in UMG goes through native binding evaluators to prevent
	//                   thunking through the VM.
	DECLARE_DYNAMIC_DELEGATE_RetVal(bool, FGetBool);
	DECLARE_DYNAMIC_DELEGATE_RetVal(float, FGetFloat);
	DECLARE_DYNAMIC_DELEGATE_RetVal(int32, FGetInt32);
	DECLARE_DYNAMIC_DELEGATE_RetVal(FText, FGetText);
	DECLARE_DYNAMIC_DELEGATE_RetVal(FSlateColor, FGetSlateColor);
	DECLARE_DYNAMIC_DELEGATE_RetVal(FLinearColor, FGetLinearColor);
	DECLARE_DYNAMIC_DELEGATE_RetVal(FSlateBrush, FGetSlateBrush);
	DECLARE_DYNAMIC_DELEGATE_RetVal(ESlateVisibility, FGetSlateVisibility);
	DECLARE_DYNAMIC_DELEGATE_RetVal(EMouseCursor::Type, FGetMouseCursor);
	DECLARE_DYNAMIC_DELEGATE_RetVal(ECheckBoxState, FGetCheckBoxState);
	DECLARE_DYNAMIC_DELEGATE_RetVal(UWidget*, FGetWidget);

	// Events
	DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(UWidget*, FGenerateWidgetForString, FString, Item);
	DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(UWidget*, FGenerateWidgetForObject, UObject*, Item);

	// Events
	DECLARE_DYNAMIC_DELEGATE_RetVal(FEventReply, FOnReply);
	DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(FEventReply, FOnPointerEvent, FGeometry, MyGeometry, const FPointerEvent&, MouseEvent);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWidgetStateBroadcast, UWidget* /*InWidget*/, const FWidgetStateBitfield& /*InStateBitfield*/);

	typedef TFunctionRef<TSharedPtr<SObjectWidget>( UUserWidget*, TSharedRef<SWidget> )> ConstructMethodType;

	/**
	 * The parent slot of the UWidget.  Allows us to easily inline edit the layout controlling this widget.
	 */
	UPROPERTY(Instanced, TextExportTransient, EditAnywhere, BlueprintReadOnly, Category=Layout, meta=(ShowOnlyInnerProperties))
	TObjectPtr<UPanelSlot> Slot;

	/** A bindable delegate for bIsEnabled */
	UPROPERTY()
	FGetBool bIsEnabledDelegate;

	/** A bindable delegate for ToolTipText */
	UPROPERTY()
	FGetText ToolTipTextDelegate;

	UE_DEPRECATED(5.1, "Direct access to ToolTipText is deprecated. Please use the getter or setter.")
	/** Tooltip text to show when the user hovers over the widget with the mouse */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetToolTipText", Category="Behavior", meta=(MultiLine=true))
	FText ToolTipText;

	UE_DEPRECATED(5.1, "Direct access to ToolTipWidget is deprecated. Please use the getter or setter.")
	/** Tooltip widget to show when the user hovers over the widget with the mouse */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Getter="GetToolTip", Setter="SetToolTip", BlueprintSetter="SetToolTip", Category="Behavior", AdvancedDisplay)
	TObjectPtr<UWidget> ToolTipWidget;

	/** A bindable delegate for ToolTipWidget */
	UPROPERTY()
	FGetWidget ToolTipWidgetDelegate;


	/** A bindable delegate for Visibility */
	UPROPERTY()
	FGetSlateVisibility VisibilityDelegate;;

public:

	UE_DEPRECATED(5.1, "Direct access to RenderTransform is deprecated. Please use the getter or setter.")
	/** The render transform of the widget allows for arbitrary 2D transforms to be applied to the widget. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetRenderTransform", Category="Render Transform", meta = (DisplayName = "Transform"))
	FWidgetTransform RenderTransform;

	UE_DEPRECATED(5.1, "Direct access to RenderTransformPivot is deprecated. Please use the getter or setter.")
	/**
	 * The render transform pivot controls the location about which transforms are applied.  
	 * This value is a normalized coordinate about which things like rotations will occur.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetRenderTransformPivot", Category="Render Transform", meta=( DisplayName="Pivot" ))
	FVector2D RenderTransformPivot;

	UE_DEPRECATED(5.1, "Direct access to FlowDirectionPreference is deprecated. Please use the getter or setter.")
	/** Allows you to set a new flow direction */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Localization")
	EFlowDirectionPreference FlowDirectionPreference;

	/**
	 * Allows controls to be exposed as variables in a blueprint.  Not all controls need to be exposed
	 * as variables, so this allows only the most useful ones to end up being exposed.
	 */
	UPROPERTY()
	uint8 bIsVariable:1;

	/** Flag if the Widget was created from a blueprint */
	UPROPERTY(Transient)
	uint8 bCreatedByConstructionScript:1;

	UE_DEPRECATED(5.1, "Direct access to bIsEnabled is deprecated. Please use the getter or setter.")
	/** Sets whether this widget can be modified interactively by the user */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, FieldNotify, Getter="GetIsEnabled", Setter="SetIsEnabled", BlueprintGetter="GetIsEnabled", BlueprintSetter="SetIsEnabled", Category="Behavior")
	uint8 bIsEnabled:1;

	/**  */
	UPROPERTY(EditAnywhere, Category="Behavior", meta=(InlineEditConditionToggle))
	uint8 bOverride_Cursor : 1;

#if WITH_EDITORONLY_DATA
	// These editor-only properties exist for two reasons:
	//   1. To make details customization easier to write, specifically in regards to the binding extension widget
	//   2. To allow subclasses to set their default values without having to subclass USlateAccessibleWidgetData
	// Every time one of these properties changes, it's data is propagated to AccessibleWidgetData if it exists.
	// The creations of AccessibleWidgetData is controlled by the details customization through a CheckBox.
	// The reason this is set up like this is to reduce the memory footprint of UWidget since overriding the default
	// accessibility rules for a particular widget will be relatively rare. In a shipped game, if no custom rules
	// are defined, there will only be the memory cost of the UObject pointer.
	//
	// IMPORTANT: Any user-editable variables added to USlateAccessibleWidgetData should be duplicated here as well.
	//            Additionally, its edit condition must be manually assigned in UMGDetailCustomizations.

	/** Override all of the default accessibility behavior and text for this widget. */
	UPROPERTY(EditAnywhere, Category="Accessibility")
	uint8 bOverrideAccessibleDefaults : 1;

	/** Whether or not children of this widget can appear as distinct accessible widgets. */
	UPROPERTY(EditAnywhere, Category="Accessibility", meta=(EditCondition="bOverrideAccessibleDefaults"))
	uint8 bCanChildrenBeAccessible : 1;

	/** Whether or not the widget is accessible, and how to describe it. If set to custom, additional customization options will appear. */
	UPROPERTY(EditAnywhere, Category="Accessibility", meta=(EditCondition="bOverrideAccessibleDefaults"))
	ESlateAccessibleBehavior AccessibleBehavior;

	/** How to describe this widget when it's being presented through a summary of a parent widget. If set to custom, additional customization options will appear. */
	UPROPERTY(EditAnywhere, Category="Accessibility", AdvancedDisplay, meta=(EditCondition="bOverrideAccessibleDefaults"))
	ESlateAccessibleBehavior AccessibleSummaryBehavior;

	/** When AccessibleBehavior is set to Custom, this is the text that will be used to describe the widget. */
	UPROPERTY(EditAnywhere, Category="Accessibility", meta=(MultiLine=true))
	FText AccessibleText;

	/** An optional delegate that may be assigned in place of AccessibleText for creating a TAttribute */
	UPROPERTY()
	USlateAccessibleWidgetData::FGetText AccessibleTextDelegate;

	/** When AccessibleSummaryBehavior is set to Custom, this is the text that will be used to describe the widget. */
	UPROPERTY(EditAnywhere, Category="Accessibility", meta=(MultiLine=true), AdvancedDisplay)
	FText AccessibleSummaryText;

	/** An optional delegate that may be assigned in place of AccessibleSummaryText for creating a TAttribute */
	UPROPERTY()
	USlateAccessibleWidgetData::FGetText AccessibleSummaryTextDelegate;
#endif

protected:

	/**
	 * If true prevents the widget or its child's geometry or layout information from being cached.  If this widget
	 * changes every frame, but you want it to still be in an invalidation panel you should make it as volatile
	 * instead of invalidating it every frame, which would prevent the invalidation panel from actually
	 * ever caching anything.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Performance")
	uint8 bIsVolatile:1;

	/** Cached value that indicate if the widget was added to the GameViewportSubsystem. */
	uint8 bIsManagedByGameViewportSubsystem:1;

	/** False will skip state broadcasts. Useful for child classes to call Super methods without broadcasting early / late. */
	bool bShouldBroadcastState : 1;

	/** True implies widget state has been initialized. */
	bool bWidgetStateInitialized : 1;

public:
	/** True if the SWidget should be Wrapped in a Component */
	UPROPERTY(Transient)
	uint8 bWrappedByComponent : 1;

public:
#if WITH_EDITORONLY_DATA
	/** Stores the design time flag setting if the widget is hidden inside the designer */
	UPROPERTY()
	uint8 bHiddenInDesigner:1;

	/** Stores the design time flag setting if the widget is expanded inside the designer */
	UPROPERTY()
	uint8 bExpandedInDesigner:1;

	/** Stores the design time flag setting if the widget is locked inside the designer */
	UPROPERTY()
	uint8 bLockedInDesigner:1;
#endif

	UE_DEPRECATED(5.1, "Direct access to Cursor is deprecated. Please use the getter or setter.")
	/** The cursor to show when the mouse is over the widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter="SetCursor", Category="Behavior", AdvancedDisplay, meta = (editcondition = "bOverride_Cursor"))
	TEnumAsByte<EMouseCursor::Type> Cursor;

	UE_DEPRECATED(5.1, "Direct access to Clipping is deprecated. Please use the getter or setter.")
	/**
	 * Controls how the clipping behavior of this widget.  Normally content that overflows the
	 * bounds of the widget continues rendering.  Enabling clipping prevents that overflowing content
	 * from being seen.
	 *
	 * NOTE: Elements in different clipping spaces can not be batched together, and so there is a
	 * performance cost to clipping.  Do not enable clipping unless a panel actually needs to prevent
	 * content from showing up outside its bounds.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category="Rendering")
	EWidgetClipping Clipping;

	UE_DEPRECATED(5.1, "Direct access to Visibility is deprecated. Please use the getter or setter.")
	/** The visibility of the widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, FieldNotify, Getter, Setter, BlueprintGetter="GetVisibility", BlueprintSetter="SetVisibility", Category="Behavior")
	ESlateVisibility Visibility;
	
private:
	/** If the widget will draw snapped to the nearest pixel.  Improves clarity but might cause visibile stepping in animation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Rendering", meta=(AllowPrivateAccess = true))
	EWidgetPixelSnapping PixelSnapping;

protected:

	UE_DEPRECATED(5.1, "Direct access to RenderOpacity is deprecated. Please use the getter or setter.")
	/** The opacity of the widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintGetter="GetRenderOpacity", BlueprintSetter="SetRenderOpacity", Category="Rendering", meta = (UIMin = "0", UIMax = "1"))
	float RenderOpacity;

private:
	/** A custom set of accessibility rules for this widget. If null, default rules for the widget are used. */
	UPROPERTY(Instanced)
	TObjectPtr<USlateAccessibleWidgetData> AccessibleWidgetData;

public:
	/**
	 * The navigation object for this widget is optionally created if the user has configured custom
	 * navigation rules for this widget in the widget designer.  Those rules determine how navigation transitions
	 * can occur between widgets.
	 */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadOnly, Category="Navigation")
	TObjectPtr<class UWidgetNavigation> Navigation;

#if WITH_EDITORONLY_DATA

	/** Stores a reference to the asset responsible for this widgets construction. */
	TWeakObjectPtr<UObject> WidgetGeneratedBy;
	
#endif

public:

#if WITH_EDITOR

	/** Is this widget locked in the designer UI */
	bool IsLockedInDesigner() const
	{
		return bLockedInDesigner;
	}

	/** @param bLockedInDesigner should this widget be locked */
	virtual void SetLockedInDesigner(bool NewLockedInDesigner)
	{
		bLockedInDesigner = NewLockedInDesigner;
	}

#endif

#if UE_HAS_WIDGET_GENERATED_BY_CLASS
	/** Stores a reference to the class responsible for this widgets construction. */
	TWeakObjectPtr<UClass> WidgetGeneratedByClass;
#endif

public:
	/** */
	UMG_API const FWidgetTransform& GetRenderTransform() const;

	/** */
	UFUNCTION(BlueprintCallable, Category="Widget|Transform")
	UMG_API void SetRenderTransform(FWidgetTransform InTransform);

	/** */
	UFUNCTION(BlueprintCallable, Category="Widget|Transform")
	UMG_API void SetRenderScale(FVector2D Scale);

	/** */
	UFUNCTION(BlueprintCallable, Category="Widget|Transform")
	UMG_API void SetRenderShear(FVector2D Shear);

	/** */
	UFUNCTION(BlueprintCallable, Category="Widget|Transform")
	UMG_API void SetRenderTransformAngle(float Angle);
	
	/** */
	UFUNCTION(BlueprintCallable, Category = "Widget|Transform")
	UMG_API float GetRenderTransformAngle() const;
	
	/** */
	UFUNCTION(BlueprintCallable, Category="Widget|Transform")
	UMG_API void SetRenderTranslation(FVector2D Translation);

	/** */
	UMG_API FVector2D GetRenderTransformPivot() const;

	/** */
	UFUNCTION(BlueprintCallable, Category="Widget|Transform")
	UMG_API void SetRenderTransformPivot(FVector2D Pivot);

	/** Gets the flow direction preference of the widget */
	UMG_API EFlowDirectionPreference GetFlowDirectionPreference() const;

	/** Sets the flow direction preference of the widget */
	UMG_API void SetFlowDirectionPreference(EFlowDirectionPreference FlowDirection);

	/** Gets the current enabled status of the widget */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool GetIsEnabled() const;

	/** Sets the current enabled status of the widget */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API virtual void SetIsEnabled(bool bInIsEnabled);

	/* @return true if the widget was added to the viewport using AddToViewport or AddToPlayerScreen. */
	UFUNCTION(BlueprintPure, BlueprintCosmetic, Category = "Appearance")
	UMG_API bool IsInViewport() const;

	/** @return the tooltip text for the widget. */
	UMG_API FText GetToolTipText() const;

	/** Sets the tooltip text for the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetToolTipText(const FText& InToolTipText);

	/** @return the custom widget as the tooltip of the widget. */
	UMG_API UWidget* GetToolTip() const;

	/** Sets a custom widget as the tooltip of the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetToolTip(UWidget* Widget);

	/** Sets the cursor to show over the widget. */
	UMG_API EMouseCursor::Type GetCursor() const;

	/** Sets the cursor to show over the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetCursor(EMouseCursor::Type InCursor);

	/** Resets the cursor to use on the widget, removing any customization for it. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void ResetCursor();
	
	/** Returns true if the widget is Visible, HitTestInvisible or SelfHitTestInvisible and the Render Opacity is greater than 0. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool IsRendered() const;

	/** Returns true if the widget is Visible, HitTestInvisible or SelfHitTestInvisible. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool IsVisible() const;

	/** Gets the current visibility of the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API ESlateVisibility GetVisibility() const;

	/** Sets the visibility of the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API virtual void SetVisibility(ESlateVisibility InVisibility);

protected:
	UMG_API void SetVisibilityInternal(ESlateVisibility InVisibility);

public:
	/** Gets the current visibility of the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API float GetRenderOpacity() const;

	/** Sets the visibility of the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetRenderOpacity(float InOpacity);

	/** Gets the clipping state of this widget. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API EWidgetClipping GetClipping() const;

	/** Sets the clipping state of this widget. */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetClipping(EWidgetClipping InClipping);
	
	/** Gets the pixel snapping method of this widget. */
	UMG_API EWidgetPixelSnapping GetPixelSnapping() const;

	/** Sets the pixel snapping method of this widget. */
	UMG_API void SetPixelSnapping(EWidgetPixelSnapping InPixelSnapping);

	/** Sets the forced volatility of the widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void ForceVolatile(bool bForce);

	/** Returns true if the widget is currently being hovered by a pointer device */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API virtual bool IsHovered() const;

	/**
	 * Checks to see if this widget currently has the keyboard focus
	 *
	 * @return  True if this widget has keyboard focus
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool HasKeyboardFocus() const;

	/**
	* Checks to see if this widget supports keyboard focus
	*
	* @return  True if this widget supports keyboard focus
	*/
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool SupportsKeyboardFocus() const;

	/**
	 * Checks to see if this widget is the current mouse captor
	 * @return  True if this widget has captured the mouse
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool HasMouseCapture() const;

	/**
	 * Checks to see if this widget is the current mouse captor
	 *	@param User index to check for capture
	 *	@param Optional pointer index to check for capture
	 *	@return  True if this widget has captured the mouse with given user and pointer
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API bool HasMouseCaptureByUser(int32 UserIndex, int32 PointerIndex = -1) const;

	/** Sets the focus to this widget. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetKeyboardFocus();

	/** Returns true if this widget is focused by a specific user. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool HasUserFocus(APlayerController* PlayerController) const;

	/** Returns true if this widget is focused by any user. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool HasAnyUserFocus() const;

	/** Returns true if any descendant widget is focused by any user. */
	UFUNCTION(BlueprintCallable, Category="Widget", meta=(DisplayName="HasAnyUserFocusedDescendants"))
	UMG_API bool HasFocusedDescendants() const;

	/** Returns true if any descendant widget is focused by a specific user. */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API bool HasUserFocusedDescendants(APlayerController* PlayerController) const;
	
	/** Sets the focus to this widget for the owning user */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetFocus();

	/** Sets the focus to this widget for a specific user (if setting focus for the owning user, prefer SetFocus()) */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void SetUserFocus(APlayerController* PlayerController);

	/**
	 * Forces a pre-pass.  A pre-pass caches the desired size of the widget hierarchy owned by this widget.  
	 * One pre-pass already happens for every widget before Tick occurs.  You only need to perform another 
	 * pre-pass if you are adding child widgets this frame and want them to immediately be visible this frame.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void ForceLayoutPrepass();

	/**
	 * Invalidates the widget from the view of a layout caching widget that may own this widget.
	 * will force the owning widget to redraw and cache children on the next paint pass.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API void InvalidateLayoutAndVolatility();

	/**
	 * Gets the widgets desired size.
	 * NOTE: The underlying Slate widget must exist and be valid, also at least one pre-pass must
	 *       have occurred before this value will be of any use.
	 * 
	 * @return The widget's desired size
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API FVector2D GetDesiredSize() const;

	/**
	 *	Sets the widget navigation rules for all directions. This can only be called on widgets that are in a widget tree.
	 *	@param Rule The rule to use when navigation is taking place
	 *	@param WidgetToFocus When using the Explicit rule, focus on this widget
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetAllNavigationRules(EUINavigationRule Rule, FName WidgetToFocus);

	/**
	 *	Sets the widget navigation rules for a specific direction. This can only be called on widgets that are in a widget tree.
	 *	@param Direction
	 *	@param Rule The rule to use when navigation is taking place
	 *	@param WidgetToFocus When using the Explicit rule, focus on this widget
	 */
	UE_DEPRECATED(4.23, "SetNavigationRule is deprecated. Please use either SetNavigationRuleBase or SetNavigationRuleExplicit or SetNavigationRuleCustom or SetNavigationRuleCustomBoundary.")
	UFUNCTION(BlueprintCallable, Category = "Widget", meta = (DeprecatedFunction, DeprecatedMessage = "Please use either SetNavigationRuleBase or SetNavigationRuleExplicit or SetNavigationRuleCustom or SetNavigationRuleCustomBoundary."))
	UMG_API void SetNavigationRule(EUINavigation Direction, EUINavigationRule Rule, FName WidgetToFocus);

	/**
	 *	Sets the widget navigation rules for a specific direction. This can only be called on widgets that are in a widget tree. This works only for non Explicit, non Custom and non CustomBoundary Rules.
	 *	@param Direction
	 *	@param Rule The rule to use when navigation is taking place
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetNavigationRuleBase(EUINavigation Direction, EUINavigationRule Rule);

	/**
	 *	Sets the widget navigation rules for a specific direction. This can only be called on widgets that are in a widget tree. This works only for Explicit Rule.
	 *	@param Direction
	 *	@param InWidget Focus on this widget instance
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetNavigationRuleExplicit(EUINavigation Direction, UWidget* InWidget);

	/**
	 *	Sets the widget navigation rules for a specific direction. This can only be called on widgets that are in a widget tree. This works only for Custom Rule.
	 *	@param Direction
	 *	@param InCustomDelegate Custom Delegate that will be called
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetNavigationRuleCustom(EUINavigation Direction, FCustomWidgetNavigationDelegate InCustomDelegate);

	/**
	 *	Sets the widget navigation rules for a specific direction. This can only be called on widgets that are in a widget tree. This works only for CustomBoundary Rule.
	 *	@param Direction
	 *	@param InCustomDelegate Custom Delegate that will be called
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API void SetNavigationRuleCustomBoundary(EUINavigation Direction, FCustomWidgetNavigationDelegate InCustomDelegate);

	/**
	 *	Sets the widget navigation method for this widget hierarchy. This can only be called on widgets that are in a widget tree. 
	 *	@param NavigationMethodStruct. Must be the StaticStruct of type FNavigationMethod
	 */
	UMG_API void SetNavigationMethod(const TInstancedStruct<FNavigationMethod>& NavigationMethod);

	/** Gets the parent widget */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API class UPanelWidget* GetParent() const;

	/**
	 * Removes the widget from its parent widget.  If this widget was added to the player's screen or the viewport
	 * it will also be removed from those containers.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API virtual void RemoveFromParent();

	/**
	 * Gets the last geometry used to Tick the widget.  This data may not exist yet if this call happens prior to 
	 * the widget having been ticked/painted, or it may be out of date, or a frame behind.
	 *
	 * We recommend not to use this data unless there's no other way to solve your problem.  Normally in Slate we
	 * try and handle these issues by making a dependent widget part of the hierarchy, as to avoid frame behind
	 * or what are referred to as hysteresis problems, both caused by depending on geometry from the previous frame
	 * being used to advise how to layout a dependent object the current frame.
	 */
	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API const FGeometry& GetCachedGeometry() const;

	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API const FGeometry& GetTickSpaceGeometry() const;

	UFUNCTION(BlueprintCallable, Category="Widget")
	UMG_API const FGeometry& GetPaintSpaceGeometry() const;

	//~ Begin INotifyFieldValueChanged Interface
public:
	UMG_API virtual FDelegateHandle AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate) override final;
	UMG_API virtual bool RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle) override final;
	UMG_API virtual int32 RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject) override final;
	UMG_API virtual int32 RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject) override final;
	UMG_API virtual void BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId) override final;
	//~ End INotifyFieldValueChanged Interface

	UFUNCTION(BlueprintCallable, Category = "FieldNotify", meta = (DisplayName = "Add Field Value Changed Delegate", ScriptName = "AddFieldValueChangedDelegate"))
	UMG_API void K2_AddFieldValueChangedDelegate(FFieldNotificationId FieldId, FFieldValueChangedDynamicDelegate Delegate);

	UFUNCTION(BlueprintCallable, Category = "FieldNotify", meta = (DisplayName = "Remove Field Value Changed Delegate", ScriptName="RemoveFieldValueChangedDelegate"))
	UMG_API void K2_RemoveFieldValueChangedDelegate(FFieldNotificationId FieldId, FFieldValueChangedDynamicDelegate Delegate);

protected:
	UFUNCTION(BlueprintCallable, Category="FieldNotify", meta = (DisplayName="Broadcast Field Value Changed", ScriptName="BroadcastFieldValueChanged"))
	UMG_API void K2_BroadcastFieldValueChanged(FFieldNotificationId FieldId);

public:
	/**
	 * Gets the underlying slate widget or constructs it if it doesn't exist.  If you're looking to replace
	 * what slate widget gets constructed look for RebuildWidget.  For extremely special cases where you actually
	 * need to change the GC Root widget of the constructed User Widget - you need to use TakeDerivedWidget
	 * you must also take care to not call TakeWidget before calling TakeDerivedWidget, as that would put the wrong
	 * expected wrapper around the resulting widget being constructed.
	 */
	UMG_API TSharedRef<SWidget> TakeWidget();

	/**
	 * Gets the underlying slate widget or constructs it if it doesn't exist.
	 * 
	 * @param ConstructMethod allows the caller to specify a custom constructor that will be provided the 
	 *						  default parameters we use to construct the slate widget, this allows the caller 
	 *						  to construct derived types of SObjectWidget in cases where additional 
	 *						  functionality at the slate level is desirable.  
	 * @example
	 *		class SObjectWidgetCustom : public SObjectWidget, public IMixinBehavior
	 *      { }
	 * 
	 *      Widget->TakeDerivedWidget<SObjectWidgetCustom>( []( UUserWidget* Widget, TSharedRef<SWidget> Content ) {
	 *			return SNew( SObjectWidgetCustom, Widget ) 
	 *					[
	 *						Content
	 *					];
	 *		});
	 * 
	 */
	template <class WidgetType = SObjectWidget>
	TSharedRef<WidgetType> TakeDerivedWidget(ConstructMethodType ConstructMethod)
	{
		static_assert(TIsDerivedFrom<WidgetType, SObjectWidget>::IsDerived, "TakeDerivedWidget can only be used to create SObjectWidget instances");
		return StaticCastSharedRef<WidgetType>(TakeWidget_Private(ConstructMethod));
	}
	
private:
	UMG_API TSharedRef<SWidget> TakeWidget_Private( ConstructMethodType ConstructMethod );

public:

	/** Gets the last created widget from RebuildWidget does not recreate the gc container for the widget if one is needed. */
	UMG_API TSharedPtr<SWidget> GetCachedWidget() const;

	/** Gets the last created widget from RebuildDesignWidget does not recreate the gc container for the widget if one is needed. */
	UMG_API TSharedPtr<SWidget> GetCachedWrappedWidget() const;


	/** Gets the last created widget does not recreate the gc container for the widget if one is needed. */
	UMG_API bool IsConstructed() const;

	/**
	 * Gets the game instance associated with this UI.
	 * @return a pointer to the owning game instance
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Widget")
	UMG_API UGameInstance* GetGameInstance() const;

	/**
	 * Gets the game instance associated with this UI.
	 * @return a pointer to the owning game instance
	 */
	template <class TGameInstance = UGameInstance>
	TGameInstance* GetGameInstance() const
	{
		return Cast<TGameInstance>(GetGameInstance());
	}

	/**
	 * Gets the player controller associated with this UI.
	 * @return The player controller that owns the UI.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Widget")
	UMG_API virtual APlayerController* GetOwningPlayer() const;

	/**
	 * Gets the player controller associated with this UI cast to the template type.
	 * @return The player controller that owns the UI. May be NULL if the cast fails.
	 */
	template <class TPlayerController = APlayerController >
	TPlayerController* GetOwningPlayer() const
	{
		return Cast<TPlayerController>(GetOwningPlayer());
	}

	/**
	 * Gets the local player associated with this UI.
	 * @return The owning local player.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Widget")
	UMG_API virtual ULocalPlayer* GetOwningLocalPlayer() const;
	
	/**
	 * Gets the local player associated with this UI cast to the template type.
	 * @return The owning local player. May be NULL if the cast fails.
	 */
	template < class T >
	T* GetOwningLocalPlayer() const
	{
		return Cast<T>(GetOwningLocalPlayer());
	}

	/**
	 * Gets the accessible text from the underlying Slate accessible widget
	 * @return The accessible text of the underlying Slate accessible widget. Returns an empty text if
	  * accessibility is dsabled or the underlying accessible widget is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API FText GetAccessibleText() const;

	/**
	 * Gets the accessible summary text from the underlying Slate accessible widget.
	 * @return The accessible summary text of the underlying Slate accessible widget. Returns an empty text if
	  * accessibility is dsabled or the underlying accessible widget is invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Widget")
	UMG_API FText GetAccessibleSummaryText() const;
	
	/**
	 * Applies all properties to the native widget if possible.  This is called after a widget is constructed.
	 * It can also be called by the editor to update modified state, so ensure all initialization to a widgets
	 * properties are performed here, or the property and visual state may become unsynced.
	 */
	UMG_API virtual void SynchronizeProperties();

	/**
	 * Called by the owning user widget after the slate widget has been created.  After the entire widget tree
	 * has been initialized, any widget reference that was needed to support navigating to another widget will
	 * now be initialized and ready for usage.
	 */
	UMG_API void BuildNavigation();

#if WITH_EDITOR
	/** Returns if the widget is currently being displayed in the designer, it may want to display different data. */
	inline bool IsDesignTime() const
	{
		return HasAnyDesignerFlags(EWidgetDesignFlags::Designing);
	}

	/** Sets the designer flags on the widget. */
	UMG_API virtual void SetDesignerFlags(EWidgetDesignFlags NewFlags);

	/** Gets the designer flags currently set on the widget. */
	inline EWidgetDesignFlags GetDesignerFlags() const
	{
		return static_cast<EWidgetDesignFlags>(DesignerFlags);
	}

	/** Tests if any of the flags exist on this widget. */
	inline bool HasAnyDesignerFlags(EWidgetDesignFlags FlagsToCheck) const
	{
		return EnumHasAnyFlags(GetDesignerFlags(), FlagsToCheck);
	}
	
	inline bool IsPreviewTime() const 
	{
		return HasAnyDesignerFlags(EWidgetDesignFlags::Previewing);
	}

	/** Returns the friendly name of the widget to display in the editor */
	const FString& GetDisplayLabel() const
	{
		return DisplayLabel;
	}

	/** Sets the friendly name of the widget to display in the editor */
	UMG_API void SetDisplayLabel(const FString& DisplayLabel);

	/** Returns the category name of the widget */
	UMG_API const FString& GetCategoryName() const;

	/** Sets the category name of the widget */
	UMG_API void SetCategoryName(const FString& InValue);

	/**
	 * Called at the end of Widget Blueprint compilation.
	 * Allows UMG elements to evaluate their default states and determine whether they are acceptable.
	 * To trigger compilation failure, add an error to the log. Warnings and notes will be visible, but will not cause compiles to fail.
	 */
	virtual void ValidateCompiledDefaults(class IWidgetCompilerLog& CompileLog) const {}

	/** Mark this object as modified, also mark the slot as modified. */
	UMG_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#else
	inline bool IsDesignTime() const { return false; }
	inline bool IsPreviewTime() const { return false; }
#endif
	
	/**
	 * Recurses up the list of parents and returns true if this widget is a descendant of the PossibleParent
	 * @return true if this widget is a child of the PossibleParent
	 */
	UMG_API bool IsChildOf(UWidget* PossibleParent);

	/**  */
	UMG_API bool AddBinding(FDelegateProperty* DelegateProperty, UObject* SourceObject, const FDynamicPropertyPath& BindingPath);

	/**
	 * Add a post-state-changed listener to this widget, will fire after a state changed and all related side effects are resolved.
	 * 
	 * Note: Currently we only support post-state-changed broadcasts.
	 *
	 * @param ListenerDelegate Delegate to fire when state changes
	 * @param bBroadcastCurrentState true if we should trigger this delegate once on registration with the current widget state (Does not globally broadcast).
	 */
	UMG_API FDelegateHandle RegisterPostStateListener(const FOnWidgetStateBroadcast::FDelegate& ListenerDelegate, bool bBroadcastCurrentState = true);

	/**
	 * Remove a post-state-changed  listener from this widget, resets state bitfield if no other state listeners exist
	 *
	 * @param ListenerDelegate Delegate to remove
	 */
	UMG_API void UnregisterPostStateListener(const FDelegateHandle& ListenerDelegate);

	static UMG_API TSubclassOf<UPropertyBinding> FindBinderClassForDestination(FProperty* Property);

	//~ Begin UObject
	UMG_API virtual UWorld* GetWorld() const override;
	UMG_API virtual void BeginDestroy() override;
	UMG_API virtual void FinishDestroy() override;
	UMG_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	//~ End UObject

	inline bool CanSafelyRouteEvent()
	{
		return !IsDesignTime() && CanSafelyRouteCall();
	}

	inline bool CanSafelyRoutePaint()
	{
		return CanSafelyRouteCall();
	}

protected:
	inline bool CanSafelyRouteCall() { return !(GIntraFrameDebuggingGameThread || IsUnreachable() || FUObjectThreadContext::Get().IsRoutingPostLoad); }

public:

#if WITH_EDITOR

	/** Is the label generated or provided by the user? */
	UMG_API bool IsGeneratedName() const;

	/** Get Label Metadata, which may be as simple as a bit of string data to help identify an anonymous text block. */
	UMG_API virtual FString GetLabelMetadata() const;

	/** Gets the label to display to the user for this widget. */
	UMG_API FText GetLabelText() const;

	/** Gets the label to display to the user for this widget, including any extra metadata like the text string for text. */
	UMG_API FText GetLabelTextWithMetadata() const;

	/** Gets the palette category of the widget */
	UMG_API virtual const FText GetPaletteCategory();

	/** Called by the palette after constructing a new widget. */
	UMG_API void CreatedFromPalette();

	/**
	 * Called after constructing a new widget from the palette.
	 * Allows the widget to perform interesting default setup that we don't want to be UObject Defaults.
	 */
	virtual void OnCreationFromPalette() { }

	/** Allows general fixups and connections only used at editor time. */
	virtual void ConnectEditorData() { }

	// UObject interface
	UMG_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	/** Is the widget visible in the designer?  If this widget is 'hidden in the designer' or a parent is, this widget will also return false here. */
	UMG_API bool IsVisibleInDesigner() const;

	// Begin Designer contextual events
	UMG_API void SelectByDesigner();
	UMG_API void DeselectByDesigner();

	virtual void OnDesignerChanged(const FDesignerChangedEventArgs& EventArgs) { }

	virtual void OnSelectedByDesigner() { }
	virtual void OnDeselectedByDesigner() { }

	virtual void OnDescendantSelectedByDesigner(UWidget* DescendantWidget) { }
	virtual void OnDescendantDeselectedByDesigner(UWidget* DescendantWidget) { }

	virtual void OnBeginEditByDesigner() { }
	virtual void OnEndEditByDesigner() { }
	// End Designer contextual events
#endif

	// Utility methods
	//@TODO UMG: Should move elsewhere
	static UMG_API EVisibility ConvertSerializedVisibilityToRuntime(ESlateVisibility Input);
	static UMG_API ESlateVisibility ConvertRuntimeToSerializedVisibility(const EVisibility& Input);

	static UMG_API FSizeParam ConvertSerializedSizeParamToRuntime(const FSlateChildSize& Input);

	static UMG_API UWidget* FindChildContainingDescendant(UWidget* Root, UWidget* Descendant);

	static UMG_API FString GetDefaultFontName();

protected:
#if WITH_EDITOR
	// This is an implementation detail that allows us to show and hide the widget in the designer
	// regardless of the actual visibility state set by the user.
	UMG_API EVisibility GetVisibilityInDesigner() const;

	UMG_API bool IsEditorWidget() const;
#endif

	UMG_API virtual void OnBindingChanged(const FName& Property);

	/**
	 * Broadcast a binary state post change
	 *
	 * @param StateChange bitfield marking states that should be changed
	 * @param bInValue true if marked states should be enabled, false otherwise
	 */
	UMG_API void BroadcastBinaryPostStateChange(const FWidgetStateBitfield& StateChange, bool bInValue);

	/**
	 * Broadcast an enum state post change
	 *
	 * @param StateChange bitfield marking states that should be changed
	 */
	UE_DEPRECATED(5.5, "FWidgetStateBitfield currently no longer supports enum states")
	UMG_API void BroadcastEnumPostStateChange(const FWidgetStateBitfield& StateChange);

protected:
	UMG_API UObject* GetSourceAssetOrClass() const;

	/** Function implemented by all subclasses of UWidget is called when the underlying SWidget needs to be constructed. */
	UMG_API virtual TSharedRef<SWidget> RebuildWidget();

	/** Function called after the underlying SWidget is constructed. */
	UMG_API virtual void OnWidgetRebuilt();
	
#if WITH_EDITOR
	UMG_API virtual TSharedRef<SWidget> RebuildDesignWidget(TSharedRef<SWidget> Content);

	UMG_API TSharedRef<SWidget> CreateDesignerOutline(TSharedRef<SWidget> Content) const;

	/** Gets the base name used to generate the display label/name of this widget. */
	UMG_API FText GetDisplayNameBase() const;
#endif

	UMG_API void UpdateRenderTransform();

	/** Copy all accessible properties to the AccessibleWidgetData object */
	UMG_API void SynchronizeAccessibleData();

protected:
	//TODO UMG Consider moving conversion functions into another class.
	// Conversion functions
	EVisibility ConvertVisibility(TAttribute<ESlateVisibility> SerializedType) const
	{
		ESlateVisibility SlateVisibility = SerializedType.Get();
		return ConvertSerializedVisibilityToRuntime(SlateVisibility);
	}

	TOptional<float> ConvertFloatToOptionalFloat(TAttribute<float> InFloat) const
	{
		return InFloat.Get();
	}

	FSlateColor ConvertLinearColorToSlateColor(TAttribute<FLinearColor> InLinearColor) const
	{
		return FSlateColor(InLinearColor.Get());
	}

	UMG_API void SetNavigationRuleInternal(EUINavigation Direction, EUINavigationRule Rule, FName WidgetToFocus = NAME_None, UWidget* InWidget = nullptr, FCustomWidgetNavigationDelegate InCustomDelegate = FCustomWidgetNavigationDelegate());

#if WITH_ACCESSIBILITY
	/** Gets the widget that accessibility properties should synchronize to. */
	UMG_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const;
#endif

protected:
	/** The underlying SWidget. */
	TWeakPtr<SWidget> MyWidget;

	/** The underlying SWidget for the Component wrapper widget. */
	TWeakPtr<SWidget> ComponentWrapperWidget;
	
	/** The underlying SWidget contained in a SObjectWidget */
	TWeakPtr<SObjectWidget> MyGCWidget;

	/** The bitfield for this widget's state */
	FWidgetStateBitfield MyWidgetStateBitfield;

	/** Delegate that broadcasts after current widget state has fully changed, including all state-related side effects */
	FOnWidgetStateBroadcast PostWidgetStateChanged;

	/** Native property bindings. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UPropertyBinding>> NativeBindings;

	static UMG_API TArray<TSubclassOf<UPropertyBinding>> BinderClasses;

private:
	TBitArray<> EnabledFieldNotifications;

#if WITH_EDITORONLY_DATA
	/** Any flags used by the designer at edit time. */
	UPROPERTY(Transient)
	uint8 DesignerFlags;

	/** The friendly name for this widget displayed in the designer and BP graph. */
	UPROPERTY()
	FString DisplayLabel;

	/** The underlying SWidget for the design time wrapper widget. */
	TWeakPtr<SWidget> DesignWrapperWidget;

	/** Category name used in the widget designer for sorting purpose */
	UPROPERTY()
	FString CategoryName;
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	UMG_API void VerifySynchronizeProperties();

	/** Did we route the synchronize properties call? */
	bool bRoutedSynchronizeProperties;

#else
	inline void VerifySynchronizeProperties() { }
#endif

private:
	PROPERTY_BINDING_IMPLEMENTATION(FText, ToolTipText);
	PROPERTY_BINDING_IMPLEMENTATION(bool, bIsEnabled);
#if WITH_EDITORONLY_DATA
	PROPERTY_BINDING_IMPLEMENTATION(FText, AccessibleText);
	PROPERTY_BINDING_IMPLEMENTATION(FText, AccessibleSummaryText);
#endif
};
