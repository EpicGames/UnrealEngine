// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Layout/ArrangedWidget.h"
#include "WorldCollision.h"
#include "Components/MeshComponent.h"
#include "Blueprint/UserWidget.h"
#include "WidgetComponent.generated.h"

class FHittestGrid;
class FPrimitiveSceneProxy;
class FWidgetRenderer;
class SVirtualWindow;
class SWindow;
class UBodySetup;
class UMaterialInstanceDynamic;
class UTextureRenderTarget2D;
enum ETextureRenderTargetFormat : int;

UENUM(BlueprintType)
enum class EWidgetSpace : uint8
{
	/** The widget is rendered in the world as mesh, it can be occluded like any other mesh in the world. */
	World,
	/** The widget is rendered in the screen, completely outside of the world, never occluded. */
	Screen
};

UENUM(BlueprintType)
enum class EWidgetTimingPolicy : uint8
{
	/** The widget will tick using real time. When not ticking, real time will accumulate and be simulated on the next tick. */
	RealTime,
	/** The widget will tick using game time, respecting pausing and time dilation. */
	GameTime
};

UENUM(BlueprintType)
enum class EWidgetBlendMode : uint8
{
	Opaque,
	Masked,
	Transparent
};

UENUM(BlueprintType)
enum class EWidgetGeometryMode : uint8
{
	/** The widget is mapped onto a plane */
	Plane,

	/** The widget is mapped onto a cylinder */
	Cylinder
};

UENUM(BlueprintType)
enum class EWindowVisibility : uint8
{
	/** The window visibility is Visible */
	Visible,

	/** The window visibility is SelfHitTestInvisible */
	SelfHitTestInvisible
};

UENUM(BlueprintType)
enum class ETickMode : uint8
{
	/** The component tick is disabled until re-enabled. */
	Disabled,

	/** The component is always ticked */
	Enabled,

	/** The component is ticked only when needed. i.e. when visible.*/
	Automatic
};


/**
 * The widget component provides a surface in the 3D environment on which to render widgets normally rendered to the screen.
 * Widgets are first rendered to a render target, then that render target is displayed in the world.
 * 
 * Material Properties set by this component on whatever material overrides the default.
 * SlateUI [Texture]
 * BackColor [Vector]
 * TintColorAndOpacity [Vector]
 * OpacityFromTexture [Scalar]
 */
UCLASS(Blueprintable, ClassGroup="UserInterface", hidecategories=(Object,Activation,"Components|Activation",Sockets,Base,Lighting,LOD,Mesh), editinlinenew, meta=(BlueprintSpawnableComponent) , MinimalAPI)
class UWidgetComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
	//UObject interface
	UMG_API virtual void Serialize(FArchive& Ar) override;
	UMG_API virtual bool CanBeInCluster() const override;
	UMG_API virtual void PostLoad() override;
	//~ End UObject Interface

	/** UActorComponent Interface */
	UMG_API virtual void BeginPlay() override;
	UMG_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	/* UPrimitiveComponent Interface */
	UMG_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UMG_API virtual FBoxSphereBounds CalcBounds(const FTransform & LocalToWorld) const override;
	UMG_API virtual UBodySetup* GetBodySetup() override;
	UMG_API virtual FCollisionShape GetCollisionShape(float Inflation) const override;
	UMG_API virtual void OnRegister() override;
	UMG_API virtual void OnUnregister() override;
	UMG_API virtual void DestroyComponent(bool bPromoteChildren = false) override;
	UMG_API UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UMG_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	UMG_API int32 GetNumMaterials() const override;
	UMG_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;/** Collect all the PSO precache data used by the static mesh component */
	UMG_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;


	UMG_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	UMG_API void ApplyComponentInstanceData(struct FWidgetComponentInstanceData* ComponentInstanceData);
	UMG_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;

#if WITH_EDITOR
	UMG_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UMG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/** Ensures the user widget is initialized */
	UMG_API virtual void InitWidget();

	/** Release resources associated with the widget. */
	UMG_API virtual void ReleaseResources();

	/** Ensures the 3d window is created its size and content. */
	UMG_API virtual void UpdateWidget();

	/** Ensure the render target is initialized and updates it if needed. */
	UMG_API virtual void UpdateRenderTarget(FIntPoint DesiredRenderTargetSize);

	/** 
	* Ensures the body setup is initialized and updates it if needed.
	* @param bDrawSizeChanged Whether the draw size of this component has changed since the last update call.
	*/
	UMG_API void UpdateBodySetup( bool bDrawSizeChanged = false );
	
	/** To be called when a SlateUser enters this world space widget. */
	UMG_API void OnWorldUserEntered(int32 SlateUserIndex);

	/**
	 * Converts a world-space hit result to a hit location on the widget
	 * @param HitResult The hit on this widget in the world
	 * @param (Out) The transformed 2D hit location on the widget
	 */
	UMG_API virtual void GetLocalHitLocation(FVector WorldHitLocation, FVector2D& OutLocalHitLocation) const;

	/**
	 * When using EWidgetGeometryMode::Cylinder, continues the trace from the front face
	 * of the widget component into the cylindrical geometry and returns adjusted hit results information.
	 * 
	 * @returns two hit locations FVector is in world space and a FVector2D is in widget-space.
	 */
	UMG_API TTuple<FVector, FVector2D> GetCylinderHitLocation(FVector WorldHitLocation, FVector WorldHitDirection) const;

	/** Gets the last local location that was hit */
	FVector2D GetLastLocalHitLocation() const
	{
		return LastLocalHitLocation;
	}
	
	/** Returns the class of the user widget displayed by this component */
	TSubclassOf<UUserWidget> GetWidgetClass() const { return WidgetClass; }

	/** Returns the user widget object displayed by this component */
	UFUNCTION(BlueprintCallable, Category=UserInterface, meta=(UnsafeDuringActorConstruction=true))
	UMG_API UUserWidget* GetUserWidgetObject() const;

	/** Returns the Slate widget that was assigned to this component, if any */
	UMG_API const TSharedPtr<SWidget>& GetSlateWidget() const;

	/** Returns the list of widgets with their geometry and the cursor position transformed into this Widget component's space. */
	UMG_API TArray<FWidgetAndPointer> GetHitWidgetPath(FVector WorldHitLocation, bool bIgnoreEnabledStatus, float CursorRadius = 0.0f);

	/** Returns the list of widgets with their geometry and the cursor position transformed into this Widget space. The widget space is expressed as a Vector2D. */
	UMG_API TArray<FWidgetAndPointer> GetHitWidgetPath(FVector2D WidgetSpaceHitCoordinate, bool bIgnoreEnabledStatus, float CursorRadius = 0.0f);

	/** Returns the render target to which the user widget is rendered */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API UTextureRenderTarget2D* GetRenderTarget() const;

	/** Returns the dynamic material instance used to render the user widget */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API UMaterialInstanceDynamic* GetMaterialInstance() const;

	/** Returns the window containing the user widget content */
	UMG_API TSharedPtr<SWindow> GetSlateWindow() const;

	/**  
	 *  Gets the widget that is used by this Widget Component. It will be null if a Slate Widget was set using SetSlateWidget function.
	 */ 
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API virtual UUserWidget* GetWidget() const;

	/**  
	 *  Sets the widget to use directly. This function will keep track of the widget till the next time it's called
	 *	with either a newer widget or a nullptr
	 */ 
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API virtual void SetWidget(UUserWidget* Widget);

	/**  
	 *  Sets a Slate widget to be rendered.  You can use this to draw native Slate widgets using a WidgetComponent, instead
	 *  of drawing user widgets.
	 */ 
	UMG_API virtual void SetSlateWidget( const TSharedPtr<SWidget>& InSlateWidget);

	/**
	 * Sets the local player that owns this widget component.  Setting the owning player controls
	 * which player's viewport the widget appears on in a split screen scenario.  Additionally it
	 * forwards the owning player to the actual UserWidget that is spawned.
	 */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API void SetOwnerPlayer(ULocalPlayer* LocalPlayer);

	/** @see bManuallyRedraw */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	bool GetManuallyRedraw() const
	{
		return bManuallyRedraw;
	};

	/** @see bManuallyRedraw */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API void SetManuallyRedraw(bool bUseManualRedraw);

	/** Gets the local player that owns this widget component. */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API ULocalPlayer* GetOwnerPlayer() const;

	/** Returns the "specified" draw size of the quad in the world */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API FVector2D GetDrawSize() const;

	/** Returns the "actual" draw size of the quad in the world */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	UMG_API FVector2D GetCurrentDrawSize() const;

	/** Sets the draw size of the quad in the world */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API void SetDrawSize(FVector2D Size);

	/** Requests that the widget be redrawn.  */
	UFUNCTION(BlueprintCallable, Category=UserInterface, meta = (DeprecatedFunction, DeprecationMessage = "Use RequestRenderUpdate instead"))
	UMG_API virtual void RequestRedraw();

	/** Requests that the widget have it's render target updated, if TickMode is disabled, this will force a tick to happen to update the render target. */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	UMG_API virtual void RequestRenderUpdate();

	/** Gets the blend mode for the widget. */
	EWidgetBlendMode GetBlendMode() const { return BlendMode; }

	/** Sets the blend mode to use for this widget */
	UMG_API void SetBlendMode( const EWidgetBlendMode NewBlendMode );

	/** Gets whether the widget is two-sided or not */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	bool GetTwoSided() const
	{
		return bIsTwoSided;
	};

	/** Sets whether the widget is two-sided or not */
	UFUNCTION(BlueprintCallable, Category = Rendering)
	UMG_API void SetTwoSided( const bool bWantTwoSided );

	/** Gets whether the widget ticks when offscreen or not */
	UFUNCTION(BlueprintCallable, Category = Animation)
	bool GetTickWhenOffscreen() const
	{
		return TickWhenOffscreen;
	};

	/** Sets whether the widget ticks when offscreen or not */
	UFUNCTION(BlueprintCallable, Category = Animation)
	void SetTickWhenOffscreen(const bool bWantTickWhenOffscreen)
	{
		TickWhenOffscreen = bWantTickWhenOffscreen;
	};

	/** Sets the background color and opacityscale for this widget */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API void SetBackgroundColor( const FLinearColor NewBackgroundColor );

	/** Sets the tint color and opacity scale for this widget */
	UFUNCTION(BlueprintCallable, Category=UserInterface)
	UMG_API void SetTintColorAndOpacity( const FLinearColor NewTintColorAndOpacity );

	/** Sets how much opacity from the UI widget's texture alpha is used when rendering to the viewport (0.0-1.0) */
	UMG_API void SetOpacityFromTexture( const float NewOpacityFromTexture );

	/** Returns the pivot point where the UI is rendered about the origin. */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	FVector2D GetPivot() const { return Pivot; }

	/**  */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	void SetPivot( const FVector2D& InPivot ) { Pivot = InPivot; }

	/**  */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	bool GetDrawAtDesiredSize() const { return bDrawAtDesiredSize; }

	/**  */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	void SetDrawAtDesiredSize(bool bInDrawAtDesiredSize) { bDrawAtDesiredSize = bInDrawAtDesiredSize; }

	/**  */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	float GetRedrawTime() const { return RedrawTime; }

	/**  */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	void SetRedrawTime(float InRedrawTime) { RedrawTime = InRedrawTime; }

	/** Get the fake window we create for widgets displayed in the world. */
	UMG_API TSharedPtr< SWindow > GetVirtualWindow() const;
	
	/** Updates the dynamic parameters on the material instance, without re-creating it */
	UMG_API void UpdateMaterialInstanceParameters();

	/** Sets the widget class used to generate the widget for this component */
	UMG_API void SetWidgetClass(TSubclassOf<UUserWidget> InWidgetClass);

	UFUNCTION(BlueprintCallable, Category = UserInterface)
	EWidgetSpace GetWidgetSpace() const { return Space; }

	UFUNCTION(BlueprintCallable, Category = UserInterface)
	void SetWidgetSpace( EWidgetSpace NewSpace ) { Space = NewSpace; }

	bool GetEditTimeUsable() const { return bEditTimeUsable; }

	void SetEditTimeUsable(bool Value) { bEditTimeUsable = Value; }

	/** @see EWidgetGeometryMode, @see GetCylinderArcAngle() */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	EWidgetGeometryMode GetGeometryMode() const 
	{
		return GeometryMode; 
	}

	UFUNCTION(BlueprintCallable, Category = UserInterface)
	void SetGeometryMode(EWidgetGeometryMode InGeometryMode) 
	{ 
		GeometryMode = InGeometryMode; 
	}

	bool GetReceiveHardwareInput() const { return bReceiveHardwareInput; }

	/** Defines the curvature of the widget component when using EWidgetGeometryMode::Cylinder; ignored otherwise.  */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	double GetCylinderArcAngle() const { return CylinderArcAngle; }

	/** Defines the curvature of the widget component when using EWidgetGeometryMode::Cylinder; ignored otherwise.  */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	void SetCylinderArcAngle(const double InCylinderArcAngle) 
	{ 
		CylinderArcAngle = InCylinderArcAngle; 
	}

	
	/** Sets shared layer name used when this widget is initialized */
	void SetInitialSharedLayerName(FName NewSharedLayerName) { SharedLayerName = NewSharedLayerName; }
	
	/** Sets layer z order used when this widget is initialized */
	void SetInitialLayerZOrder(int32 NewLayerZOrder) { LayerZOrder = NewLayerZOrder; }

	/** @see bWindowFocusable */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	bool GetWindowFocusable() const
	{
		return bWindowFocusable;
	};

	/** @see bWindowFocusable */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	UMG_API void SetWindowFocusable(bool bInWindowFocusable);

	/** Gets the visibility of the virtual window created to host the widget focusable. */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	EWindowVisibility GetWindowVisiblility() const
	{
		return WindowVisibility;
	}

	/** Sets the visibility of the virtual window created to host the widget focusable. */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	UMG_API void SetWindowVisibility(EWindowVisibility InVisibility);

	/** Sets the Tick mode of the Widget Component.*/
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	UMG_API void SetTickMode(ETickMode InTickMode);

	/** Returns true if the the Slate window is visible and that the widget is also visible, false otherwise. */
	UFUNCTION(BlueprintCallable, Category = UserInterface)
	UMG_API bool IsWidgetVisible() const;

	/** Hook to allow this component modify the local position of the widget after it has been projected from world space to screen space. */
	virtual FVector2D ModifyProjectedLocalPosition(const FGeometry& ViewportGeometry, const FVector2D& LocalPosition) { return LocalPosition; }

	DECLARE_MULTICAST_DELEGATE(FOnMaterialInstanceUpdated);
	FOnMaterialInstanceUpdated OnMaterialInstanceUpdated;

protected:
	UMG_API void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

	/** Just because the user attempts to receive hardware input does not mean it's possible. */
	UMG_API bool CanReceiveHardwareInput() const;

	UMG_API void RegisterHitTesterWithViewport(TSharedPtr<SViewport> ViewportWidget);
	UMG_API void UnregisterHitTesterWithViewport(TSharedPtr<SViewport> ViewportWidget);

	UMG_API void RegisterWindow();
	UMG_API void UnregisterWindow();
	UMG_API void RemoveWidgetFromScreen();

	/** Allows subclasses to control if the widget should be drawn.  Called right before we draw the widget. */
	UMG_API virtual bool ShouldDrawWidget() const;

	/** Draws the current widget to the render target if possible. */
	UMG_API virtual void DrawWidgetToRenderTarget(float DeltaTime);

	/** Returns the width of the widget component taking GeometryMode into account. */
	UMG_API double ComputeComponentWidth() const;

	UMG_API void UpdateMaterialInstance();

	UMG_API virtual void OnHiddenInGameChanged() override;

protected:
	/** The coordinate space in which to render the widget */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	EWidgetSpace Space;

	/** How this widget should deal with timing, pausing, etc. */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	EWidgetTimingPolicy TimingPolicy;

	/** The class of User Widget to create and display an instance of */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	TSubclassOf<UUserWidget> WidgetClass;
	
	/** The size of the displayed quad. */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	FIntPoint DrawSize;

	/** Should we wait to be told to redraw to actually draw? */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	bool bManuallyRedraw;

	/** Has anyone requested we redraw? */
	UPROPERTY(Transient, DuplicateTransient)
	bool bRedrawRequested;

	/**
	 * The time in between draws, if 0 - we would redraw every frame.  If 1, we would redraw every second.
	 * This will work with bManuallyRedraw as well.  So you can say, manually redraw, but only redraw at this
	 * maximum rate.
	 */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	float RedrawTime;

	/** What was the last time we rendered the widget? */
	double LastWidgetRenderTime;

	/** Returns current absolute time, respecting TimingPolicy. */
	UMG_API double GetCurrentTime() const;

	/**
	 * The actual draw size, this changes based on DrawSize - or the desired size of the widget if
	 * bDrawAtDesiredSize is true.
	 */
	UPROPERTY(Transient, DuplicateTransient)
	FIntPoint CurrentDrawSize;

	/**
	 * Use the invalidation system to update this widget.
	 * Only valid in World space. In Screen space, the widget is updated by the viewport owners.
	 */
	UPROPERTY(EditAnywhere, Category = UserInterface, meta=(EditCondition="Space==EWidgetSpace::World", DisplayName="Use Invalidation"))
	bool bUseInvalidationInWorldSpace;

	/**
	 * Causes the render target to automatically match the desired size.
	 * 
	 * WARNING: If you change this every frame, it will be very expensive. If you need 
	 *    that effect, you should keep the outer widget's sized locked and dynamically
	 *    scale or resize some inner widget.
	 */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	bool bDrawAtDesiredSize;

	/** The Alignment/Pivot point that the widget is placed at relative to the position. */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	FVector2D Pivot;

	/**
	 * Register with the viewport for hardware input from the true mouse and keyboard.  These widgets
	 * will more or less react like regular 2D widgets in the viewport, e.g. they can and will steal focus
	 * from the viewport.
	 * 
	 * WARNING: If you are making a VR game, definitely do not change this to true.  This option should ONLY be used
	 * if you're making what would otherwise be a normal menu for a game, just in 3D.  If you also need the game to 
	 * remain responsive and for the player to be able to interact with UI and move around the world (such as a keypad on a door), 
	 * use the WidgetInteractionComponent instead.
	 */
	UPROPERTY(EditAnywhere, Category=Interaction)
	bool bReceiveHardwareInput;

	/** Is the virtual window created to host the widget focusable? */
	UPROPERTY(EditAnywhere, Category=Interaction)
	bool bWindowFocusable;

	/** The visibility of the virtual window created to host the widget */
	UPROPERTY(EditAnywhere, Category = Interaction)
	EWindowVisibility WindowVisibility;

	/**
	 * Widget components that appear in the world will be gamma corrected by the 3D renderer.
	 * In some cases, widget components are blitted directly into the backbuffer, in which case gamma correction should be enabled.
	 */
	UPROPERTY(EditAnywhere, Category = UserInterface, AdvancedDisplay)
	bool bApplyGammaCorrection;

	/**
	 * The owner player for a widget component, if this widget is drawn on the screen, this controls
	 * what player's screen it appears on for split screen, if not set, users player 0.
	 */
	UPROPERTY()
	TObjectPtr<ULocalPlayer> OwnerPlayer;

	/** The background color of the component */
	UPROPERTY(EditAnywhere, Category=Rendering)
	FLinearColor BackgroundColor;

	/** Tint color and opacity for this component */
	UPROPERTY(EditAnywhere, Category=Rendering)
	FLinearColor TintColorAndOpacity;

	/** Sets the amount of opacity from the widget's UI texture to use when rendering the translucent or masked UI to the viewport (0.0-1.0) */
	UPROPERTY(EditAnywhere, Category=Rendering, meta=(ClampMin=0.0f, ClampMax=1.0f))
	float OpacityFromTexture;

	/** The blend mode for the widget. */
	UPROPERTY(EditAnywhere, Category=Rendering)
	EWidgetBlendMode BlendMode;

	/** When enabled, overrides the render target pixel format instead of using the default Slate format (RGBA8). */
	UPROPERTY(EditAnywhere, Category=Rendering, AdvancedDisplay)
	bool bOverrideRenderTargetFormat;

	/** The render target format to use when bOverrideRenderTargetFormat is true. */
	UPROPERTY(EditAnywhere, Category=Rendering, AdvancedDisplay, meta=(EditCondition="bOverrideRenderTargetFormat"))
	TEnumAsByte<ETextureRenderTargetFormat> RenderTargetFormatOverride;

	/** Is the component visible from behind? */
	UPROPERTY(EditAnywhere, Category=Rendering)
	bool bIsTwoSided;

	/** Should the component tick the widget when it's off screen? */
	UPROPERTY(EditAnywhere, Category=Animation)
	bool TickWhenOffscreen;

	/** The body setup of the displayed quad */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<class UBodySetup> BodySetup;

	/** The material instance for translucent widget components */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> TranslucentMaterial;

	/** The material instance for translucent, one-sided widget components */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> TranslucentMaterial_OneSided;

	/** The material instance for opaque widget components */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> OpaqueMaterial;

	/** The material instance for opaque, one-sided widget components */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> OpaqueMaterial_OneSided;

	/** The material instance for masked widget components. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> MaskedMaterial;

	/** The material instance for masked, one-sided widget components. */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> MaskedMaterial_OneSided;

	/** The target to which the user widget is rendered */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

	/** The dynamic instance of the material that the render target is attached to */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UMaterialInstanceDynamic> MaterialInstance;

	UPROPERTY(Transient, DuplicateTransient)
	bool bAddedToScreen;

	/**
	 * Allows the widget component to be used at editor time.  For use in the VR-Editor.
	 */
	UPROPERTY()
	bool bEditTimeUsable;

protected:

	/** Layer Name the widget will live on */
	UPROPERTY(EditDefaultsOnly, Category = Layers)
	FName SharedLayerName;

	/** ZOrder the layer will be created on, note this only matters on the first time a new layer is created, subsequent additions to the same layer will use the initially defined ZOrder */
	UPROPERTY(EditDefaultsOnly, Category = Layers)
	int32 LayerZOrder;

	/** Controls the geometry of the widget component. See EWidgetGeometryMode. */
	UPROPERTY(EditAnywhere, Category=UserInterface)
	EWidgetGeometryMode GeometryMode;

	/** Curvature of a cylindrical widget in degrees. */
	UPROPERTY(EditAnywhere, Category=UserInterface, meta=(ClampMin=1.0f, ClampMax=180.0f))
	double CylinderArcAngle;

	UPROPERTY(EditAnywhere, Category = UserInterface)
	ETickMode TickMode;

	/** The slate window that contains the user widget content */
	TSharedPtr<class SVirtualWindow> SlateWindow;

	/** The relative location of the last hit on this component */
	FVector2D LastLocalHitLocation;

	/** The hit tester to use for this component */
	static UMG_API TSharedPtr<class FWidget3DHitTester> WidgetHitTester;

	/** Helper class for drawing widgets to a render target. */
	class FWidgetRenderer* WidgetRenderer;

private: 
	UMG_API bool ShouldReenableComponentTickWhenWidgetBecomesVisible() const;

	/** The User Widget object displayed and managed by this component */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UUserWidget> Widget;

	/** The Slate widget to be displayed by this component.  Only one of either Widget or SlateWidget can be used */
	TSharedPtr<SWidget> SlateWidget;
	
	/** The Slate user ids of the users which have entered our world-space widget (and thus could've gotten focus). */
	TSet<int32> UsersEnteredWorldWidget;

	/** The slate widget currently being drawn. */
	TWeakPtr<SWidget> CurrentSlateWidget;

	static UMG_API EVisibility ConvertWindowVisibilityToVisibility(EWindowVisibility visibility);

	UMG_API void OnWidgetVisibilityChanged(ESlateVisibility InVisibility);

	/** When using Screen space, this will update the Widget on Screen. **/
	UMG_API void UpdateWidgetOnScreen();

	/** When using Screen space, this code will add the widget to the screen. **/
	UMG_API void AddWidgetToScreen(ULocalPlayer* TargetPlayer);
	
	/** When using World space, this will reset all the users focused on the active widget.  */
	UMG_API void ResetWorldUserFocusOnWidget();

	/** Set to true after a draw of an empty component.*/
	bool bRenderCleared;
	bool bOnWidgetVisibilityChangedRegistered;
};

USTRUCT()
struct FWidgetComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()
public:
	FWidgetComponentInstanceData()
		: RenderTarget(nullptr)
	{}

	FWidgetComponentInstanceData(const UWidgetComponent* SourceComponent)
		: FSceneComponentInstanceData(SourceComponent)
		, WidgetClass(SourceComponent->GetWidgetClass())
		, RenderTarget(SourceComponent->GetRenderTarget())
	{}
	virtual ~FWidgetComponentInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<UWidgetComponent>(Component)->ApplyComponentInstanceData(this);
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Super::AddReferencedObjects(Collector);

		Collector.AddReferencedObject(WidgetClass.GetGCPtr());
		Collector.AddReferencedObject(RenderTarget);
	}

public:
	TSubclassOf<UUserWidget> WidgetClass;
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;
};
