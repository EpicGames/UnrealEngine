// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "InputCoreTypes.h"
#include "Engine/HitResult.h"
#include "Components/SceneComponent.h"
#include "GenericPlatform/GenericApplication.h"
#include "Layout/WidgetPath.h"
#include "WidgetInteractionComponent.generated.h"

class FSlateVirtualUserHandle;
class UPrimitiveComponent;
class UWidgetComponent;

/**
 * The interaction source for the widget interaction component, e.g. where do we try and
 * trace from to try to find a widget under a virtual pointer device.
 */
UENUM(BlueprintType)
enum class EWidgetInteractionSource : uint8
{
	/** Sends traces from the world location and orientation of the interaction component. */
	World,
	/** Sends traces from the mouse location of the first local player controller. */
	Mouse,
	/** Sends trace from the center of the first local player's screen. */
	CenterScreen,
	/**
	 * Sends traces from a custom location determined by the user.  Will use whatever 
	 * FHitResult is set by the call to SetCustomHitResult.
	 */
	Custom
};

// TODO CenterScreen needs to be able to work with multiple player controllers, perhaps finding
// the PC via the outer/owner chain?  Maybe you need to set the PC that owns this?  Maybe we should
// key off the Virtual User Index?

// TODO Expose modifier key state.

// TODO Come up with a better way to let people forward a lot of keyboard input without a bunch of glue

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnHoveredWidgetChanged, UWidgetComponent*, WidgetComponent, UWidgetComponent*, PreviousWidgetComponent);

/**
 * This is a component to allow interaction with the Widget Component.  This class allows you to 
 * simulate a sort of laser pointer device, when it hovers over widgets it will send the basic signals 
 * to show as if the mouse were moving on top of it.  You'll then tell the component to simulate key presses, 
 * like Left Mouse, down and up, to simulate a mouse click.
 */
UCLASS(ClassGroup="UserInterface", meta=(BlueprintSpawnableComponent) , MinimalAPI)
class UWidgetInteractionComponent : public USceneComponent
{
	GENERATED_BODY()

public:
	/**
	 * Called when the hovered Widget Component changes.  The interaction component functions at the Slate
	 * level - so it's unable to report anything about what UWidget is under the hit result.
	 */
	UPROPERTY(BlueprintAssignable, Category="Interaction|Event")
	FOnHoveredWidgetChanged OnHoveredWidgetChanged;

public:
	UMG_API UWidgetInteractionComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	// Begin ActorComponent interface
	UMG_API virtual void OnComponentCreated() override;
	UMG_API virtual void Activate(bool bReset = false) override;
	UMG_API virtual void Deactivate() override;
	UMG_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	// End UActorComponent
	
	/**
	 * Presses a key as if the mouse/pointer were the source of it.  Normally you would just use
	 * Left/Right mouse button for the Key.  However - advanced uses could also be imagined where you
	 * send other keys to signal widgets to take special actions if they're under the cursor.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API virtual void PressPointerKey(FKey Key);
	
	/**
	 * Releases a key as if the mouse/pointer were the source of it.  Normally you would just use
	 * Left/Right mouse button for the Key.  However - advanced uses could also be imagined where you
	 * send other keys to signal widgets to take special actions if they're under the cursor.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API virtual void ReleasePointerKey(FKey Key);

	/**
	 * Press a key as if it had come from the keyboard.  Avoid using this for 'a-z|A-Z', things like
	 * the Editable Textbox in Slate expect OnKeyChar to be called to signal a specific character being
	 * send to the widget.  So for those cases you should use SendKeyChar.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API virtual bool PressKey(FKey Key, bool bRepeat = false);
	
	/**
	 * Releases a key as if it had been released by the keyboard.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API virtual bool ReleaseKey(FKey Key);

	/**
	 * Does both the press and release of a simulated keyboard key.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API virtual bool PressAndReleaseKey(FKey Key);

	/**
	 * Transmits a list of characters to a widget by simulating a OnKeyChar event for each key listed in
	 * the string.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API virtual bool SendKeyChar(FString Characters, bool bRepeat = false);
	
	/**
	 * Sends a scroll wheel event to the widget under the last hit result.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API virtual void ScrollWheel(float ScrollDelta);
	
	/**
	 * Get the currently hovered widget component.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API UWidgetComponent* GetHoveredWidgetComponent() const;

	/**
	 * Returns true if a widget under the hit result is interactive.  e.g. Slate widgets 
	 * that return true for IsInteractable().
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API bool IsOverInteractableWidget() const;

	/**
	 * Returns true if a widget under the hit result is focusable.  e.g. Slate widgets that 
	 * return true for SupportsKeyboardFocus().
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API bool IsOverFocusableWidget() const;

	/**
	 * Returns true if a widget under the hit result is has a visibility that makes it hit test 
	 * visible.  e.g. Slate widgets that return true for GetVisibility().IsHitTestVisible().
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API bool IsOverHitTestVisibleWidget() const;

	/**
	 * Gets the widget path for the slate widgets under the last hit result.
	 */
	UMG_API const FWeakWidgetPath& GetHoveredWidgetPath() const;

	/**
	 * Gets the last hit result generated by the component.  Returns the custom hit result if that was set.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API const FHitResult& GetLastHitResult() const;

	/**
	 * Gets the last hit location on the widget in 2D, local pixel units of the render target.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API FVector2D Get2DHitLocation() const;

	/**
	 * Set custom hit result.  This is only taken into account if InteractionSource is set to EWidgetInteractionSource::Custom.
	 */
	UFUNCTION(BlueprintCallable, Category="Interaction")
	UMG_API void SetCustomHitResult(const FHitResult& HitResult);

	/**
	 * Set the focus target of the virtual user managed by this component
	 */
	UFUNCTION(BlueprintCallable, Category = "Interaction")
	UMG_API void SetFocus(UWidget* FocusWidget);

protected:
	/**
	 * Represents the virtual user in slate.  When this component is registered, it gets a handle to the 
	 * virtual slate user it will be, so virtual slate user 0, is probably real slate user 8, as that's the first
	 * index by default that virtual users begin - the goal is to never have them overlap with real input
	 * hardware as that will likely conflict with focus states you don't actually want to change - like where
	 * the mouse and keyboard focus input (the viewport), so that things like the player controller receive
	 * standard hardware input.
	 */
	TSharedPtr<FSlateVirtualUserHandle> VirtualUser;

public:

	/**
	 * Represents the Virtual User Index.  Each virtual user should be represented by a different 
	 * index number, this will maintain separate capture and focus states for them.  Each
	 * controller or finger-tip should get a unique PointerIndex.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interaction", meta=( ClampMin = "0", ExposeOnSpawn = true ))
	int32 VirtualUserIndex;

	/**
	 * Each user virtual controller or virtual finger tips being simulated should use a different pointer index.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interaction", meta=( ClampMin = "0", UIMin = "0", UIMax = "9", ExposeOnSpawn = true ))
	int32 PointerIndex;

public:

	/**
	 * The trace channel to use when tracing for widget components in the world.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interaction")
	TEnumAsByte<ECollisionChannel> TraceChannel;

	/**
	 * The distance in game units the component should be able to interact with a widget component.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interaction")
	float InteractionDistance;

	/**
	 * Should we project from the world location of the component?  If you set this to false, you'll
	 * need to call SetCustomHitResult(), and provide the result of a custom hit test form whatever
	 * location you wish.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interaction")
	EWidgetInteractionSource InteractionSource;

	/**
	 * Should the interaction component perform hit testing (Automatic or Custom) and attempt to 
	 * simulate hover - if you were going to emulate a keyboard you would want to turn this option off
	 * if the virtual keyboard was separate from the virtual pointer device and used a second interaction
	 * component.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Interaction")
	bool bEnableHitTesting;

public:

	/**
	 * Shows some debugging lines and a hit sphere to help you debug interactions.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Debugging")
	bool bShowDebug;

	/**
	 * Determines the line thickness of the debug sphere.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Debugging", AdvancedDisplay, meta=( ClampMin = "0.001" ))
	float DebugSphereLineThickness;

	/**
	 * Determines the thickness of the debug lines.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Debugging", AdvancedDisplay, meta=( ClampMin = "0.001", ClampMax = "50"))
	float DebugLineThickness;

	/**
	 * Determines the color of the debug lines.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Debugging", AdvancedDisplay)
	FLinearColor DebugColor;

protected:

	// Gets the key and char codes for sending keys for the platform.
	UMG_API void GetKeyAndCharCodes(const FKey& Key, bool& bHasKeyCode, uint32& KeyCode, bool& bHasCharCode, uint32& CharCode);

	/** Is it safe for this interaction component to run?  Might not be in a server situation with no slate application. */
	UMG_API bool CanSendInput();

	/** Performs the simulation of pointer movement.  Does not run if bEnableHitTesting is set to false. */
	UMG_API void SimulatePointerMovement();

	struct FWidgetTraceResult
	{
		FWidgetTraceResult()
			: HitResult()
			, LocalHitLocation(FVector2D::ZeroVector)
			, HitWidgetComponent(nullptr)
			, HitWidgetPath()
			, bWasHit(false)
			, LineStartLocation(FVector::ZeroVector)
			, LineEndLocation(FVector::ZeroVector)
		{
		}

		FHitResult HitResult;
		FVector2D LocalHitLocation;
		UWidgetComponent* HitWidgetComponent;
		FWidgetPath HitWidgetPath;
		bool bWasHit;
		FVector LineStartLocation;
		FVector LineEndLocation;
	};

	/** Gets the WidgetPath for the widget being hovered over based on the hit result. */
	UMG_API virtual FWidgetPath FindHoveredWidgetPath(const FWidgetTraceResult& TraceResult)  const;

	/** Performs the trace and gets the hit result under the specified InteractionSource */
	UMG_API virtual FWidgetTraceResult PerformTrace() const;

	/**
	 * Gets the list of components to ignore during hit testing.  Which is everything that is a parent/sibling of this 
	 * component that's not a Widget Component.  This is so traces don't get blocked by capsules and such around the player.
	 */
	UMG_API void GetRelatedComponentsToIgnoreInAutomaticHitTesting(TArray<UPrimitiveComponent*>& IgnorePrimitives) const;

	/** Returns true if the inteaction component can interact with the supplied widget component */
	UMG_API bool CanInteractWithComponent(UWidgetComponent* Component) const;

protected:

	/** The last widget path under the hit result. */
	FWeakWidgetPath LastWidgetPath;

	/** The modifier keys to simulate during key presses. */
	FModifierKeysState ModifierKeys;
	
	/** The current set of pressed keys we maintain the state of. */
	TSet<FKey> PressedKeys;

	/** Stores the custom hit result set by the player. */
	UPROPERTY(Transient)
	FHitResult CustomHitResult;
	
	/** The 2D location on the widget component that was hit. */
	UPROPERTY(Transient)
	FVector2D LocalHitLocation;
	
	/** The last 2D location on the widget component that was hit. */
	UPROPERTY(Transient)
	FVector2D LastLocalHitLocation;

	/** DEPRECATED - Use WeakHoveredWidgetComponent instead */
	UE_DEPRECATED(5.6, "Property deprecated, please use WeakHoveredWidgetComponent instead.")
	UPROPERTY(Transient)
	TObjectPtr<UWidgetComponent> HoveredWidgetComponent = nullptr;

	/** The widget component we're currently hovering over. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UWidgetComponent> WeakHoveredWidgetComponent;

	/** The last hit result we used. */
	UPROPERTY(Transient)
	FHitResult LastHitResult;

	/** Are we hovering over any interactive widgets. */
	UPROPERTY(Transient)
	bool bIsHoveredWidgetInteractable;

	/** Are we hovering over any focusable widget? */
	UPROPERTY(Transient)
	bool bIsHoveredWidgetFocusable;

	/** Are we hovered over a widget that is hit test visible? */
	UPROPERTY(Transient)
	bool bIsHoveredWidgetHitTestVisible;

private:

#if WITH_EDITOR
	void OnPIEEnded(class UGameInstance*);
#endif
	
	/** Returns the path to the widget that is currently beneath the pointer */
	FWidgetPath DetermineWidgetUnderPointer();

private:
#if WITH_EDITORONLY_DATA

	/** The arrow component we show at editor time. */
	UPROPERTY()
	TObjectPtr<class UArrowComponent> ArrowComponent;

#endif

};
