// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Components/PanelWidget.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "WidgetSwitcher.generated.h"

/**
 * A widget switcher is like a tab control, but without tabs. At most one widget is visible at time.
 */
UCLASS(MinimalAPI)
class UWidgetSwitcher : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.2, "Direct access to ActiveWidgetIndex is deprecated. Please use the getter or setter.")
	/** The slot index to display */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetActiveWidgetIndex", BlueprintGetter = "GetActiveWidgetIndex", FieldNotify, Category = "Switcher", meta = (UIMin = 0, ClampMin = 0))
	int32 ActiveWidgetIndex;

public:

	/** Gets the number of widgets that this switcher manages. */
	UFUNCTION(BlueprintCallable, Category="Switcher")
	UMG_API int32 GetNumWidgets() const;

	/** Gets the slot index of the currently active widget */
	UFUNCTION(BlueprintCallable, Category="Switcher")
	UMG_API int32 GetActiveWidgetIndex() const;

	/** Activates the widget at the specified index. */
	UFUNCTION(BlueprintCallable, Category="Switcher")
	UMG_API virtual void SetActiveWidgetIndex( int32 Index );

	/** Activates the widget and makes it the active index. */
	UFUNCTION(BlueprintCallable, Category="Switcher")
	UMG_API virtual void SetActiveWidget(UWidget* Widget);

	/** Get a widget at the provided index */
	UFUNCTION( BlueprintCallable, Category = "Switcher" )
	UMG_API UWidget* GetWidgetAtIndex( int32 Index ) const;
	
	/** Get the reference of the currently active widget */
	UFUNCTION(BlueprintCallable, Category = "Switcher")
	UMG_API UWidget* GetActiveWidget() const;
	
	// UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UWidget interface

#if WITH_EDITOR
	UMG_API virtual const FText GetPaletteCategory() override;
	UMG_API virtual void OnDescendantSelectedByDesigner(UWidget* DescendantWidget) override;
	UMG_API virtual void OnDescendantDeselectedByDesigner(UWidget* DescendantWidget) override;

	// UObject interface
	UMG_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface
#endif

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

	UMG_API void SetActiveWidgetIndexForSlateWidget();
protected:

	TSharedPtr<class SWidgetSwitcher> MyWidgetSwitcher;

protected:
	// UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface
};
