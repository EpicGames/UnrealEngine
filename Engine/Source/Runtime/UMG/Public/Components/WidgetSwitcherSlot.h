// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Components/PanelSlot.h"

#include "Widgets/Layout/SWidgetSwitcher.h"
#include "WidgetSwitcherSlot.generated.h"

class UWidget;

/** The Slot for the UWidgetSwitcher, contains the widget that is flowed vertically */
UCLASS(MinimalAPI)
class UWidgetSwitcherSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

private:
	//TODO UMG Slots should hold weak or shared refs to slots.

	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SWidgetSwitcher::FSlot* Slot;

public:

	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetPadding", Category="Layout|Widget Switcher Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Widget Switcher Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Widget Switcher Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Widget Switcher Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Widget Switcher Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Widget Switcher Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

public:

	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SWidgetSwitcher> InWidgetSwitcher);

	/** Sets the content of this slot, removing existing content if needed. */
	UMG_API void SetContent(UWidget* NewContent);
};
