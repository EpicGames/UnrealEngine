// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "GenericPlatform/GenericWindow.h"
#include "Widgets/SWidget.h"
#include "Components/ContentWidget.h"

#include "WindowTitleBarArea.generated.h"

class SWindowTitleBarArea;

/**
* A panel for defining a region of the UI that should allow users to drag the window on desktop platforms.
*/

UCLASS(MinimalAPI)
class UWindowTitleBarArea : public UContentWidget
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.2, "Direct access to bWindowButtonsEnabled is deprecated. Please use the getter or setter.")
	/** Should the title bar area diaplay window minimize/maximize/close buttons. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsWindowButtonsEnabled", Setter = "SetWindowButtonsEnabled", Category = "Interaction", meta = (DisplayName = "Window Buttons Enabled"))
	bool bWindowButtonsEnabled;

	UE_DEPRECATED(5.2, "Direct access to bDoubleClickTogglesFullscreen is deprecated. Please use the getter or setter.")
	/** Should double clicking the title bar area toggle fullscreen instead of maximizing the window. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsDoubleClickTogglesFullscreen", Setter = "SetDoubleClickTogglesFullscreen", Category = "Interaction", meta = (DisplayName = "Toggle Fullscreen"))
	bool bDoubleClickTogglesFullscreen;

public:

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetPadding(FMargin InPadding);

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UFUNCTION(BlueprintCallable, Category = "Appearance")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	UMG_API void SetWindowButtonsEnabled(bool InWindowButtonsEnabled);

	UMG_API bool IsWindowButtonsEnabled() const;

	UMG_API void SetDoubleClickTogglesFullscreen(bool InDoubleClickTogglesFullscreen);

	UMG_API bool IsDoubleClickTogglesFullscreen() const;

public:

	//~ Begin UVisual Interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End UVisual Interface

	//~ Begin UObject Interface
	UMG_API virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UWidget Interface
#if WITH_EDITOR	
	UMG_API virtual const FText GetPaletteCategory() override;
#endif
	//~ End UWidget Interface

protected:
	//~ Begin UWidget Interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End UWidget Interface

protected:

	// UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	// End UPanelWidget

protected:
	/** Cached pointer to the underlying slate button owned by this UWidget */
	TSharedPtr<SWindowTitleBarArea> MyWindowTitleBarArea;

private:

	bool HandleWindowAction(const TSharedRef<FGenericWindow>& PlatformWindow, EWindowAction::Type WindowAction);
	void RequestToggleFullscreen();

private:

	FDelegateHandle WindowActionNotificationHandle;
};
