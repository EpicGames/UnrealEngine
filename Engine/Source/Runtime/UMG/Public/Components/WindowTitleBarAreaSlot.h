// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Layout/Margin.h"
#include "Components/PanelSlot.h"
#include "Components/WindowTitleBarArea.h"

#include "WindowTitleBarAreaSlot.generated.h"

class SWindowTitleBarArea;

/** The Slot for the UWindowTitleBarArea */
UCLASS(MinimalAPI)
class UWindowTitleBarAreaSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|WindowTitleBarArea Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|WindowTitleBarArea Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category = "Layout|WindowTitleBarArea Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

protected:

	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter = "SetPadding", Category = "Layout|WindowTitleBarArea Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter = "SetHorizontalAlignment", Category = "Layout|WindowTitleBarArea Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter = "SetVerticalAlignment", Category = "Layout|WindowTitleBarArea Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

public:

	//~ UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UPanelSlot interface

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SWindowTitleBarArea> WindowTitleBarArea);

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

private:

	/** A pointer to the window zone to allow us to adjust the size, padding...etc at runtime. */
	TSharedPtr<SWindowTitleBarArea> WindowTitleBarArea;

	friend UWindowTitleBarArea;
};
