// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Components/PanelWidget.h"
#include "WrapBox.generated.h"

class UWrapBoxSlot;

/**
 * Arranges widgets left-to-right or top-to-bottom dependently of the orientation.  When the widgets exceed the wrapSize it will place widgets on the next line.
 * 
 * * Many Children
 * * Flows
 * * Wraps
 */
UCLASS(MinimalAPI)
class UWrapBox : public UPanelWidget
{
	GENERATED_UCLASS_BODY()

public:
	UE_DEPRECATED(5.1, "Direct access to InnerSlotPadding is deprecated. Please use the getter or setter.")
	/** The inner slot padding goes between slots sharing borders */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetInnerSlotPadding", Category = "Content Layout")
	FVector2D InnerSlotPadding;

	UE_DEPRECATED(5.1, "Direct access to WrapSize is deprecated. Please use the getter or setter.")
	/** When this size is exceeded, elements will start appearing on the next line. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Content Layout", meta=(EditCondition = "bExplicitWrapSize"))
	float WrapSize;

	UE_DEPRECATED(5.1, "Direct access to bExplicitWrapSize is deprecated. Please use the getter or setter.")
	/** Use explicit wrap size whenever possible. It greatly simplifies layout calculations and reduces likelihood of "wiggling UI" */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter="UseExplicitWrapSize", Setter = "SetExplicitWrapSize", Category = "Content Layout")
	bool bExplicitWrapSize;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of each line of wrapped content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, BlueprintSetter = "SetHorizontalAlignment", Category = "Content Layout", meta = (EditCondition = "Orientation == EOrientation::Orient_Horizontal"))
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to Orientation is deprecated. Please use the getter or setter.")
	/** Determines if the Wrap Box should arranges the widgets left-to-right or top-to-bottom */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Content Layout")
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Horizontal;

public:
	/** Gets the inner slot padding goes between slots sharing borders */
	UMG_API FVector2D GetInnerSlotPadding() const;

	/** Sets the inner slot padding goes between slots sharing borders */
	UFUNCTION(BlueprintCallable, Category="Content Layout")
	UMG_API void SetInnerSlotPadding(FVector2D InPadding);

	/** */
	UMG_API float GetWrapSize() const;

	/** */
	UMG_API void SetWrapSize(float InWrapSize);

	/** */
	UMG_API bool UseExplicitWrapSize() const;

	/** */
	UMG_API void SetExplicitWrapSize(bool bInExplicitWrapSize);

	/** */
	UMG_API EHorizontalAlignment GetHorizontalAlignment() const;

	/** */
	UFUNCTION(BlueprintCallable, Category="Content Layout")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	/** */
	UMG_API EOrientation GetOrientation() const;

	/** */
	UMG_API void SetOrientation(EOrientation InOrientation);

public:
	UFUNCTION(BlueprintCallable, Category="Panel")
	UMG_API UWrapBoxSlot* AddChildToWrapBox(UWidget* Content);

#if WITH_EDITOR
	//~ UWidget interface
	UMG_API virtual const FText GetPaletteCategory() override;
	//~ End UWidget interface
#endif

protected:

	//~ UPanelWidget
	UMG_API virtual UClass* GetSlotClass() const override;
	UMG_API virtual void OnSlotAdded(UPanelSlot* Slot) override;
	UMG_API virtual void OnSlotRemoved(UPanelSlot* Slot) override;
	//~ End UPanelWidget

	//~ UWidget interface
	UMG_API virtual void SynchronizeProperties() override;
	//~ End of UWidget interface

	//~ UVisual interface
	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	//~ End of UVisual interface

protected:

	TSharedPtr<class SWrapBox> MyWrapBox;

protected:
	//~ UWidget interface
	UMG_API virtual TSharedRef<SWidget> RebuildWidget() override;
	//~ End of UWidget interface
};
