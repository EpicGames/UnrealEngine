// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Layout/Margin.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Components/PanelSlot.h"

#include "WrapBoxSlot.generated.h"

/** The Slot for the UWrapBox, contains the widget that is flowed vertically */
UCLASS(MinimalAPI)
class UWrapBoxSlot : public UPanelSlot
{
	GENERATED_UCLASS_BODY()

public:

	UE_DEPRECATED(5.1, "Direct access to Padding is deprecated. Please use the getter or setter.")
	/** The padding area between the slot and the content it contains. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetPadding", Category="Layout|Wrap Box Slot")
	FMargin Padding;

	UE_DEPRECATED(5.1, "Direct access to FillSpanWhenLessThan is deprecated. Please use the getter or setter.")
	/**
	 * If the total available space in the wrap panel drops below this threshold, this slot will attempt to fill an entire line.
	 * NOTE: A value of 0, denotes no filling will occur.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetFillSpanWhenLessThan", Category="Layout|Wrap Box Slot")
	float FillSpanWhenLessThan;

	UE_DEPRECATED(5.1, "Direct access to HorizontalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object horizontally. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetHorizontalAlignment", Category="Layout|Wrap Box Slot")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UE_DEPRECATED(5.1, "Direct access to VerticalAlignment is deprecated. Please use the getter or setter.")
	/** The alignment of the object vertically. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, BlueprintSetter="SetVerticalAlignment", Category="Layout|Wrap Box Slot")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

	UE_DEPRECATED(5.1, "Direct access to bFillEmptySpace is deprecated. Please use the getter or setter.")
	/** Should this slot fill the remaining space on the line? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter="SetFillEmptySpace", BlueprintSetter="SetFillEmptySpace", Category = "Layout|Wrap Box Slot")
	bool bFillEmptySpace;

	UE_DEPRECATED(5.1, "Direct access to bForceNewLine is deprecated. Please use the getter or setter.")
	/** Should this slot start on a new line? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter="SetNewLine", BlueprintSetter="SetNewLine", Category = "Layout|Wrap Box Slot")
	bool bForceNewLine;

public:

	UMG_API FMargin GetPadding() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Wrap Box Slot")
	UMG_API void SetPadding(FMargin InPadding);

	UMG_API bool DoesFillEmptySpace() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Wrap Box Slot")
	UMG_API void SetFillEmptySpace(bool InbFillEmptySpace);

	UMG_API float GetFillSpanWhenLessThan() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Wrap Box Slot")
	UMG_API void SetFillSpanWhenLessThan(float InFillSpanWhenLessThan);

	UMG_API EHorizontalAlignment  GetHorizontalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Wrap Box Slot")
	UMG_API void SetHorizontalAlignment(EHorizontalAlignment InHorizontalAlignment);

	UMG_API EVerticalAlignment GetVerticalAlignment() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Wrap Box Slot")
	UMG_API void SetVerticalAlignment(EVerticalAlignment InVerticalAlignment);

	UMG_API bool DoesForceNewLine() const;

	UFUNCTION(BlueprintCallable, Category="Layout|Wrap Box Slot")
	UMG_API void SetNewLine(bool InForceNewLine);

public:

	// UPanelSlot interface
	UMG_API virtual void SynchronizeProperties() override;
	// End of UPanelSlot interface

	UMG_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

	/** Builds the underlying FSlot for the Slate layout panel. */
	UMG_API void BuildSlot(TSharedRef<SWrapBox> InWrapBox);

private:
	//TODO UMG Slots should hold weak or shared refs to slots.

	/** A raw pointer to the slot to allow us to adjust the size, padding...etc at runtime. */
	SWrapBox::FSlot* Slot;
};
