// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/SubclassOf.h"

#if WITH_EDITOR

class UUserWidget;

class IWidgetCompilerLog
{
public:
	TSharedRef<FTokenizedMessage> Error(const FText& Message)
	{
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Error);
		Line->AddToken(FTextToken::Create(Message));
		InternalLogMessage(Line);
		return Line;
	}

	TSharedRef<FTokenizedMessage> Warning(const FText& Message)
	{
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Warning);
		Line->AddToken(FTextToken::Create(Message));
		InternalLogMessage(Line);
		return Line;
	}

	TSharedRef<FTokenizedMessage> Note(const FText& Message)
	{
		TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Info);
		Line->AddToken(FTextToken::Create(Message));
		InternalLogMessage(Line);
		return Line;
	}

	virtual TSubclassOf<UUserWidget> GetContextClass() const = 0;
	
protected:
	virtual void InternalLogMessage(TSharedRef<FTokenizedMessage>& Message) = 0;
};

#endif