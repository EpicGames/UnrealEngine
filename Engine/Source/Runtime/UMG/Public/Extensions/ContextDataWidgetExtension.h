// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UserWidgetExtension.h"

#include "ContextDataWidgetExtension.generated.h"

/**
 * This extention is just a data container that can be used to attach arbitrary data to a widget
 */
UCLASS()
class UContextDataWidgetExtension : public UUserWidgetExtension
{
	GENERATED_BODY()
	
public:
	template <typename DataType>
	DataType* FindData() const
	{
		return CastChecked<DataType>(FindData(DataType::StaticClass()), ECastCheckedType::NullAllowed);
	}

	UObject* FindData(const UClass* Class) const
	{
		for (UObject* Data : ContextData)
		{
			if (Data && Data->IsA(Class)) // Can become null from GC
			{
				return Data;
			}
		}
		return nullptr;
	}

	void AddData(UObject* Data)
	{
		// Todo: Support multiple instances? What about existing data that is of this type?
		if (ensure(Data) && ensure(!FindData(Data->GetClass())))
		{
			ContextData.Add(Data);
		}
	}

private:
	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>> ContextData;
};
