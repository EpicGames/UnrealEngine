// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "FieldNotificationDeclaration.h"
#include "FieldNotificationDelegate.h"
#include "UObject/ObjectMacros.h"
#include "IFieldNotificationClassDescriptor.h"
#include "INotifyFieldValueChanged.h"
#include "Misc/DataValidation.h"

#include "UIComponent.generated.h"

class UEdGraph;
class UWidget;
class SWidget;

/** 
 * This is the base class to for UI Components that can be added to any UMG Widgets
 * in UMG Designer.When initialized, it will pass the widget it's attached to.
 */
UCLASS(Abstract, MinimalAPI, CustomFieldNotify)
class UUIComponent : public UObject, public INotifyFieldValueChanged
{
	GENERATED_BODY()

public:
	struct FFieldNotificationClassDescriptor : public ::UE::FieldNotification::IClassDescriptor
	{
		UMG_API virtual void ForEachField(const UClass* Class, TFunctionRef<bool(UE::FieldNotification::FFieldId FielId)> Callback) const override;
	};
	/**
	 * Called when the owner widget is initialized.
	 */
	UMG_API void Initialize(UWidget* Target);

	/**
	 * Called when the owner widget is pre-constructed. Called in both Editor and runtime.
	 * @param bIsDesignTime True when the Widget is constructed for design time
	 */
	UMG_API void PreConstruct(bool bIsDesignTime);
	
	/**
	 * Called when the owner widget is constructed.
	 */
	UMG_API void Construct();

	/**
	 * Called when the owner widget is destructed.
	 */
	UMG_API void Destruct();

	/**
	 * Called when a graph owned by the user widget is renamed.
	 */
	UMG_API virtual void GraphRenamed(UEdGraph* Graph, const FName& OldName, const FName& NewName);

	/**
	 * Returns the Owner Widget this component is attached to.
	 *  @returns The owner widget
	*/
	UMG_API TWeakObjectPtr<UWidget> GetOwner() const;
	
	/**
	* Called when the OwnerWidget is Rebuilt to allow to build a more complex hierarchy between the owner parents and the Owner widget.
	* @param Content The Owner Widget Content.
	* @returns The SWidget that wraps the Content passed.   
	*/
	UMG_API virtual TSharedRef<SWidget> RebuildWidgetWithContent(TSharedRef<SWidget> Content);

	//~ Begin INotifyFieldValueChanged Interface
	UMG_API virtual FDelegateHandle AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate) override final;
	UMG_API virtual bool RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle) override final;
	UMG_API virtual int32 RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject) override final;
	UMG_API virtual int32 RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject) override final;
	UMG_API virtual void BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId) override final;
	UMG_API virtual const UE::FieldNotification::IClassDescriptor& GetFieldNotificationDescriptor() const override;
	//~ End INotifyFieldValueChanged Interface

#if WITH_EDITOR
	/**
	 * Called during asset validation to validate this component in the context of its owner widget.
	 * Override to perform owner-aware validation. Calls IsDataValid by default.
	 * @param OwnerWidget	The widget this component is attached to, or nullptr if it could not be resolved.
	 * @param Context		The validation context to add errors/warnings to.
	 */
	UMG_API virtual EDataValidationResult IsDataValidForOwner(const UWidget* OwnerWidget, FDataValidationContext& Context) const;
#endif // WITH_EDITOR

protected:
	
	UMG_API virtual void OnInitialize();
	UMG_API virtual void OnPreConstruct(bool bIsDesignTime);
	UMG_API virtual void OnConstruct();
	UMG_API virtual void OnDestruct();
	UMG_API virtual void OnGraphRenamed(UEdGraph* Graph, const FName& OldName, const FName& NewName);

private:
	UPROPERTY(transient, DuplicateTransient)
	TWeakObjectPtr<UWidget> Owner;
	UE::FieldNotification::FFieldMulticastDelegate Delegates;

	bool bInitialized = false;
};
