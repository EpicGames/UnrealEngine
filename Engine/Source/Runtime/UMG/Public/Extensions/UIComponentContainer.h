// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "UIComponentContainer.generated.h"

#define UE_API UMG_API

class UUserWidget;
class UWidget;
class UUIComponent;
class UWidgetTree;

USTRUCT()
struct FUIComponentTarget
{
	GENERATED_BODY();
public:

	FUIComponentTarget();
	FUIComponentTarget(UUIComponent* Component, FName InChildName);

	/** Resolves the Widget ptr using it's name. */
	UE_API UWidget* Resolve(const class UWidgetTree* WidgetTree);
	FName GetTargetName() const { return TargetName; }
	void SetTargetName(FName NewName);
	UUIComponent* GetComponent() const { return Component; }
	bool IsValid() const { return !TargetName.IsNone() && Component != nullptr; }

private:

	// We use a TargetName to resolve the Widget only at compile time and on the Runtime Widget. 
	// It simplify edition in UMG Designer and make sure we do not need to keep Association in sync with the WidgetTree.
	UPROPERTY()
	FName TargetName;
	
	UPROPERTY(Instanced)
	TObjectPtr<UUIComponent> Component;
};


/**
 * Class that holds all the UIComponents for a UUserWidget.
 */
UCLASS(MinimalAPI, NotBlueprintType)
class UUIComponentContainer : public UObject
{
	GENERATED_BODY()

public:
	UE_API void AddComponent(FName TargetName, UUIComponent* Component);
	UE_API void RemoveComponent(FName TargetName, UUIComponent* Component);
	UE_API void RemoveAllComponentsOfType(const UClass* ComponentClass, FName TargetName);
	UE_API void RemoveAllComponentsFor(FName TargetName);

	UE_API void MoveComponent(FName TargetName, const UClass* ComponentClass, const UClass* RelativeToComponentClass, bool bMoveAfter);

	UE_API UUIComponent* GetComponent(const UClass* ComponentClass, FName TargetName) const; 
	
	UE_API void ForEachComponent(TFunctionRef<void(UUIComponent*)> Predicate) const;
	UE_API void ForEachComponentTarget(TFunctionRef<void(const FUIComponentTarget&)> Predicate) const;

	UE_API void InitializeComponents(const UUserWidget* UserWidget);
	
	UE_API bool IsEmpty() const;

	static UE_API FName GetPropertyNameForComponent(const UUIComponent* Component, const FName& TargetName);
#ifdef WITH_EDITOR
	UE_API void RenameWidget(const FName& OldName, const FName& NewName);
	UE_API void CleanupUIComponents(const UWidgetTree* WidgetTree);	
#endif //WITH_EDITOR
	
private:
	UPROPERTY()
	TArray<FUIComponentTarget> Components;
};

#undef UE_API
