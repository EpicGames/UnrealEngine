// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Extensions/UserWidgetExtension.h"

#include "UIComponentUserWidgetExtension.generated.h"

#define UE_API UMG_API

class UUserWidget;
class UWidget;
class UUIComponent;
class UUIComponentContainer;

/**
 * Class that holds all the UIComponents for a UUserWidget.
 */
UCLASS(MinimalAPI, NotBlueprintType, DefaultToInstanced, Experimental)
class UUIComponentUserWidgetExtension : public UUserWidgetExtension // This Contains all runtime version the components. They should be already resolved.
{
	GENERATED_BODY()

public:
	UE_API virtual void Initialize() override;
	UE_API virtual void PreConstruct(bool bIsDesignTime) override;
	UE_API virtual void Construct() override;
	UE_API virtual void Destruct() override;

	UE_API TArray<UUIComponent*> GetComponentsFor(UWidget* Target) const;
	UE_API UUIComponent* GetComponent(const UClass* ComponentClass, FName OwnerName) const;

	UE_API void InitializeContainer(UUIComponentContainer* InComponentContainer);
	UE_API bool IsContainerInitialized() const;
	

#if WITH_EDITOR
	UE_API virtual void InitializeInEditor() override;

	UE_API void RenameWidget(const FName& OldVarName, const FName& NewVarName);
	
	// Used only to create a Component on the PreviewWidget in the editor, based on the Component Archetype object in the WidgetBlueprint.
	UE_API void CreateAndAddComponent(UUIComponent* ArchetypeComponent, FName OwnerName);

	UE_API void RemoveComponent(const UClass* ComponentClass, FName OwnerName);
	
	UE_API void CleanupComponents();
	
	UE_API void MoveComponent(FName OwnerName, const UClass* ComponentClass, const UClass* RelativeToComponentClass, bool bMoveAfter);

#endif // WITH_EDITOR

private:
	void InitializeComponents();
	
	// Use a single TArray for the Entire UUserWidget to reduce memory usage.
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UUIComponentContainer> ComponentContainer;

	bool bInitialized = false;
};

#undef UE_API
