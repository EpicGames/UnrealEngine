// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Extensions/WidgetBlueprintGeneratedClassExtension.h"

#include "UIComponentWidgetBlueprintGeneratedClassExtension.generated.h"

#define UE_API UMG_API

class UUserWidget;
class UWidget;
class UUIComponent;
class UUIComponentContainer;

/**
 *  Class that hold the archetype version of the Components to be duplicated.
 */
UCLASS(MinimalAPI)
class UUIComponentWidgetBlueprintGeneratedClassExtension : public UWidgetBlueprintGeneratedClassExtension
{
	GENERATED_BODY()
public:

    UE_API virtual void Initialize(UUserWidget* UserWidget) override;

	UE_API void InitializeContainer(UUIComponentContainer* InComponentContainer);

	UE_API virtual void PreConstruct(UUserWidget* UserWidget, bool IsDesignTime) override;

#if WITH_EDITOR
	UE_API virtual void InitializeInEditor(UUserWidget* UserWidget) override;
	UE_API bool VerifyAllWidgetsExists(const UWidgetTree* WidgetTree) const;
#endif // WITH_EDITOR
	
private:
	UUIComponentContainer* DuplicateContainer(UUserWidget* UserWidget) const;
	bool VerifyContainer(const UUserWidget* UserWidget) const;
	
	UPROPERTY(DuplicateTransient)
	TObjectPtr<UUIComponentContainer> ComponentContainer;
};

#undef UE_API
