// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/NavigationMetadata.h"
#include "Extensions/UIComponent.h"
#include "UObject/ObjectMacros.h"

#include "NavigationUIComponent.generated.h"

class UWidget;

/**
* If Entered, Other Widget is the Widget Navigated From 
* If Exited , Other Widget is the Widget Navigated To
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnNavigationTransition, EUINavigation, Type, UWidget*, OldFocusedWidget, UWidget*, NewFocusedWidget);

UCLASS(BlueprintType, Experimental)
class UNavigationUIComponent : public UUIComponent
{
	GENERATED_BODY()

private:
	virtual void OnConstruct() override;
	virtual void OnDestruct() override;
	virtual void OnGraphRenamed(UEdGraph* Graph, const FName& OldName, const FName& NewName) override;

	void HandleNavigationTransition(const FNavigationTransition& NavigationTransition);

public:
	/** 
	* FNames are references to user provided functions. 
	* No direct access to properties, modifiable in DetailsView via FNavigationUIComponentCustomization::CustomizeNavigationTransitionFunction
	*/
	UPROPERTY()
	FName OnNavigationEntered; 

	UPROPERTY()
	FName OnNavigationExited;

private:

	/* Delegates are the concrete functions from the provided user provided function names */
	UPROPERTY(BlueprintAssignable, Category = "Navigation", meta = (AllowPrivateAccess = true, DisplayName = "On Navigation Entered"))
	FOnNavigationTransition OnNavigationEnteredDelegate;

	UPROPERTY(BlueprintAssignable, Category = "Navigation", meta = (AllowPrivateAccess = true, DisplayName = "On Navigation Exited"))
	FOnNavigationTransition OnNavigationExitedDelegate;

	TSharedPtr<FNavigationTransitionMetadata> NavigationTransitionMetadata;
};


