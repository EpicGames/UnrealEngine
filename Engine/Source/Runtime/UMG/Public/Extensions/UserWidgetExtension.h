// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "UserWidgetExtension.generated.h"


class UUserWidget;

/**
 * UserWidgetExtension is the base class for components that define reusable behavior that can be added to different types of Widgets.
 */
UCLASS(Abstract, DefaultToInstanced, Within=UserWidget, MinimalAPI)
class UUserWidgetExtension : public UObject
{
	GENERATED_BODY()
	
public:

#ifdef WITH_EDITOR
	/** Initialize function called only in Editor */
	virtual void InitializeInEditor() { }
#endif

	/** Extend the UUserWidget::Initialize function */
	virtual void Initialize()
	{
	}

	/** Extend the UUserWidget::PreConstruct function */
	virtual void PreConstruct(bool bIsDesignTime)
	{		
	}

	/** Extend the UUserWidget::Construct function */
	virtual void Construct()
	{
	}

	/** Extend the UUserWidget::Destruct function */
	virtual void Destruct()
	{
	}

	/** Does the extension requires tick. */
	virtual bool RequiresTick() const
	{
		return false;
	}

	/**
	 * Extend the UUserWidget::Tick function.
	 * If the UserWidget ticks, then all extensions will tick regardless of RequiresTick.
	 */
	virtual void Tick(const FGeometry& MyGeometry, float InDeltaTime)
	{
	}

protected:
	UUserWidget* GetUserWidget() const
	{
		return GetOuterUUserWidget();
	}
};
