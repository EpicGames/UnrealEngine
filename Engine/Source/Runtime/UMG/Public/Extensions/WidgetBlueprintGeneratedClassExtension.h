// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/Widget.h"

#include "WidgetBlueprintGeneratedClassExtension.generated.h"


/**
 * WidgetExtension is the base class for components that define reusable behavior that can be added to different types of Widgets.
 */
UCLASS(Abstract, DefaultToInstanced, MinimalAPI)
class UWidgetBlueprintGeneratedClassExtension : public UObject
{
	GENERATED_BODY()
	
public:

#ifdef WITH_EDITOR
	/** Initialize function called only in Editor */
	virtual void InitializeInEditor(UUserWidget* UserWidget) { }
#endif
	/**
	 * Extend the UUserWidget::Initialize function.
	 * Called once when the UUserWidget is created.
	 */
	virtual void Initialize(UUserWidget* UserWidget) { }
	/** Extend the UUserWidget::PreConstruct function */
	virtual void PreConstruct(UUserWidget* UserWidget, bool IsDesignTime) { }
	/**
	 * Extend the UUserWidget::Construct function.
	 * Called after when the SWidget is created. (TakeWidget)
	 */
	virtual void Construct(UUserWidget* UserWidget) { }
	/**
	 * Extend the UUserWidget::Destruct function.
	 * Called when the SWidget is going to be destroy. (ReleaseSlateWidget)
	 */
	virtual void Destruct(UUserWidget* UserWidget) { }
};
