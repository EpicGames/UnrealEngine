// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "FieldNotificationVariant.h"

UE_DEPRECATED(5.3, "Including FieldNotification/FieldVariant.h is deprecated. The file location changed. Please include FieldNotificationVariant.h instead and add the module FieldNotification to your .build.cs")
inline void FieldVariantHeaderDeprecatedWarning()
{
}

inline void TriggerFieldVariantHeaderDeprecatedWarning()
{
	FieldVariantHeaderDeprecatedWarning();
}
