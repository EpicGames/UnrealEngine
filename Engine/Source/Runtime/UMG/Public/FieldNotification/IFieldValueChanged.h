// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "INotifyFieldValueChanged.h"

UE_DEPRECATED(5.3, "Including FieldNotification/IFieldValueChanged.h is deprecated. The file location changed. Please include INotifyFieldValueChanged.h instead and add the module FieldNotification to your .build.cs")
inline void IFieldValueChangedHeaderDeprecatedWarning()
{
}

inline void TriggerIFieldValueChangedHeaderDeprecatedWarning()
{
	IFieldValueChangedHeaderDeprecatedWarning();
}
