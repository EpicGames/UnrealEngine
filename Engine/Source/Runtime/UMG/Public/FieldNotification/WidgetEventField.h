// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WidgetEventField.generated.h"

/**
 * Generic structure to notify when an event occurs.
 * 
 * class UMyWidget : public UWidget
 * {
 *   UPROPERTY(FieldNotify)
 *   FWidgetEventField SomeEvent;
 *
 *   void OnSomeEvent()
 *   {
 *     BroadcastFieldValueChanged(FFieldNotificationClassDescriptor::Text);
 *   }
 * };
};
 */
USTRUCT(BlueprintType)
struct FWidgetEventField
{
	GENERATED_BODY()
};
