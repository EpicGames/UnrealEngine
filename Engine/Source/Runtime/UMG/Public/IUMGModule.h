// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for the UMG Module.
 */
class IUMGModule : public IModuleInterface
{
public:

};
