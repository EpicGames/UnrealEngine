// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Textures/SlateShaderResource.h"
#include "Rendering/RenderingCommon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class UMaterialInstanceDynamic;
class USlateVectorArtData;
struct FSlateBrush;

/**
 * A widget that draws vertexes provided by a 2.5D StaticMesh.
 * The Mesh's material is used.
 * Hardware instancing is supported.
 */
class SMeshWidget : public SLeafWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SMeshWidget)
		: _MeshData(nullptr)
	{}
		/** The StaticMesh asset that should be drawn. */
		SLATE_ARGUMENT(USlateVectorArtData*, MeshData)
	SLATE_END_ARGS()

	UMG_API void Construct(const FArguments& Args);

	/**
	 * Draw the InStaticMesh when this widget paints.
	 *
	 * @return the Index of the mesh data that was added; cache this value for use with @see FRenderRun.
	 */
	UMG_API uint32 AddMesh(USlateVectorArtData& InMeshData);

	/** Much like AddMesh, but also enables instancing support for this MeshId. */
	UMG_API uint32 AddMeshWithInstancing(USlateVectorArtData& InMeshData, int32 InitialBufferSize = 1);

	/**
	 * Switch from static material to material instance dynamic.
	 * 
	 * @param MeshId    The index of the mesh; returned from AddMesh.
	 * 
	 * @return The MID for this Asset on which parameters can be set.
	 */
	UMG_API UMaterialInstanceDynamic* ConvertToMID( uint32 MeshId );

	/** Discard any previous runs and reserve space for new render runs if needed. */
	UMG_API void ClearRuns(int32 NumRuns);

	/**
	 * Tell the widget to draw instances of a mesh a given number of times starting at
	 * a given offset.
	 *
	 * @param InMeshIndex        Which mesh to draw; returned by @see AddMesh
	 * @param InInstanceOffset   Start drawing with this instance
	 * @param InNumInstances     Draw this many instances
	 */
	inline void AddRenderRun(uint32 InMeshIndex, uint32 InInstanceOffset, uint32 InNumInstances)
	{
		RenderRuns.Add(FRenderRun(InMeshIndex, InInstanceOffset, InNumInstances));
	}

	/** Enable hardware instancing */
	UMG_API void EnableInstancing(uint32 MeshId, int32 InitialSize);

	/** Updates the per instance buffer. Automatically enables hardware instancing. */
	UMG_API void UpdatePerInstanceBuffer(uint32 MeshId, FSlateInstanceBufferData& Data);

protected:
	// BEGIN SLeafWidget interface
	UMG_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UMG_API virtual FVector2D ComputeDesiredSize(float) const override;
	// END SLeafWidget interface

	// ~ FGCObject
	UMG_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UMG_API virtual FString GetReferencerName() const override;
	// ~ FGCObject

protected:
	static UMG_API void PushUpdate(uint32 VectorArtId, SMeshWidget& Widget, const FVector2D& Position, float Scale, uint32 BaseAddress);
	static UMG_API void PushUpdate(uint32 VectorArtId, SMeshWidget& Widget, const FVector2D& Position, float Scale, float OptionalFloat = 0);

	struct FRenderData
	{
		/** Holds a copy of the Static Mesh's data converted to a format that Slate understands. */
		TArray<FSlateVertex> VertexData;
		/** Connectivity data: Order in which the vertexes occur to make up a series of triangles. */
		TArray<SlateIndex> IndexData;
		/** Holds on to the material that is found on the StaticMesh. */
		TSharedPtr<FSlateBrush> Brush;
		/** A rendering handle used to quickly access the rendering data for the slate element*/
		FSlateResourceHandle RenderingResourceHandle;
		/** Per instance data that can be passed to */
		TSharedPtr<ISlateUpdatableInstanceBuffer> PerInstanceBuffer;
	};
	TArray<FRenderData, TInlineAllocator<3>> RenderData;

private:
	/** Which mesh to draw, starting with which instance offset and how many instances to draw in this run/batch. */
	class FRenderRun
	{
	public:
		FRenderRun(uint32 InMeshIndex, uint32 InInstanceOffset, uint32 InNumInstances)
			: MeshIndex(InMeshIndex)
			, InstanceOffset(InInstanceOffset)
			, NumInstances(InNumInstances)
		{
		}

		uint32 GetMeshIndex() const { return MeshIndex; }
		uint32 GetInstanceOffset() const { return InstanceOffset; }
		uint32 GetNumInstances() const { return NumInstances; }

	private:
		uint32 MeshIndex;
		uint32 InstanceOffset;
		uint32 NumInstances;
	};
	TArray<FRenderRun> RenderRuns;
};
