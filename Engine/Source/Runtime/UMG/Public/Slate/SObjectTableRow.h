// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Slate/SObjectWidget.h"
#include "Blueprint/IUserObjectListEntry.h"
#include "Blueprint/WidgetLayoutLibrary.h"
#include "Blueprint/DragDropOperation.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Slate/UMGDragDropOp.h"


class IObjectTableRow : public ITableRow
{
public:
	virtual UListViewBase* GetOwningListView() const = 0;
	virtual UUserWidget* GetUserWidget() const = 0;

	static TSharedPtr<const IObjectTableRow> ObjectRowFromUserWidget(const UUserWidget* RowUserWidget)
	{
		TWeakPtr<const IObjectTableRow>* ObjectRow = ObjectRowsByUserWidget.Find(RowUserWidget);
		if (ObjectRow && ObjectRow->IsValid())
		{
			return ObjectRow->Pin();
		}
		return nullptr;
	}

protected:
	// Intentionally being a bit nontraditional here - we track associations between UserWidget rows and their underlying IObjectTableRow.
	// This allows us to mirror the ITableRow API very easily on IUserListEntry without requiring rote setting/getting of row states on every UMG subclass.
	UMG_API static TMap<TWeakObjectPtr<const UUserWidget>, TWeakPtr<const IObjectTableRow>> ObjectRowsByUserWidget;
};

DECLARE_DELEGATE_OneParam(FOnRowHovered, UUserWidget&);
DECLARE_DELEGATE_RetVal_ThreeParams(TOptional<EItemDropZone>, FOnObjectRowCanAcceptDrop, const FDragDropEvent&, EItemDropZone, UUserWidget&);
DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FOnObjectRowAcceptDrop, const FDragDropEvent&, EItemDropZone, UUserWidget&);
DECLARE_DELEGATE_RetVal_ThreeParams(UDragDropOperation*, FOnObjectRowDragDetected, const FGeometry&, const FPointerEvent&, UUserWidget&);
DECLARE_DELEGATE_TwoParams(FOnObjectRowDragEnter, FDragDropEvent const&, UUserWidget&);
DECLARE_DELEGATE_TwoParams(FOnObjectRowDragLeave, FDragDropEvent const&, UUserWidget&);
DECLARE_DELEGATE_OneParam(FOnObjectRowDragCancelled, const FDragDropEvent&);

class UListViewBase;

/** 
 * It's an SObjectWidget! It's an ITableRow! It does it all!
 *
 * By using UUserWidget::TakeDerivedWidget<T>(), this class allows UMG to fully leverage the robust Slate list view widgets.
 * The major gain from this is item virtualization, which is an even bigger deal when unnecessary widgets come with a boatload of additional UObject allocations.
 * 
 * The owning UUserWidget is expected to implement the IUserListEntry UInterface, which allows the row widget to respond to various list-related events.
 *
 * Note: Much of the implementation here matches STableRow<T> exactly, so refer there if looking for additional information.
 */
template <typename ItemType>
class SObjectTableRow : public SObjectWidget, public IObjectTableRow
{
public:
	SLATE_BEGIN_ARGS(SObjectTableRow<ItemType>)
		:_bAllowDragging(true)
		,_bAllowDragDrop(false)
	{}
		SLATE_ARGUMENT(bool, bAllowDragging)
		SLATE_ARGUMENT(bool, bAllowDragDrop)
		SLATE_ARGUMENT(bool, bAllowKeepPreselectedItems)
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FOnRowHovered, OnHovered)
		SLATE_EVENT(FOnRowHovered, OnUnhovered)
		//	Drag and Drop functionality
		SLATE_EVENT(FOnObjectRowCanAcceptDrop, OnRowCanAcceptDrop)
		SLATE_EVENT(FOnObjectRowAcceptDrop, OnRowAcceptDrop)
		SLATE_EVENT(FOnObjectRowDragDetected, OnRowDragDetected)
		SLATE_EVENT(FOnObjectRowDragEnter, OnRowDragEnter)
		SLATE_EVENT(FOnObjectRowDragLeave, OnRowDragLeave)
		SLATE_EVENT(FOnObjectRowDragCancelled, OnRowDragCancelled)
		//	End Drag and Drop
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, UUserWidget& InWidgetObject, UListViewBase* InOwnerListView = nullptr)
	{
		TSharedPtr<SWidget> ContentWidget;

		if (ensureMsgf(InWidgetObject.Implements<UUserListEntry>(), TEXT("Any UserWidget generated as a table row must implement the IUserListEntry interface")))
		{
			ObjectRowsByUserWidget.Add(&InWidgetObject, SharedThis(this));

			OwnerListView = InOwnerListView;
			OwnerTablePtr = StaticCastSharedRef<SListView<ItemType>>(InOwnerTableView);

			bAllowDragging = InArgs._bAllowDragging;
			bAllowDragDrop = InArgs._bAllowDragDrop;
			bAllowKeepPreselectedItems = InArgs._bAllowKeepPreselectedItems;
			OnHovered = InArgs._OnHovered;
			OnUnhovered = InArgs._OnUnhovered;
			OnRowCanAcceptDrop = InArgs._OnRowCanAcceptDrop;
			OnRowAcceptDrop = InArgs._OnRowAcceptDrop;
			OnRowDragDetected = InArgs._OnRowDragDetected;
			OnRowDragLeave = InArgs._OnRowDragLeave;
			OnRowDragEnter = InArgs._OnRowDragEnter;
			OnRowDragCancelled = InArgs._OnRowDragCancelled;
			ContentWidget = InArgs._Content.Widget;
		}
		else
		{
			ContentWidget = SNew(STextBlock)
				.Text(NSLOCTEXT("SObjectTableRow", "InvalidWidgetClass", "Any UserWidget generated as a table row must implement the IUserListEntry interface"));
		}

		SObjectWidget::Construct(
			SObjectWidget::FArguments()
			.Content()
			[
				ContentWidget.ToSharedRef()
			], &InWidgetObject);

		// Register an active timer, not an OnTick to determine if item selection changed.
		// If we use OnTick, it will be potentially stomped by DisableNativeTick, when the
		// SObjectTableRow is used to wrap the UUserWidget construction.
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SObjectTableRow::DetectItemSelectionChanged));
	}

	virtual ~SObjectTableRow()
	{
		// Remove the association between this widget and its user widget
		ObjectRowsByUserWidget.Remove(WidgetObject);
	}

	virtual UUserWidget* GetUserWidget() const
	{
		return WidgetObject;
	}

	virtual UListViewBase* GetOwningListView() const
	{
		if (OwnerListView.IsValid())
		{
			return OwnerListView.Get();
		}
		return nullptr;
	}

	EActiveTimerReturnType DetectItemSelectionChanged(double InCurrentTime, float InDeltaTime)
	{
		DetectItemSelectionChanged();
		return EActiveTimerReturnType::Continue;
	}

	virtual void NotifyItemExpansionChanged(bool bIsExpanded)
	{
		if (WidgetObject)
		{
			IUserListEntry::UpdateItemExpansion(*WidgetObject, bIsExpanded);
		}
	}

	//~ ITableRow interface
	virtual void InitializeRow() override final
	{
		// ObjectRows can be generated in the widget designer with dummy data, which we want to ignore
		if (WidgetObject && !WidgetObject->IsDesignTime())
		{
			InitializeObjectRow();
		}
	}

	virtual void ResetRow() override final
	{
		if (WidgetObject && !WidgetObject->IsDesignTime())
		{
			ResetObjectRow();
		}
	}

	virtual TSharedRef<SWidget> AsWidget() override
	{
		return SharedThis(this);
	}
	virtual void SetIndexInList(int32 InIndexInList) override
	{
		IndexInList = InIndexInList;
	}
	virtual int32 GetIndexInList() override
	{
		return IndexInList;
	}
	virtual TSharedPtr<SWidget> GetContent() override
	{
		return ChildSlot.GetChildAt(0);
	}
	virtual int32 GetIndentLevel() const override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		return OwnerTable ? OwnerTable->Private_GetNestingDepth(IndexInList) : 0;
	}
	virtual int32 DoesItemHaveChildren() const override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		return OwnerTable ? OwnerTable->Private_DoesItemHaveChildren(IndexInList) : 0;
	}
	virtual void Private_OnExpanderArrowShiftClicked() override
	{
		/* Intentionally blank - far too specific to be a valid game UI interaction */
	}
	virtual ESelectionMode::Type GetSelectionMode() const override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		return OwnerTable ? OwnerTable->Private_GetSelectionMode() : ESelectionMode::None;
	}
	virtual FVector2D GetRowSizeForColumn(const FName& InColumnName) const override
	{
		return FVector2D::ZeroVector;
	}

	virtual bool IsItemExpanded() const override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable ? GetItemForThis(OwnerTable.ToSharedRef()) : nullptr;
		if (MyItemPtr)
		{
			return OwnerTable->Private_IsItemExpanded(*MyItemPtr);
		}

		return false;
	}

	virtual void ToggleExpansion() override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		if (OwnerTable && OwnerTable->Private_DoesItemHaveChildren(IndexInList))
		{
			if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable.ToSharedRef()))
			{
				OwnerTable->Private_SetItemExpansion(*MyItemPtr, !OwnerTable->Private_IsItemExpanded(*MyItemPtr));
			}
		}
	}

	virtual bool IsItemSelected() const override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable ? GetItemForThis(OwnerTable.ToSharedRef()) : nullptr;
		if (MyItemPtr)
		{
			return OwnerTable->Private_IsItemSelected(*MyItemPtr);
		}

		return false;
	}

	virtual TBitArray<> GetWiresNeededByDepth() const override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		return OwnerTable ? OwnerTable->Private_GetWiresNeededByDepth(IndexInList) : TBitArray<>();
	}

	virtual bool IsLastChild() const override
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		return OwnerTable ? OwnerTable->Private_IsLastChild(IndexInList) : true;
	}

	//~ ITableRow interface

	//~ SWidget interface
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		SObjectWidget::OnMouseEnter(MyGeometry, MouseEvent);
		if (WidgetObject && OnHovered.IsBound())
		{
			OnHovered.ExecuteIfBound(*WidgetObject);
		}
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SObjectWidget::OnMouseLeave(MouseEvent);
		if (WidgetObject && OnUnhovered.IsBound())
		{
			OnUnhovered.ExecuteIfBound(*WidgetObject);
		}
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
			const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable ? GetItemForThis(OwnerTable.ToSharedRef()) : nullptr;
			if (MyItemPtr)
			{
				OwnerTable->Private_OnItemDoubleClicked(*MyItemPtr);
				return FReply::Handled();
			}
		}
		return FReply::Unhandled();
	}

	virtual FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		//TODO: FReply Reply = SObjectWidget::OnTouchStarted(MyGeometry, InTouchEvent);
		bProcessingSelectionTouch = true;

		return FReply::Handled()
			.DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		FReply Reply = SObjectWidget::OnTouchEnded(MyGeometry, InTouchEvent);

		if (bProcessingSelectionTouch)
		{
			bProcessingSelectionTouch = false;
			TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
			const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable ? GetItemForThis(OwnerTable.ToSharedRef()) : nullptr;
			if (MyItemPtr)
			{
				if (IsItemSelectable())
				{
					ESelectionMode::Type SelectionMode = GetSelectionMode();
					if (SelectionMode != ESelectionMode::None)
					{
						const bool bIsSelected = OwnerTable->Private_IsItemSelected(*MyItemPtr);
						if (!bIsSelected)
						{
							if (SelectionMode != ESelectionMode::Multi)
							{
								OwnerTable->Private_ClearSelection();
							}
							OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
							OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);

							Reply = FReply::Handled();
						}
						else if (SelectionMode == ESelectionMode::SingleToggle || SelectionMode == ESelectionMode::Multi)
						{
							OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
							OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);

							Reply = FReply::Handled();
						}
					}
				}

				if (OwnerTable->Private_OnItemClicked(*MyItemPtr))
				{
					Reply = FReply::Handled();
				}
			}
		}

		return Reply;
	}

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = FReply::Unhandled();

		if (bAllowDragging || bAllowDragDrop)
		{
			TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();

			if (bProcessingSelectionTouch)
			{
				bProcessingSelectionTouch = false;
				return OwnerTable ? FReply::Handled().CaptureMouse(OwnerTable->AsWidget()) : FReply::Unhandled();
			}
			//@todo DanH TableRow: does this potentially trigger twice? If not, why does an unhandled drag detection result in not calling mouse up?
			else if (HasMouseCapture() && bChangedSelectionOnMouseDown)
			{
				// Do not change the selection on mouse up if we are dragging
				bDragWasDetected = true;

				if (OwnerTable)
				{
					OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
				}
			}

			Reply = SObjectWidget::OnDragDetected(MyGeometry, MouseEvent);
			if (!Reply.IsEventHandled() && bAllowDragDrop)
			{
				if (OnRowDragDetected.IsBound() && WidgetObject)
				{
					// Allow the view to create and pass back a UMG drag drop operation using config data
					UDragDropOperation* Operation = OnRowDragDetected.Execute(MyGeometry, MouseEvent, *WidgetObject);
					if (Operation)
					{
						// Only update the entry drag over state if a valid operation was created
						IUserListEntry::UpdateEntryDragOverState(*WidgetObject, true);

						FVector2D ScreenCursorPos = MouseEvent.GetScreenSpacePosition();
						FVector2D ScreenDragPosition = MyGeometry.GetAbsolutePosition();
						float DPIScale = UWidgetLayoutLibrary::GetViewportScale(WidgetObject);
						uint32 PointerIndex = MouseEvent.GetPointerIndex();

						TSharedRef<FUMGDragDropOp> DragDropOp = FUMGDragDropOp::New(Operation, PointerIndex, ScreenCursorPos, ScreenDragPosition, DPIScale, SharedThis(this));

						// Set the item value of the row to the drag drop visual.  
						// If the drag visual is not a UUserWidget this will have to be done manually by binding to the list OnDragDetected delegate.
						UUserWidget* DragVisualWidget = Cast<UUserWidget>(Operation->DefaultDragVisual.Get());
						if (DragVisualWidget && DragVisualWidget->Implements<UUserObjectListEntry>())
						{
							IUserObjectListEntry::SetListItemObject(*DragVisualWidget, Operation->Payload);
						}

						return FReply::Handled().BeginDragDrop(DragDropOp);
					}
					return FReply::Unhandled();
				}
			}
		}
		bProcessingSelectionTouch = false;
		return Reply;
	}

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		SObjectWidget::OnDragEnter(MyGeometry, DragDropEvent);

		if (WidgetObject && OnRowDragEnter.IsBound())
		{
			IUserListEntry::UpdateEntryDragOverState(*WidgetObject, true);
			OnRowDragEnter.Execute(DragDropEvent, *WidgetObject);
		}
	}

	virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override
	{
		SObjectWidget::OnDragLeave(DragDropEvent);

		// Clear out the current drop zone when we leave the drag area
		ItemDropZone = NullOpt;

		if (WidgetObject && OnRowDragLeave.IsBound())
		{
			IUserListEntry::UpdateEntryDragOverState(*WidgetObject, false);
			OnRowDragLeave.Execute(DragDropEvent, *WidgetObject);
		}
	}

	/** @return the zone (above, onto, below) based on where the user is hovering over within the row */
	EItemDropZone ZoneFromPointerPosition(UE::Slate::FDeprecateVector2DParameter LocalPointerPos, UE::Slate::FDeprecateVector2DParameter LocalSize, EOrientation Orientation)
	{
		const float PointerPos = Orientation == EOrientation::Orient_Horizontal ? LocalPointerPos.X : LocalPointerPos.Y;
		const float Size = Orientation == EOrientation::Orient_Horizontal ? LocalSize.X : LocalSize.Y;

		const float ZoneBoundarySu = FMath::Clamp(Size * 0.25f, 3.0f, 10.0f);
		if (PointerPos < ZoneBoundarySu)
		{
			return EItemDropZone::AboveItem;
		}
		else if (PointerPos > Size - ZoneBoundarySu)
		{
			return EItemDropZone::BelowItem;
		}
		else
		{
			return EItemDropZone::OntoItem;
		}
	}

	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		FReply Reply = SObjectWidget::OnDragOver(MyGeometry, DragDropEvent);

		if (OnRowCanAcceptDrop.IsBound() && !Reply.IsEventHandled())
		{
			const TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
			const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
			const EItemDropZone ItemHoverZone = ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize(), OwnerTable->Private_GetOrientation());

			ItemDropZone = [ItemHoverZone, DragDropEvent, this]()
				{
					if (WidgetObject)
					{
						return OnRowCanAcceptDrop.Execute(DragDropEvent, ItemHoverZone, *WidgetObject);
					}
					return TOptional<EItemDropZone>();
				}();

			if (WidgetObject)
			{
				IUserListEntry::UpdateEntryDropIndicator(*WidgetObject, ItemDropZone);
			}
		}

		return Reply;
	}

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		if (!bAllowDragDrop)
		{
			return FReply::Unhandled();
		}

		FReply Reply = SObjectWidget::OnDrop(MyGeometry, DragDropEvent);

		if (OnRowAcceptDrop.IsBound() && !Reply.IsEventHandled())
		{
			const TSharedRef< ITypedTableView<ItemType> > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

			// A drop finishes the drag/drop operation, so we are no longer providing any feedback.
			ItemDropZone = TOptional<EItemDropZone>();

			// Find item associated with this widget.
			if (const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable))
			{
				// Which physical drop zone is the drop about to be performed onto?
				const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
				EItemDropZone HoveredZone = ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize(), OwnerTable->Private_GetOrientation());
				 
				// The row gets final say over which zone to drop onto regardless of physical location.
				TOptional<EItemDropZone> ReportedZone;
				if (WidgetObject && OnRowCanAcceptDrop.IsBound())
				{
					ReportedZone = OnRowCanAcceptDrop.Execute(DragDropEvent, HoveredZone, *WidgetObject);
				}
				else
				{
					ReportedZone = HoveredZone;
				}
				 
				if (ReportedZone.IsSet())
				{
					Reply = FReply::Unhandled();
					if (WidgetObject)
					{
						Reply = OnRowAcceptDrop.Execute(DragDropEvent, ReportedZone.GetValue(), *WidgetObject);
						bool DropReplyHandled = false;
						if (Reply.IsEventHandled())
						{
							DropReplyHandled = true;
						}
						IUserListEntry::EndEntryDropOperation(*WidgetObject, DropReplyHandled);
					}

					if (Reply.IsEventHandled() && ReportedZone.GetValue() == EItemDropZone::OntoItem)
					{
						// Expand the drop target just in case, so that what we dropped is visible.
						OwnerTable->Private_SetItemExpansion(*MyItemPtr, true);
					}

					return Reply;
				}
				else
				{
					// If CanAcceptDrop returned a negative result, handle and cancel the drop event.
					OnDragCancelled(DragDropEvent, nullptr);
					return FReply::Handled().EndDragDrop();
				}
			}
			// If there is not an item associated with this widget, the drop is unhandled
			return FReply::Unhandled();
		}
		// If there is no listener bound|dragging disabled|reply already handled - return the reply
		return Reply;
	}

	void OnDragCancelled(const FDragDropEvent& DragDropEvent, UDragDropOperation* Operation) override
	{
		SObjectWidget::OnDragCancelled(DragDropEvent, Operation);

		if (WidgetObject && OnRowDragCancelled.IsBound())
		{
			IUserListEntry::EndEntryDropOperation(*WidgetObject, false);
			OnRowDragCancelled.Execute(DragDropEvent);
		}
	}

	virtual void HandleEntryDropped(UDragDropOperation* Operation)
	{
		if (WidgetObject)
		{
			IUserListEntry::HandleEntryDropped(*WidgetObject, Operation);
		}
	}

	virtual void HandleEntryDragged(UDragDropOperation* Operation)
	{
		if (WidgetObject)
		{
			IUserListEntry::HandleEntryDragged(*WidgetObject, Operation);
		}
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		bChangedSelectionOnMouseDown = false;
		bDragWasDetected = false;

		FReply Reply = SObjectWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
		if (!Reply.IsEventHandled())
		{
			TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
			if (OwnerTable)
			{
				const ESelectionMode::Type SelectionMode = GetSelectionMode();
				if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && SelectionMode != ESelectionMode::None)
				{
					if (IsItemSelectable())
					{
						const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = GetItemForThis(OwnerTable.ToSharedRef());
						// New selections are handled on mouse down, deselection is handled on mouse up
						if (MyItemPtr)
						{
							const ItemType& MyItem = *MyItemPtr;
							const bool bIsSelected = OwnerTable->Private_IsItemSelected(MyItem);

							if (SelectionMode == ESelectionMode::Multi)
							{
								if (MouseEvent.IsShiftDown())
								{
									OwnerTable->Private_SelectRangeFromCurrentTo(MyItem);
									bChangedSelectionOnMouseDown = true;

								}
								else if (MouseEvent.IsControlDown())
								{
									OwnerTable->Private_SetItemSelection(MyItem, !bIsSelected, true);
									bChangedSelectionOnMouseDown = true;
								}
							}

							if (!bIsSelected && !bChangedSelectionOnMouseDown)
							{
								if (SelectionMode != ESelectionMode::Multi || !bAllowKeepPreselectedItems)
								{
									OwnerTable->Private_ClearSelection();
								}
								OwnerTable->Private_SetItemSelection(MyItem, true, true);
								bChangedSelectionOnMouseDown = true;
							}
						}

						Reply = FReply::Handled()
							.DetectDrag(SharedThis(this), EKeys::LeftMouseButton)
							.CaptureMouse(SharedThis(this));

						// Set focus back to the owning widget if the item is invalid somehow or its not selectable or can be navigated to
						if (!MyItemPtr || !OwnerTable->Private_IsItemSelectableOrNavigable(*MyItemPtr))
						{
							Reply.SetUserFocus(OwnerTable->AsWidget(), EFocusCause::Mouse);
						}
					}
				}
			}
		}
		
		return Reply;
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = SObjectWidget::OnMouseButtonUp(MyGeometry, MouseEvent);

		if (!Reply.IsEventHandled())
		{
			TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
			const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable ? GetItemForThis(OwnerTable.ToSharedRef()) : nullptr;
			if (MyItemPtr)
			{
				const ESelectionMode::Type SelectionMode = GetSelectionMode();
				
				if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && HasMouseCapture())
				{
					bool bSignalSelectionChanged = bChangedSelectionOnMouseDown;

					// Don't change selection on mouse up if it already changed on mouse down.  Don't change the selection if we are dragging.
					if (!bChangedSelectionOnMouseDown && IsItemSelectable() && MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()) && !bDragWasDetected)
					{
						if (SelectionMode == ESelectionMode::SingleToggle)
						{
							OwnerTable->Private_ClearSelection();
							bSignalSelectionChanged = true;
						}
						else if (SelectionMode == ESelectionMode::Multi &&
							OwnerTable->Private_GetNumSelectedItems() > 1 &&
							OwnerTable->Private_IsItemSelected(*MyItemPtr))
						{
							if (!MouseEvent.IsControlDown() && !MouseEvent.IsShiftDown())
							{
								// Releasing mouse over one of the multiple selected items - leave this one as the sole selected item
								OwnerTable->Private_ClearSelection();
								OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
								bSignalSelectionChanged = true;
							}
						}
					}

					if (bSignalSelectionChanged)
					{
						OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
						Reply = FReply::Handled();
					}

					if (OwnerTable->Private_OnItemClicked(*MyItemPtr))
					{
						Reply = FReply::Handled();
					}

					Reply = Reply.ReleaseMouseCapture();
				}
				else if (SelectionMode != ESelectionMode::None && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
				{
					// Ignore the right click release if it was being used for scrolling
					TSharedPtr<STableViewBase> OwnerTableViewBase = StaticCastSharedPtr<SListView<ItemType>>(OwnerTable);
					if (!OwnerTableViewBase->IsRightClickScrolling())
					{
						if (IsItemSelectable() && !OwnerTable->Private_IsItemSelected(*MyItemPtr))
						{
							// If this item isn't selected, it becomes the sole selected item. Otherwise we leave selection untouched.
							OwnerTable->Private_ClearSelection();
							OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
							OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
						}

						OwnerTable->Private_OnItemRightClicked(*MyItemPtr, MouseEvent);

						Reply = FReply::Handled();
					}
				}
			}
		}
		return Reply;
	}
	// ~SWidget interface

	bool GetAllowDragDrop() const
	{
		return bAllowDragDrop;
	}

protected:
	virtual void InitializeObjectRow()
	{
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable ? GetItemForThis(OwnerTable.ToSharedRef()) : nullptr;
		if (MyItemPtr)
		{
			InitObjectRowInternal(*WidgetObject, *MyItemPtr);

			// Unselectable items should never be selected
			if (!ensure(!OwnerTable->Private_IsItemSelected(*MyItemPtr) || IsItemSelectable()))
			{
				OwnerTable->Private_SetItemSelection(*MyItemPtr, false, false);
			}
		}
	}

	virtual void ResetObjectRow()
	{
		bIsAppearingSelected = false;
		if (WidgetObject)
		{
			IUserListEntry::ReleaseEntry(*WidgetObject);
		}
	}

	virtual void DetectItemSelectionChanged()
	{
		// List views were built assuming the use of attributes on rows to check on selection status, so there is no
		// clean way to inform individual rows of changes to the selection state of their current items.
		// Since event-based selection changes are only really needed in a game scenario, we (crudely) monitor it here to generate events.
		// If desired, per-item selection events could be added as a longer-term todo
		TSharedPtr<ITypedTableView<ItemType>> OwnerTable = OwnerTablePtr.Pin();
		const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable ? GetItemForThis(OwnerTable.ToSharedRef()) : nullptr;
		if (MyItemPtr)
		{
			if (bIsAppearingSelected != OwnerTable->Private_IsItemSelected(*MyItemPtr))
			{
				bIsAppearingSelected = !bIsAppearingSelected;
				OnItemSelectionChanged(bIsAppearingSelected);
			}
		}
	}

	virtual void OnItemSelectionChanged(bool bIsItemSelected)
	{
		if (WidgetObject)
		{
			IUserListEntry::UpdateItemSelection(*WidgetObject, bIsItemSelected);
		}
	}

	bool IsItemSelectable() const 
	{
		IUserListEntry* NativeListEntryImpl = Cast<IUserListEntry>(WidgetObject);
		return NativeListEntryImpl ? NativeListEntryImpl->IsListItemSelectable() : true;
	}

	const TObjectPtrWrapTypeOf<ItemType>* GetItemForThis(const TSharedRef<ITypedTableView<ItemType>>& OwnerTable) const
	{
		const TObjectPtrWrapTypeOf<ItemType>* MyItemPtr = OwnerTable->Private_ItemFromWidget(this);
		if (MyItemPtr)
		{
			return MyItemPtr;
		}
		else
		{
			checkf(OwnerTable->Private_IsPendingRefresh(), TEXT("We were unable to find the item for this widget. If it was removed from the source collection, the list should be pending a refresh. %s"), *FReflectionMetaData::GetWidgetPath(this, false, false));
		}

		return nullptr;
	}

	FOnRowHovered OnHovered;
	FOnRowHovered OnUnhovered;
	FOnObjectRowCanAcceptDrop OnRowCanAcceptDrop;
	FOnObjectRowAcceptDrop OnRowAcceptDrop;
	FOnObjectRowDragDetected OnRowDragDetected;
	FOnObjectRowDragEnter OnRowDragEnter;
	FOnObjectRowDragLeave OnRowDragLeave;
	FOnObjectRowDragCancelled OnRowDragCancelled;

	TWeakObjectPtr<UListViewBase> OwnerListView;
	TWeakPtr<ITypedTableView<ItemType>> OwnerTablePtr;

private:
	void InitObjectRowInternal(UUserWidget& ListEntryWidget, ItemType ListItemObject) {}
	
	int32 IndexInList = INDEX_NONE;
	bool bChangedSelectionOnMouseDown = false;
	bool bIsAppearingSelected = false;

	bool bProcessingSelectionTouch = false;

	/** If true, when selecting an item via mouse button, we allow pre-selected items to remain selected */
	bool bAllowKeepPreselectedItems = true;

	/** Whether to allow dragging of this item */
	bool bAllowDragging;

	/** Whether to allow drag drop operations to be performed on the list */
	bool bAllowDragDrop;

	/** Whether or not drag was detected */
	bool bDragWasDetected = false;

	/** Are we currently dragging/dropping over this item? */
	TOptional<EItemDropZone> ItemDropZone;
};

template <>
inline void SObjectTableRow<UObject*>::InitObjectRowInternal(UUserWidget& ListEntryWidget, UObject* ListItemObject)
{
	if (ListEntryWidget.Implements<UUserObjectListEntry>())
	{
		IUserObjectListEntry::SetListItemObject(*WidgetObject, ListItemObject);
	}
}
