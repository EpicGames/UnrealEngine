// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Input/NavigationReply.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "UObject/GCObject.h"
#include "Blueprint/UserWidget.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class FWeakWidgetPath;
class FWidgetPath;
class UDragDropOperation;

template< typename ObjectType > class TAttribute;

/**
 * The SObjectWidget allows UMG to insert an SWidget into the hierarchy that manages the lifetime of the
 * UMG UWidget that created it.  Once the SObjectWidget is destroyed it frees the reference it holds to
 * The UWidget allowing it to be garbage collected.  It also forwards the slate events to the UUserWidget
 * so that it can forward them to listeners.
 */
class SObjectWidget : public SCompoundWidget, public FGCObject
{
	SLATE_DECLARE_WIDGET_API(SObjectWidget, SCompoundWidget, UMG_API)
	SLATE_BEGIN_ARGS(SObjectWidget)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	UMG_API SObjectWidget();
	UMG_API virtual ~SObjectWidget();

	UMG_API void Construct(const FArguments& InArgs, UUserWidget* InWidgetObject);

	UMG_API void ResetWidget();

	// FGCObject interface
	UMG_API virtual FString GetReferencerName() const override;
	UMG_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FGCObject interface

	UUserWidget* GetWidgetObject() const { return WidgetObject; }

	UMG_API void SetPadding(const TAttribute<FMargin>& InMargin);

	/** SWidget Tick override.  Note this will not be called if bCanTick is set to false by the UserWidget */
	UMG_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UMG_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	UMG_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

	UMG_API virtual bool IsInteractable() const override;
	UMG_API virtual bool SupportsKeyboardFocus() const override;

	UMG_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	UMG_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	UMG_API virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override;
	
	UMG_API virtual FReply OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) override;
	UMG_API virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UMG_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UMG_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UMG_API virtual FReply OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent) override;
	
	UMG_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UMG_API virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UMG_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UMG_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UMG_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UMG_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	UMG_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UMG_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	UMG_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	
	UMG_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UMG_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UMG_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	UMG_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UMG_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	UMG_API virtual void OnDragCancelled(const FDragDropEvent& DragDropEvent, UDragDropOperation* Operation);
	
	UMG_API virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent) override;
	UMG_API virtual FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	UMG_API virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	UMG_API virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	UMG_API virtual FReply OnMotionDetected(const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent) override;
	UMG_API virtual FReply OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	UMG_API virtual FReply OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	UMG_API virtual void OnFinishedPointerInput() override;

	UMG_API virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;

	UMG_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
protected:

	/** The UWidget that created this SObjectWidget who needs to be kept alive. */
	TObjectPtr<UUserWidget> WidgetObject;

private:
	inline bool CanRouteEvent() const
	{
		return WidgetObject && WidgetObject->CanSafelyRouteEvent();
	}

	inline bool CanRoutePaint() const
	{
		return WidgetObject && WidgetObject->CanSafelyRoutePaint();
	}

#if SLATE_VERBOSE_NAMED_EVENTS
	FString DebugName;
	FString DebugTickEventName;
	FString DebugPaintEventName;
#endif
};
