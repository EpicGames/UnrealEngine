// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/SlateTypes.h"

class FPaintArgs;
class FSlateWindowElementList;
class FSlateRHIPostBufferProcessorProxy;

/**
 * Custom Slate drawer to update slate post buffer
 * 
 * Note: Declared in .cpp to avoid UMG header dependencies on SlateRHIRenderer on Server
 */
class FPostBufferUpdater;

/**
 * Class that can update a given Slate post buffer processor via its renderthread proxy.
 *
 * This proxy is also deleted on the renderthread
 */
class FSlatePostProcessorUpdaterProxy : public TSharedFromThis<FSlatePostProcessorUpdaterProxy>
{
public:

public:
	
	virtual ~FSlatePostProcessorUpdaterProxy () = default;

	virtual void UpdateProcessor_RenderThread(TSharedPtr<FSlateRHIPostBufferProcessorProxy>) const = 0;
};

/**
 * Implements a widget that triggers a post buffer update on draw
 */
class SPostBufferUpdate
	: public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SPostBufferUpdate, SLeafWidget, UMG_API)

public:
	SLATE_BEGIN_ARGS( SPostBufferUpdate )
		: _bUsePaintGeometry(false)
		{ }

		/** True if we should perform the default post buffer update */
		SLATE_ARGUMENT(bool, bUsePaintGeometry)
	SLATE_END_ARGS()

	/** Constructor */
	UMG_API SPostBufferUpdate();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UMG_API void Construct( const FArguments& InArgs );

public:
	/** Set if we should use paint geometry */
	UMG_API void SetUsePaintGeometry(bool bInUsePaintGeometry);

	/**
	 * Set buffers to, this method only affects the 'FPostBufferUpdater' custom drawer once during initialization. 
	 * Afterwards it will be a no-op due to the custom drawer having update buffers initialized.
	 * This set-once behavior is done to avoid renderthread race conditions, & since you shouldn't need to modify
	 * the buffers to update at runtime since we only draw if used anyway.
	 */
	UMG_API void SetBuffersToUpdate(const TArrayView<ESlatePostRT> InBuffersToUpdate);

	/**
	 * Set processsor updaters for the given buffers by index.
	 */
	UMG_API void SetProcessorUpdaters(TMap<ESlatePostRT, TSharedPtr<FSlatePostProcessorUpdaterProxy>> InProcessorUpdaters);

	/** Get buffers to update */
	UMG_API const TArrayView<const ESlatePostRT> GetBuffersToUpdate() const;

	/** Release Post Buffer Updater, called out of caution in case of reuse during 'ReleaseSlateResources' in UWidget */
	UMG_API void ReleasePostBufferUpdater();

public:

	//~Begin Widget Interface
	UMG_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	//~End Widget Interface

protected:

	//~Begin Widget Interface
	UMG_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	//~End Widget Interface

private:

	/** True if we should limit processing to our paint geometry */
	bool bUsePaintGeometry;

	/** Buffers that we should update, all of these buffers will be affected by 'bPerformDefaultPostBufferUpdate' if disabled */
	TArray<ESlatePostRT> BuffersToUpdate;

	/** Custom drawer used to trigger a post buffer update */
	TSharedPtr<FPostBufferUpdater> PostBufferUpdater;
};
