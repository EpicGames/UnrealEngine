// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Layout/Visibility.h"
#include "Layout/SlateRect.h"
#include "Layout/Geometry.h"
#include "Input/Events.h"
#include "Layout/ArrangedWidget.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateBrush.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Children.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "Input/HittestGrid.h"
#include "Slate/WidgetRenderer.h"
#include "Misc/FrameValue.h"

class FArrangedChildren;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTextureRenderTarget2D;
class FRetainerWidgetRenderingResources;

DECLARE_MULTICAST_DELEGATE( FOnRetainedModeChanged );



/**
 * The SRetainerWidget renders children widgets to a render target first before
 * later rendering that render target to the screen.  This allows both frequency
 * and phase to be controlled so that the UI can actually render less often than the
 * frequency of the main game render.  It also has the side benefit of allow materials
 * to be applied to the render target after drawing the widgets to apply a simple post process.
 */
class SRetainerWidget : public SCompoundWidget,  public FSlateInvalidationRoot
{
public:
	static UMG_API int32 Shared_MaxRetainerWorkPerFrame;

public:
	SLATE_BEGIN_ARGS(SRetainerWidget)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
		_Phase = 0;
		_PhaseCount = 1;
		_RenderOnPhase = true;
		_RenderOnInvalidation = false;
	}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(bool, RenderOnPhase)
		SLATE_ARGUMENT(bool, RenderOnInvalidation)
		SLATE_ARGUMENT_DEPRECATED(bool, RenderWithLocalTransform, 5.4, "This argument has no effect anymore, because bugs from each render path (with RenderWithLocalTransform set to true or false) have been fixed, making it no longer necessary.")
		SLATE_ARGUMENT(int32, Phase)
		SLATE_ARGUMENT(int32, PhaseCount)
		SLATE_ARGUMENT(FName, StatId)
#if WITH_EDITOR
		SLATE_ARGUMENT(bool, bWarnOnInvalidSize)
#endif
		SLATE_END_ARGS()

	UMG_API SRetainerWidget();
	UMG_API ~SRetainerWidget();

	/** Constructor */
	UMG_API void Construct(const FArguments& Args);

	UMG_API void SetRenderingPhase(int32 Phase, int32 PhaseCount);

	/** Requests that the retainer redraw the hosted content next time it's painted. */
	UMG_API void RequestRender();

#if WITH_EDITOR
	UMG_API void SetIsDesignTime(bool bInIsDesignTime);
	UMG_API void SetShowEffectsInDesigner(bool bInShowEffectsInDesigner);
#endif

	UMG_API void SetRetainedRendering(bool bRetainRendering);

	UMG_API void SetContent(const TSharedRef< SWidget >& InContent);

	UMG_API UMaterialInstanceDynamic* GetEffectMaterial() const;

	UMG_API void SetEffectMaterial(UMaterialInterface* EffectMaterial);

	UMG_API void SetTextureParameter(FName TextureParameter);

	//~ SWidget interface
	UMG_API virtual FChildren* GetChildren() override;
#if WITH_SLATE_DEBUGGING
	UMG_API virtual FChildren* Debug_GetChildrenForReflector() override;
#endif

	UMG_API void SetWorld(UWorld* World);

protected:
	/** SCompoundWidget interface */
	UMG_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UMG_API virtual FVector2D ComputeDesiredSize(float Scale) const override;
	virtual bool Advanced_IsInvalidationRoot() const { return bEnableRetainedRendering; }
	virtual const FSlateInvalidationRoot* Advanced_AsInvalidationRoot() const override { return bEnableRetainedRendering ? this : nullptr; }
	UMG_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;

	//~ Begin FSlateInvalidationRoot interface
	UMG_API virtual TSharedRef<SWidget> GetRootWidget() override;
	UMG_API virtual int32 PaintSlowPath(const FSlateInvalidationContext& Context) override;

	enum class EPaintRetainedContentResult
	{
		NotPainted,
		Painted,
		Queued,
		TextureSizeTooBig,
		TextureSizeZero,
	};
	UMG_API EPaintRetainedContentResult PaintRetainedContentImpl(const FSlateInvalidationContext& Context, const FGeometry& AllottedGeometry, int32 LayerId);
	//~ End FSlateInvalidationRoot interface

	UMG_API void RefreshRenderingMode();
	UMG_API bool ShouldBeRenderingOffscreen() const;
	UMG_API bool IsAnythingVisibleToRender() const;
	UMG_API void OnRetainerModeChanged();
	UMG_API void OnRootInvalidated();

private:
	UMG_API void OnGlobalInvalidationToggled(bool bGlobalInvalidationEnabled);
#if !UE_BUILD_SHIPPING
	static UMG_API void OnRetainerModeCVarChanged( IConsoleVariable* CVar );
	static UMG_API FOnRetainedModeChanged OnRetainerModeChangedDelegate;
#endif

	mutable FSlateBrush SurfaceBrush;

	FIntPoint PreviousRenderSize;
	FGeometry PreviousAllottedGeometry;
	FIntPoint PreviousClipRectSize;
	TOptional<FSlateClippingState> PreviousClippingState;
	FColor PreviousColorAndOpacity;
	int32 LastIncomingLayerId;

	UMG_API void UpdateWidgetRenderer();

	TSharedPtr<SWidget> MyWidget;
	TSharedRef<SVirtualWindow> VirtualWindow;

	TSharedRef<FHittestGrid> HittestGrid;

	int32 Phase;
	int32 PhaseCount;

	bool bEnableRetainedRenderingDesire;
	bool bEnableRetainedRendering;

#if WITH_EDITOR
	/** True if widget is used in design time */
	bool bIsDesignTime;

	/** True if we should retain rendering in designer */
	bool bShowEffectsInDesigner;

	/** True if we should warn when the requested size for the retainer is 0 or too large */
	bool bWarnOnInvalidSize;
#endif

	bool RenderOnPhase;
	bool RenderOnInvalidation;

	bool bRenderRequested;
	bool bInvalidSizeLogged;

	double LastDrawTime;
	int64 LastTickedFrame;

	TWeakObjectPtr<UWorld> OuterWorld;

	FRetainerWidgetRenderingResources* RenderingResources;

	STAT(TStatId MyStatId;)

	FSlateBrush DynamicBrush;

	FName DynamicEffectTextureParameter;

	static UMG_API TArray<SRetainerWidget*, TInlineAllocator<3>> Shared_WaitingToRender;
	static UMG_API TFrameValue<int32> Shared_RetainerWorkThisFrame;
};
