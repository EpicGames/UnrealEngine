// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/LocalPlayer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "UObject/ObjectKey.h"

class USceneComponent;

class SWorldWidgetScreenLayer : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWorldWidgetScreenLayer)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const FLocalPlayerContext& InPlayerContext);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	void SetWidgetDrawSize(FVector2D DrawSize);

	void SetWidgetPivot(FVector2D Pivot);

	void AddComponent(USceneComponent* Component, TSharedRef<SWidget> Widget);

	void RemoveComponent(USceneComponent* Component);

private:
	FLocalPlayerContext PlayerContext;

	FVector2D DrawSize;
	FVector2D Pivot;

	class FComponentEntry
	{
	public:
		FComponentEntry();
		~FComponentEntry();

	public:

		bool bRemoving = false;
		TWeakObjectPtr<USceneComponent> Component;
		class UWidgetComponent* WidgetComponent;

		TSharedPtr<SWidget> ContainerWidget;
		TSharedPtr<SWidget> Widget;
		SConstraintCanvas::FSlot* Slot;
	};

	void RemoveEntryFromCanvas(SWorldWidgetScreenLayer::FComponentEntry& Entry);

	TMap<FObjectKey, FComponentEntry> ComponentMap;
	TSharedPtr<SConstraintCanvas> Canvas;
};
