// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SlateVectorArtData.generated.h"

class UMaterialInstanceDynamic;
class UMaterialInterface;
class UStaticMesh;

USTRUCT()
struct FSlateMeshVertex
{
	GENERATED_USTRUCT_BODY()

	static const int32 MaxNumUVs = 6;

	FSlateMeshVertex()
		:
		Position(FVector2f::ZeroVector)
		, Color(ForceInitToZero)
		, UV0(FVector2f::ZeroVector)
		, UV1(FVector2f::ZeroVector)
		, UV2(FVector2f::ZeroVector)
		, UV3(FVector2f::ZeroVector)
		, UV4(FVector2f::ZeroVector)
		, UV5(FVector2f::ZeroVector)
	{
	}

	FSlateMeshVertex(
		  FVector2f InPos
		, FColor InColor
		, FVector2f InUV0
		, FVector2f InUV1
		, FVector2f InUV2
		, FVector2f InUV3
		, FVector2f InUV4
		, FVector2f InUV5
		)
		: Position(InPos)
		, Color(InColor)
		, UV0(InUV0)
		, UV1(InUV1)
		, UV2(InUV2)
		, UV3(InUV3)
		, UV4(InUV4)
		, UV5(InUV5)	
	{
	}

	UPROPERTY()
	FVector2f Position;

	UPROPERTY()
	FColor Color;

	UPROPERTY()
	FVector2f UV0;

	UPROPERTY()
	FVector2f UV1;

	UPROPERTY()
	FVector2f UV2;

	UPROPERTY()
	FVector2f UV3;

	UPROPERTY()
	FVector2f UV4;

	UPROPERTY()
	FVector2f UV5;
};

/**
 * Turn static mesh data into Slate's simple vector art format.
 */
UCLASS(MinimalAPI)
class USlateVectorArtData : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Access the slate vertexes. */
	UMG_API const TArray<FSlateMeshVertex>& GetVertexData() const;
	
	/** Access the indexes for the order in which to draw the vertexes. */
	UMG_API const TArray<uint32>& GetIndexData() const;
	
	/** Material to be used with the specified vector art data. */
	UMG_API UMaterialInterface* GetMaterial() const;
	
	/** Convert the material into an MID and get a pointer to the MID so that parameters can be set on it. */
	UMG_API UMaterialInstanceDynamic* ConvertToMaterialInstanceDynamic();

	/** Convert the static mesh data into slate vector art on demand. Does nothing in a cooked build. */
	UMG_API void EnsureValidData();

	UMG_API FVector2D GetDesiredSize() const;

	UMG_API FVector2D GetExtentMin() const;

	UMG_API FVector2D GetExtentMax() const;

private:
	// ~ UObject Interface
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	// ~ UObject Interface

#if WITH_EDITORONLY_DATA
	/** Does the actual work of converting mesh data into slate vector art */
	void InitFromStaticMesh(const UStaticMesh& InSourceMesh);

	/** The mesh data asset from which the vector art is sourced */
	UPROPERTY(EditAnywhere, Category="Vector Art" )
	TObjectPtr<UStaticMesh> MeshAsset;

	/** The material which we are using, or the material from with the MIC was constructed. */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> SourceMaterial;
#endif

	/** @see GetVertexData() */
	UPROPERTY()
	TArray<FSlateMeshVertex> VertexData;

	/** @see GetIndexData() */
	UPROPERTY()
	TArray<uint32> IndexData;

	/** @see GetMaterial() */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY()
	FVector2D ExtentMin;

	UPROPERTY()
	FVector2D ExtentMax;
};
