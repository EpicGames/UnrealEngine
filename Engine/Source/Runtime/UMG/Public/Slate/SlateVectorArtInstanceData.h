// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"

struct FSlateVectorArtInstanceData
{
public:
	const FVector4& GetData() const { return Data; }
	FVector4& GetData() { return Data; }

	UMG_API void SetPositionFixedPoint16(FVector2D Position);
	UMG_API void SetScaleFixedPoint16(float Scale);

	UMG_API void SetPosition(FVector2D Position);
	UMG_API void SetScale(float Scale);
	UMG_API void SetBaseAddress(float Address);

	template<int32 Component, int32 ByteIndex>
	void PackFloatIntoByte(float InValue)
	{
		PackByteIntoByte<Component, ByteIndex>(static_cast<uint8>(FMath::RoundToInt(InValue * 0xFFu)));
	}

	template<int32 Component, int32 ByteIndex>
	void PackByteIntoByte(uint8 InValue)
	{
		// Each float has 24 usable bits of mantissa, but we cannot access the bits directly.
		// We will not respect the IEEE "normalized mantissa" rules, so let 
		// the compiler/FPU do conversions from byte to float and vice versa for us.

		//Produces a value like 0xFFFF00FF; where the 00s are shifted by `Position` bytes.
		static const uint32 Mask = ~(0xFF << ByteIndex * 8);

		// Clear out a byte to OR with while maintaining the rest of the data intact.
		uint32 CurrentData = static_cast<uint32>(Data[Component]) & Mask;
		// OR in the value
		Data[Component] = CurrentData | (InValue << ByteIndex * 8);
	}

	template<int32 Component, int32 ByteIndex>
	void PackByteIntoByte(float InValue)
	{
		checkf(false, TEXT("PackByteIntoByte() being used with a float value. This is almost definitely an error. Use PackFloatIntoByte()."));
	}

protected:
	// LWC_TODO: should this and GetData() use FVector4f?
	FVector4 Data;
};
