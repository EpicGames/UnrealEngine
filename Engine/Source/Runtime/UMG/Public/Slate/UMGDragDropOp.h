// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/DragAndDrop.h"
#include "UObject/GCObject.h"

class SObjectWidget;
class SWidget;
class UDragDropOperation;
class UGameViewportClient;

/**
 * This is the drag/drop class used for UMG, all UMG drag drop operations utilize this operation.
 * It supports moving a UObject payload and using a UWidget decorator.
 */
class FUMGDragDropOp : public FGameDragDropOperation, public FGCObject
{
public:
	DRAG_DROP_OPERATOR_TYPE(FUMGDragDropOp, FGameDragDropOperation)
	
	UMG_API static TSharedRef<FUMGDragDropOp> New(UDragDropOperation* Operation, const int32 PointerIndex, const FVector2D &CursorPosition, const FVector2D &ScreenPositionOfNode, float DPIScale, TSharedPtr<SObjectWidget> SourceUserWidget);

	FUMGDragDropOp();

	// Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	// End FGCObject

	virtual bool AffectedByPointerEvent(const FPointerEvent& PointerEvent) override;
	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override;
	virtual void OnDragged( const class FDragDropEvent& DragDropEvent ) override;
	virtual FCursorReply OnCursorQuery() override;
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	UDragDropOperation* GetOperation() const { return DragOperation; }

protected:
	virtual void Construct() override;

private:

	// Raw pointer to the drag operation, kept alive by AddReferencedObjects.
	TObjectPtr<UDragDropOperation> DragOperation;

	/** Source User Widget */
	TWeakPtr<SObjectWidget> SourceUserWidget;

	/** The viewport this drag/drop operation is associated with. */
	TWeakObjectPtr<UGameViewportClient> GameViewport;

	/** The widget used during the drag/drop action to show something being dragged. */
	TSharedPtr<SWidget> DecoratorWidget;

	/** The offset to use when dragging the object so that it says the same distance away from the mouse. */
	FVector2D MouseDownOffset;

	/** The starting screen location where the drag operation started. */
	FVector2D StartingScreenPos;

	int32 PointerIndex;

	/** Allows smooth interpolation of the dragged visual over a few frames. */
	double StartTime;
};
