// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/SlateRect.h"
#include "Layout/Geometry.h"
#include "Input/PopupMethodReply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Rendering/DrawElements.h"
#include "Engine/Texture.h"
#include "RenderDeferredCleanup.h"
#include "Widgets/SWindow.h"
#include "Widgets/SVirtualWindow.h"

class FArrangedChildren;
class FHittestGrid;
class FSlateDrawBuffer;
class ISlate3DRenderer;
class STooltipPresenter;
class UTextureRenderTarget2D;
class FRenderTarget;

/**
 * 
 */
class FWidgetRenderer : public FDeferredCleanupInterface
{
public:
	UMG_API FWidgetRenderer(bool bUseGammaCorrection = false, bool bInClearTarget = true);
	UMG_API ~FWidgetRenderer();

	bool GetIsPrepassNeeded() const { return bPrepassNeeded; }
	void SetIsPrepassNeeded(bool bInPrepassNeeded) { bPrepassNeeded = bInPrepassNeeded; }

	bool GetClearHitTestGrid() const { return bClearHitTestGrid; }
	void SetClearHitTestGrid(bool bInClearHitTestGrid) { bClearHitTestGrid = bInClearHitTestGrid; }

	void SetShouldClearTarget(bool bShouldClear) { bClearTarget = bShouldClear; }

	bool GetUseGammaCorrection() const { return bUseGammaSpace; }
	UMG_API void SetUseGammaCorrection(bool bInUseGammaSpace);

	UMG_API void SetApplyColorDeficiencyCorrection(bool bInApplyColorCorrection);

	UMG_API ISlate3DRenderer* GetSlateRenderer();

	static UMG_API UTextureRenderTarget2D* CreateTargetFor(FVector2D DrawSize, TextureFilter InFilter, bool bUseGammaCorrection);

	UMG_API UTextureRenderTarget2D* DrawWidget(const TSharedRef<SWidget>& Widget, FVector2D DrawSize);

	UMG_API void DrawWidget(
		FRenderTarget* RenderTarget,
		const TSharedRef<SWidget>& Widget,
		FVector2D DrawSize,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWidget(
		UTextureRenderTarget2D* RenderTarget,
		const TSharedRef<SWidget>& Widget,
		FVector2D DrawSize,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWidget(
		FRenderTarget* RenderTarget,
		const TSharedRef<SWidget>& Widget,
		float Scale,
		FVector2D DrawSize,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWidget(
		UTextureRenderTarget2D* RenderTarget,
		const TSharedRef<SWidget>& Widget,
		float Scale,
		FVector2D DrawSize,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWindow(
		FRenderTarget* RenderTarget,
		FHittestGrid& HitTestGrid,
		TSharedRef<SWindow> Window,
		float Scale,
		FVector2D DrawSize,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWindow(
		UTextureRenderTarget2D* RenderTarget,
		FHittestGrid& HitTestGrid,
		TSharedRef<SWindow> Window,
		float Scale,
		FVector2D DrawSize,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWindow(
		FRenderTarget* RenderTarget,
		FHittestGrid& HitTestGrid,
		TSharedRef<SWindow> Window,
		FGeometry WindowGeometry,
		FSlateRect WindowClipRect,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWindow(
		UTextureRenderTarget2D* RenderTarget,
		FHittestGrid& HitTestGrid,
		TSharedRef<SWindow> Window,
		FGeometry WindowGeometry,
		FSlateRect WindowClipRect,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWindow(
		const FPaintArgs& PaintArgs,
		FRenderTarget* RenderTarget,
		TSharedRef<SWindow> Window,
		FGeometry WindowGeometry,
		FSlateRect WindowClipRect,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API void DrawWindow(
		const FPaintArgs& PaintArgs,
		UTextureRenderTarget2D* RenderTarget,
		TSharedRef<SWindow> Window,
		FGeometry WindowGeometry,
		FSlateRect WindowClipRect,
		float DeltaTime,
		bool bDeferRenderTargetUpdate = false);

	UMG_API bool DrawInvalidationRoot(
		TSharedRef<SVirtualWindow>& VirtualWindow,
		UTextureRenderTarget2D* RenderTarget,
		FSlateInvalidationRoot& Root,
		const FSlateInvalidationContext& Context,
		bool bDeferRenderTargetUpdate = false);

	UMG_API bool DrawInvalidationRoot(
		TSharedRef<SVirtualWindow>& VirtualWindow,
		UTextureRenderTarget2D* RenderTarget,
		FPaintArgs PaintArgs,
		float DrawScale,
		FVector2D DrawSize,
		bool bDeferRenderTargetUpdate = false);

	TArray< TSharedPtr<FSlateWindowElementList::FDeferredPaint> > DeferredPaints;
private:
	/** The slate 3D renderer used to render the user slate widget */
	TSharedPtr<ISlate3DRenderer, ESPMode::ThreadSafe> Renderer;
	/** Prepass Needed when drawing the widget? */
	bool bPrepassNeeded;
	/** Clearing hit test grid needed? */
	bool bClearHitTestGrid;
	/** Is gamma space needed? */
	bool bUseGammaSpace;
	/** Should we clear the render target before rendering. */
	bool bClearTarget;
public:
	FVector2D ViewOffset;
};