// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Math/TransformCalculus2D.h"
#include "Rendering/SlateRenderTransform.h"
#include "WidgetTransform.generated.h"

/**
 * Describes the standard transformation of a widget
 */
USTRUCT(BlueprintType)
struct FWidgetTransform
{
	GENERATED_USTRUCT_BODY()

public:

	/** The amount to translate the widget in slate units */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transform, meta=( Delta = "1" ))
	FVector2D Translation;

	/** The scale to apply to the widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transform, meta=( UIMin = "-5", UIMax = "5", Delta = "0.05" ))
	FVector2D Scale;

	/** The amount to shear the widget in slate units */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transform, meta=( UIMin = "-89", ClampMin = "-89", UIMax = "89", ClampMax = "89", Delta = "1" ))
	FVector2D Shear;

	/** The angle in degrees to rotate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transform, meta=( UIMin = "-180", UIMax = "180", Delta = "1" ))
	float Angle;

public:

	FWidgetTransform()
		: Translation(FVector2D::ZeroVector)
		, Scale(FVector2D::UnitVector)
		, Shear(FVector2D::ZeroVector)
		, Angle(0)
	{
	}

	FWidgetTransform(const FVector2D& InTranslation, const FVector2D& InScale, const FVector2D& InShear, float InAngle)
		: Translation(InTranslation)
		, Scale(InScale)
		, Shear(InShear)
		, Angle(InAngle)
	{
	}

	bool IsIdentity() const
	{
		const static FWidgetTransform Identity;

		return Identity == *this;
	}

	inline bool operator==( const FWidgetTransform &Other ) const
	{
		return Scale == Other.Scale && Shear == Other.Shear && Angle == Other.Angle && Translation == Other.Translation;
	}

	inline bool operator!=( const FWidgetTransform& Other ) const
	{
		return !( *this == Other );
	}

	inline FSlateRenderTransform ToSlateRenderTransform() const
	{
		return ::Concatenate(
			FScale2D(Scale),
			FShear2D::FromShearAngles(Shear),
			FQuat2D(FMath::DegreesToRadians(Angle)),
			FVector2D(Translation));
	}
};

