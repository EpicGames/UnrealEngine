// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Public Dependencies
 *****************************************************************************/

#include "Core.h"
#include "CoreUObject.h"
#include "Engine.h"
#include "InputCore.h"
#include "SlateBasics.h"
#include "SlateCore.h"

#include "Widgets/Layout/Anchors.h"
#include "Widgets/Layout/SConstraintCanvas.h"

#include "Slate/SlateBrushAsset.h"

/* Public Includes
 *****************************************************************************/

// Slate Types
#include "Components/SlateWrapperTypes.h"

// Binding
#include "Binding/DynamicPropertyPath.h"
#include "Binding/PropertyBinding.h"

// Components
#include "Blueprint/DragDropOperation.h"

#include "Components/Visual.h"

#include "Components/PanelSlot.h"

#include "Blueprint/WidgetNavigation.h"
#include "Slate/WidgetTransform.h"
#include "Components/Widget.h"

#include "Components/PanelWidget.h"
#include "Components/ContentWidget.h"

#include "Components/NamedSlot.h"
#include "Components/NamedSlotInterface.h"

#include "Components/CanvasPanelSlot.h"
#include "Components/CanvasPanel.h"

#include "Components/HorizontalBoxSlot.h"
#include "Components/HorizontalBox.h"

#include "Components/VerticalBoxSlot.h"
#include "Components/VerticalBox.h"

#include "Components/WrapBoxSlot.h"
#include "Components/WrapBox.h"

#include "Components/UniformGridSlot.h"
#include "Components/UniformGridPanel.h"

#include "Components/GridSlot.h"
#include "Components/GridPanel.h"

#include "Components/OverlaySlot.h"
#include "Components/Overlay.h"

#include "Components/ScrollBoxSlot.h"
#include "Components/ScrollBox.h"

#include "Components/WidgetSwitcherSlot.h"
#include "Components/WidgetSwitcher.h"

#include "Components/BackgroundBlurSlot.h"
#include "Components/BackgroundBlur.h"

#include "Components/BorderSlot.h"
#include "Components/Border.h"

#include "Components/ButtonSlot.h"
#include "Components/Button.h"

#include "Components/SizeBoxSlot.h"
#include "Components/SizeBox.h"

#include "Components/TextWidgetTypes.h"
#include "Components/TextBlock.h"

#include "Components/CheckBox.h"
#include "Components/Image.h"

#include "Components/SpinBox.h"

#include "Components/EditableText.h"
#include "Components/EditableTextBox.h"

#include "Components/MultiLineEditableText.h"
#include "Components/MultiLineEditableTextBox.h"

#include "Components/Slider.h"
#include "Components/ProgressBar.h"

#include "Components/ComboBox.h"
#include "Components/ComboBoxString.h"

#include "Widgets/Layout/SScaleBox.h"
#include "Components/ScaleBoxSlot.h"
#include "Components/ScaleBox.h"

#include "Components/ExpandableArea.h"

#include "Components/Spacer.h"
#include "Components/MenuAnchor.h"
#include "Components/ScrollBar.h"

#include "Components/Throbber.h"
#include "Components/CircularThrobber.h"

#include "Components/ListView.h"
#include "Components/TileView.h"
#include "Components/TreeView.h"

#include "Components/Viewport.h"

#include "Components/NativeWidgetHost.h"

#include "Components/InputKeySelector.h"

#include "Components/WindowTitleBarArea.h"
#include "Components/WindowTitleBarAreaSlot.h"

// Slate
#include "Slate/SObjectWidget.h"

// Blueprint
#include "Blueprint/WidgetTree.h"
#include "Blueprint/UserWidget.h"

#include "Blueprint/WidgetBlueprintGeneratedClass.h"

// Interfaces
#include "IUMGModule.h"

// Animation 
#include "Animation/MovieScene2DTransformSection.h"
#include "Animation/MovieScene2DTransformTrack.h"
#include "Animation/MovieSceneMarginSection.h"
#include "Animation/MovieSceneMarginTrack.h"
#include "Animation/UMGSequencePlayer.h"
#include "Animation/WidgetAnimation.h"
#include "Animation/WidgetAnimationBinding.h"
