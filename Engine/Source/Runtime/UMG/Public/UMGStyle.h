// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

/**  */
class FUMGStyle
{
public:

	static UMG_API void Initialize();

	static UMG_API void Shutdown();

	/** reloads textures used by slate renderer */
	static UMG_API void ReloadTextures();

	/** @return The Slate style set for the UMG Style */
	static UMG_API const ISlateStyle& Get();

	static UMG_API FName GetStyleSetName();

private:

	static TSharedRef< class FSlateStyleSet > Create();

private:

	static TSharedPtr< class FSlateStyleSet > UMGStyleInstance;
};
