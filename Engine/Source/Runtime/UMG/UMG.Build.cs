// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UMG : ModuleRules
{
	public UMG(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"TraceLog",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"RenderCore",
				"Renderer",
				"RHI",
				"ApplicationCore"
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"FieldNotification",
				"HTTP",
				"MovieScene",
				"MovieSceneTracks",
				"PropertyPath",
				"TimeManagement"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"SlateRHIRenderer",
				"ImageWrapper",
				"TargetPlatform",
			}
		);

		if (Target.Type != TargetType.Server)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"ImageWrapper",
				}
			);

			// Implementations depending on 'SlateRHIRenderer' should 
			// be guarded with: '#if !UE_SERVER'.
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SlateRHIRenderer",
				}
			);
		}
	}
}
